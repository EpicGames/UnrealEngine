// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class AssetRegistry : ModuleRules
{
	public AssetRegistry(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"TraceLog",
				"ApplicationCore",
				"Projects",
				"TelemetryUtils",
				"PakFile"
			}
			);

		if (Target.bBuildEditor == true)
		{
			PrivateIncludePathModuleNames.AddRange(new string[] { "DirectoryWatcher", "TargetPlatform" });
			DynamicallyLoadedModuleNames.AddRange(new string[] { "DirectoryWatcher", "TargetPlatform" });
		}

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		PublicDebugVisualizerPaths.Add(Path.Combine(ModuleDirectory, "AssetRegistry.natvis"));
	}
}
