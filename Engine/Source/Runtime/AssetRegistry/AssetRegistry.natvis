<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">
  <Type Name="VersePathRegistry::FVersePathRegistry">
    <DisplayString>PathDatas={PathDatas.ArrayNum}, Roots={Roots._Num()}</DisplayString>
    <Expand>
      <Synthetic Name="Roots">
        <DisplayString>Num={Roots._Num()}</DisplayString>
        <Expand>
          <CustomListItems MaxItemsPerView="256">
            <Variable Name="i" InitialValue="1" />
            <Variable Name="Count" InitialValue="Roots._Num()" />
            <Loop Condition="Count > i">
              <Item Name="[{((VersePathRegistry::FPathData*)PathDatas._GetData())[*Roots._GetElementPtr(i)].Path}]">
                ((VersePathRegistry::FPathData*)PathDatas._GetData())[*Roots._GetElementPtr(i)]
              </Item>
              <Exec>++i</Exec>
            </Loop>
          </CustomListItems>
        </Expand>
      </Synthetic>
      <Synthetic Name="ParentToChildren">
        <DisplayString>Num={ParentToChildren._Num()}</DisplayString>
        <Expand>
          <CustomListItems MaxItemsPerView="256">
            <Variable Name="i" InitialValue="0" />
            <Variable Name="Count" InitialValue="ParentToChildren._Num()" />
            <Size>ParentToChildren._Num()</Size>
            <Loop Condition="Count > i">
              <Item Name="[{((VersePathRegistry::FPathData*)PathDatas._GetData())[ParentToChildren._GetElementPtr(i)->Key].Path}]">
                ParentToChildren._GetElementPtr(i)->Value
              </Item>
              <Exec>++i</Exec>
            </Loop>
          </CustomListItems>
        </Expand>
      </Synthetic>
      <Item Name="PathDatas">PathDatas</Item>
    </Expand>
  </Type>

</AutoVisualizer>