// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EnumClassFlags.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/ObjectResource.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/SoftObjectPath.h"

struct FAssetData;
class FAssetPackageData;
class FLinkerTables;
class FMetaData;
class FPackageDependencyData;
class FWorldTileInfo;
namespace UE { class FPackageTrailer; }
namespace UE::AssetRegistry { enum class EExtraDependencyFlags : uint32; }
namespace UE::Serialization::Private { class FImportTypeHierarchy; }
struct FGatherableTextData;
struct FObjectFullNameAndThumbnail;


class FPackageReader : public FArchiveUObject
{
public:
	ASSETREGISTRY_API FPackageReader();
	ASSETREGISTRY_API ~FPackageReader();

	// Note: Keep up to date with LexToString implementation
	enum class EOpenPackageResult : uint8
	{
		/** The package summary loaded successfully */
		Success,
		/** The package reader was not given a valid archive to load from */
		NoLoader,
		/** The package tag could not be found, the package is probably corrupted */
		MalformedTag,
		/** The package is too old to be loaded */
		VersionTooOld,
		/** The package is from a newer version of the engine */
		VersionTooNew,
		/** The package contains an unknown custom version */
		CustomVersionMissing,
		/** The package contains a custom version that failed it's validator */
		CustomVersionInvalid,
		/** Package was unversioned but the process cannot load unversioned packages */
		Unversioned,
	};

	/** Information provided by PackageReader about a parsed Import or Export. */
	struct FObjectData
	{
		FSoftObjectPath ClassPath;
		bool bUsedInGame = true;
	};

	/** Creates a loader for the filename */
	ASSETREGISTRY_API bool OpenPackageFile(FStringView PackageFilename, EOpenPackageResult* OutErrorCode = nullptr);
	ASSETREGISTRY_API bool OpenPackageFile(FStringView LongPackageName, FStringView PackageFilename,
		EOpenPackageResult* OutErrorCode = nullptr);
	ASSETREGISTRY_API bool OpenPackageFile(FArchive* Loader, FStringView LongPackageName = FStringView(), 
		EOpenPackageResult* OutErrorCode = nullptr);
	ASSETREGISTRY_API bool OpenPackageFile(TUniquePtr<FArchive> Loader, EOpenPackageResult* OutErrorCode = nullptr);
	ASSETREGISTRY_API bool OpenPackageFile(EOpenPackageResult& OutErrorCode);

	/**
	 * Returns the LongPackageName from constructor if provided, otherwise calculates it from
	 * FPackageName::TryConvertFilenameToLongPackageName.
	 */
	ASSETREGISTRY_API bool TryGetLongPackageName(FString& OutLongPackageName) const;
	/** Returns the LongPackageName as in TryGetPackageName; asserts if not found. */
	ASSETREGISTRY_API FString GetLongPackageName() const;

	/** Returns the Summary that was ready by OpenPackageFile. */
	ASSETREGISTRY_API const FPackageFileSummary& GetPackageFileSummary() const;
	/** Reads Names if not already read and returns a copy. */
	ASSETREGISTRY_API bool GetNames(TArray<FName>& OutNames);
	/** Reads Imports if not already read and returns a copy. */
	ASSETREGISTRY_API bool GetImports(TArray<FObjectImport>& OutImportMap);
	/** Reads Exports if not already read and returns a copy. */
	ASSETREGISTRY_API bool GetExports(TArray<FObjectExport>& OutExportMap);
	/** Reads DependsMap if not already read and returns a copy. */
	ASSETREGISTRY_API bool GetDependsMap(TArray<TArray<FPackageIndex>>& OutDependsMap);
	/** Reads the list of SoftPackageDependencies if not already read and returns a copy. */
	ASSETREGISTRY_API bool GetSoftPackageReferenceList(TArray<FName>& OutSoftPackageReferenceList);
	/** Reads the list of SoftObjectPaths if not already read and returns a copy. */
	ASSETREGISTRY_API bool GetSoftObjectPaths(TArray<FSoftObjectPath>& OutSoftObjectPaths);
	/** Reads the EditorOnly flags for Imports and Exports and returns them. not cached, reparsed each call. */
	ASSETREGISTRY_API bool ReadEditorOnlyFlags(TBitArray<>& OutImportUsedInGame, TBitArray<>& OutSoftPackageUsedInGame);
	/** Reads the list of GatherableTextData if not already read and returns a copy. */
	ASSETREGISTRY_API bool GetGatherableTextData(TArray<FGatherableTextData>& OutText);
	/** Reads the list of thumbnails if not already read and returns a copy. */
	ASSETREGISTRY_API bool GetThumbnails(TArray<FObjectFullNameAndThumbnail>& OutThumbnails);

	/** Reads the searchable names and returns them. Not cached, reparsed each call. */
	ASSETREGISTRY_API bool ReadSearchableNames(TMap<FTopLevelAssetPath, TArray<FName>>& OutSearchableNames);
	/** Reads information from the AR section and converts it to FAssetData. Not cached, reparsed each call. */
	ASSETREGISTRY_API bool ReadAssetRegistryData(TArray<FAssetData*>& AssetDataList, bool& bOutIsCookedWithoutAssetData);

#if WITH_METADATA
	/** 
	 * Reads information from the metadata section. Not cached, reparsed each call. 
	 * Note that metadata redirects are not applied here, what's in the file is returned as-is.
	 */
	ASSETREGISTRY_API bool ReadMetaData(FMetaData& OutMetaData);
#endif // WITH_METADATA

	/** Reads Imports if not already read and parses the names of imported classes out of the Imports. */
	ASSETREGISTRY_API bool ReadImportedClasses(TArray<FName>& OutClassNames);
	
	/** Reads import type hierarchy information. Not cached, reparsed each call. */ 
	ASSETREGISTRY_API bool ReadImportTypeHierarchies(TMap<FSoftObjectPath, UE::Serialization::Private::FImportTypeHierarchy>& OutTypeHierarchies);

	/** Reads VCell exports. Not cached, reparsed each call. */
	ASSETREGISTRY_API bool ReadCellExports(TArray<FCellExport>& OutExports);

	/** Reads VCell imports. Not cached, reparsed each call. */
	ASSETREGISTRY_API bool ReadCellImports(TArray<FCellImport>& OutImports);

	/** Reads FWorldTileInfo. Not cached, reparsed each call. */
	ASSETREGISTRY_API bool ReadWorldTileInfo(FWorldTileInfo& OutInfo);
	
	/** Reads the package trailer. Not cached, reparsed each call. */
	ASSETREGISTRY_API bool ReadPackageTrailer(UE::FPackageTrailer& OutTrailer);

	/** Reads the data resource map. Parses export indices in FObjectDataResource into FSoftObjectPath. Not cached, reparsed each call. */
	ASSETREGISTRY_API bool ReadDataResourceMap(TArray<TTuple<FSoftObjectPath, FObjectDataResource>>& OutDataResources);

	/**
	 * Reads LinkerTables if not already read, reads the Editor-Only flags (not cached, reparsed each call), 
	 * And constructs a Map from ObjectPath -> ObjectData for each import, export, and SoftPackageReference.
	 * @param OutExports[SoftObjectPath].Value == FObjectData
	 * @param OutImports[SoftObjectPath].Value == FObjectData
	 * @param OutSoftPackageReferences[PackageName].Value == bUsedInGame
	 */
	ASSETREGISTRY_API bool ReadLinkerObjects(TMap<FSoftObjectPath, FObjectData>& OutExports,
		TMap<FSoftObjectPath, FObjectData>& OutImports, TMap<FName, bool>& OutSoftPackageReferences);

	/** Options for what to read in functions that read multiple things at once. */
	enum class EReadOptions
	{
		None = 0,
		PackageData = 1 << 0,
		Dependencies = 1 << 1,
		Default = PackageData | Dependencies,
	};
	/** Reads information used by the dependency graph. Not cached, reparsed each call. */
	ASSETREGISTRY_API bool ReadDependencyData(FPackageDependencyData& OutDependencyData, EReadOptions Options);

	// Farchive implementation to redirect requests to the Loader
	ASSETREGISTRY_API virtual void Serialize( void* V, int64 Length ) override;
	ASSETREGISTRY_API virtual bool Precache( int64 PrecacheOffset, int64 PrecacheSize ) override;
	ASSETREGISTRY_API virtual void Seek( int64 InPos ) override;
	ASSETREGISTRY_API virtual int64 Tell() override;
	ASSETREGISTRY_API virtual int64 TotalSize() override;
	ASSETREGISTRY_API virtual FArchive& operator<<( FName& Name ) override;
	ASSETREGISTRY_API virtual FArchive& operator<<(FSoftObjectPath& Name) override;
	virtual FString GetArchiveName() const override
	{
		return PackageFilename;
	}

private:
	/** Attempts to get object class name from the thumbnail cache for packages older than VER_UE4_ASSET_REGISTRY_TAGS */
	bool ReadAssetDataFromThumbnailCache(TArray<FAssetData*>& AssetDataList);
	/** Creates asset data reconstructing all the required data from cooked package info */
	bool ReadAssetRegistryDataFromCookedPackage(TArray<FAssetData*>& AssetDataList, bool& bOutIsCookedWithoutAssetData);
	bool StartSerializeSection(int64 Offset);

	/** Serializers for different package maps */
	bool SerializeNameMap();
	bool SerializeImportMap();
	bool SerializeExportMap();
	bool SerializeDependsMap();
	bool SerializeImportedClasses(const TArray<FObjectImport>& InImportMap, TArray<FName>& OutClassNames);
	bool SerializeSoftPackageReferenceList();
	bool SerializeSoftObjectPathMap();
	bool SerializeGatherableTextDataMap();
	bool SerializeThumbnailMap();
	bool SerializeEditorOnlyFlags(TBitArray<>& OutImportUsedInGame, TBitArray<>& OutSoftPackageUsedInGame);
	bool SerializeSearchableNamesMap(FLinkerTables& OutSearchableNames);
	bool SerializeAssetRegistryDependencyData(TBitArray<>& OutImportUsedInGame, TBitArray<>& OutSoftPackageUsedInGame,
		TArray<TPair<FName, UE::AssetRegistry::EExtraDependencyFlags>>& OutExtraPackageDependencies);
	bool SerializePackageTrailer(FAssetPackageData& PackageData);

	/** 
	 * Convert an index into the linker tables to a full object path. Requires ImportMap and ExportMap to have been serialized.
	 * Less efficient than ConvertLinkerTableToPaths.
	 */
	TOptional<FSoftObjectPath> LinkerIndexToSoftObjectPath(FPackageIndex Index);

	void ApplyRelocationToImportMapAndSoftPackageReferenceList(FStringView LoadedPackageName,
		TArray<FName>& InOutSoftPackageReferenceList, 
		TArray<TPair<FName, UE::AssetRegistry::EExtraDependencyFlags>>& InOutExtraPackageDependencies);
	static void ConvertLinkerTableToPaths(FName PackageName, TArray<FObjectExport>& ExportMap,
		TArray<FObjectImport>& ImportMap, TArray<FSoftObjectPath>& OutExports, TArray<FSoftObjectPath>& OutImports);

	/** Returns flags the asset package was saved with */
	uint32 GetPackageFlags() const;

	FString LongPackageName;
	FString PackageFilename;
	/*
	 * Loader is the interface used to read the bytes from the package's repository. All interpretation of the bytes is
	 * done by serializing into *this, which is also an FArchive.
	 */
	FArchive* Loader = nullptr;
	FPackageFileSummary PackageFileSummary;
	TArray<FName> NameMap;
	TArray<FObjectImport> ImportMap;
	TArray<FObjectExport> ExportMap;
	TArray<TArray<FPackageIndex>> DependsMap;
	TArray<FName> SoftPackageReferenceList;
	TArray<FSoftObjectPath> SoftObjectPathMap;
	TArray<FGatherableTextData> GatherableTextDataMap;
	TArray<FObjectFullNameAndThumbnail> ThumbnailMap;
	int64 PackageFileSize = 0;
	int64 AssetRegistryDependencyDataOffset = INDEX_NONE;
	// Defined as uint32 to avoid including AssetData.h in this header
	uint32 AssetRegistryVersion = static_cast<uint32>(~0);
	bool bLoaderOwner = false;
};
ENUM_CLASS_FLAGS(FPackageReader::EReadOptions);

ASSETREGISTRY_API const TCHAR* LexToString(FPackageReader::EOpenPackageResult Result);
