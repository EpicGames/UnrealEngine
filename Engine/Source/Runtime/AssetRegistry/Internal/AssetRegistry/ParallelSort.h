// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/RingBuffer.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Tasks/Task.h"
#include "Templates/Less.h"
#include "Templates/UnrealTemplate.h"

namespace Algo
{

/**
 * Sort a range of elements using a user-defined predicate class.  The sort is stable.
 * The sort operation will be distributed to task threads using UE::Tasks::Launch, and the Predicate
 * will be called on elements in the input range in parallel (possibly even the same element being
 * read from multiple threads at the same time); the Predicate must be thread safe.
 *
 * bool Predicate(const T& A, const T& B) -> true if A < B.
 * 
 * @param  Range      The range to sort.
 * @param  Predicate  A binary predicate object used to specify if one element should precede another.
*/
template <typename RangeType, typename PredicateType>
void ParallelSort(RangeType&& Range, PredicateType Predicate);

/** Debug version of ParallelSort that shares implementation but is single threaded. */
template <typename RangeType, typename PredicateType>
void ParallelSortForceSingleThreaded(RangeType&& Range, PredicateType Predicate);

} // namespace Algo


///////////////////////////////////////////////////////
// Implementation forward declares
///////////////////////////////////////////////////////

namespace Algo::MergeSort
{
template <typename T>
struct TMergeBuffer
{
	TRingBuffer<T> Buffer;
	int32 AllocatedSize = 0;
	int32 ConstructorSite = 0;
	int32 InputNum = 0;
	int32 BatchSize = 0;
	int32 Round = 0;
};
template <typename T, typename IndexType, typename PredicateType>
void ParallelMergeSortPointerAndNum(T* First, IndexType Num, PredicateType Predicate);
template <typename T, typename IndexType, typename PredicateType>
void MergeSortPointerAndNum(T* First, IndexType Num, PredicateType Predicate);
template <typename T, typename IndexType, typename PredicateType>
void MergeSortRecursive(T* First, IndexType Num, PredicateType Predicate, TMergeBuffer<T>& MergeBuffer);
template <typename T, typename IndexType, typename PredicateType>
void Merge(T* First, IndexType Num, IndexType FloorMidIndex, PredicateType Predicate, TMergeBuffer<T>& MergeBuffer);
template <typename T, typename IndexType, typename PredicateType>
void SmallSort(T* First, IndexType Num, PredicateType Predicate);

} // namespace Algo::MergeSort


///////////////////////////////////////////////////////
// Template implementations
///////////////////////////////////////////////////////


namespace Algo
{

template <typename RangeType, typename PredicateType>
void ParallelSort(RangeType&& Range, PredicateType Predicate)
{
	Algo::MergeSort::ParallelMergeSortPointerAndNum(GetData(Range), GetNum(Range), MoveTemp(Predicate));
}

template <typename RangeType>
void ParallelSort(RangeType&& Range)
{
	Algo::MergeSort::ParallelMergeSortPointerAndNum(GetData(Range), GetNum(Range), TLess<>());
}

template <typename RangeType, typename PredicateType>
void ParallelSortForceSingleThreaded(RangeType&& Range, PredicateType Predicate)
{
	Algo::MergeSort::MergeSortPointerAndNum(GetData(Range), GetNum(Range), MoveTemp(Predicate));
}

template <typename RangeType, typename PredicateType>
void ParallelSortForceSingleThreaded(RangeType&& Range)
{
	Algo::MergeSort::MergeSortPointerAndNum(GetData(Range), GetNum(Range), TLess<>());
}

} // namespace Algo

namespace Algo::MergeSort
{

template <typename T, typename IndexType, typename PredicateType>
void ParallelMergeSortPointerAndNum(T* First, IndexType Num, PredicateType Predicate)
{
	using namespace UE::Tasks;

	// In Round 0 we divide the Array up initially into TaskCount segments, and create an initial set of tasks that
	// recursively mergesorts all segments in parallel.

	// Rounds 1 through NumRounds-1 merge each pair of recursively sorted segments from the previous round into a
	// combined segment, ending with NumRound -1 merging up into a single segment for the entire range.

	// We require TaskCount is a Power of 2
	// Set MaxAsymptoticTaskCount somewhat arbitrarily at 16 subtasks. There are diminishing returns to further division;
	// total time is ~ Sum(i=1,log(threadCount))(N/i) + N/(log(threadCount+1)) log (N/(log(threadCount+1)
	// e.g. for  8 threads and N = 2^20, it is N + N/2 + N/4 + N/8 + N/16 log(2^20/2^4) == 2*N - 2*N/16 + (N/16)*16 ~ 2.875*N
	//      for 16 threads and N = 2^20, it is N + N/2 + N/4 + N/8 + N/16 + N/32 log(2^20/2^5) == 2*N - 2*N/32 + (N/32)*15 ~ 2.40*N
	// Approaches time == 2*N in the limit of Number of threads == N
	//
	// Depending on N, MaxTaskCount might be less than MaxAsymptoticTaskCount because we want to avoid the overhead of
	// launching threads to sort a small number of elements, so we set a MinBatchSize.
	// TODO: Find some objective way to set MinBatchSize.
	constexpr int32 MinBatchSize = 16;
	constexpr int32 MaxAsymptoticTaskCount = 16;

	// MinBatchSize = DivAndRoundUp(Num, MaxTaskCount) == (Num + MaxTaskCount - 1) / MaxTaskCount;
	// ==>
	// MaxTaskCount == DivAndRoundDown(Num - 1, MinBatchSize - 1)
	int32 MaxTaskCount = (Num - 1) / (MinBatchSize - 1);
	if (MaxTaskCount < 2)
	{
		return Algo::MergeSort::MergeSortPointerAndNum(First, Num, Predicate);
	}
	MaxTaskCount = FMath::Min(MaxTaskCount, MaxAsymptoticTaskCount);
	int32 TaskCount = 1 << FPlatformMath::FloorLog2(MaxTaskCount);
	IndexType BatchSize = (Num + TaskCount - 1) / TaskCount;
	
	// Round 0 is the initial recursive sort of segments, Round 1 is the first merge round, ...
	// Round TaskCount is the merge into the final full Range.
	const int32 NumRounds = FPlatformMath::FloorLog2(TaskCount) + 1;

	// We define a recursive merge function for round N that launches a task for the Upper in round NumRounds - 1,
	// then works itself on the Lower for round N - 1, then merges. We end up with TaskCount threads (including the
	// main thread) doing the single-threaded recursive call to mergesort in round 0, then half that many threads
	// doing the merge in round 1, ..., and only the main thread doing the merge in round NumRounds - 1.

	// Each thread working on a round needs scratch space for the merge, we give it a mergebuffer. It uses the same
	// mergebuffer for each of the recursive calls that it works on itself; the tasks it kicks off create their
	// own mergebuffers and do the same.
	// Merging two segments, each of size <= BatchSize into a merged 2*BatchSize length segment requires scratch
	// space equal to BatchSize, (plus 1 more because we push before we pop).
	auto GetMergeBufferForRound = [BatchSize, Num](int32 Round, int32 ConstructorSite)
		{
			// BatchSize << Round is the size of the segment produced for the round, we only need half that (plus one).
			const IndexType BufferSize = ((BatchSize << Round) + 1) / 2 + 1;
			TMergeBuffer<T> MergeBuffer;
			MergeBuffer.Buffer.Reserve(BufferSize);
			MergeBuffer.AllocatedSize = BufferSize;
			MergeBuffer.ConstructorSite = ConstructorSite;
			MergeBuffer.InputNum = Num;
			MergeBuffer.Round = Round;
			MergeBuffer.BatchSize = BatchSize;
			return MergeBuffer;
		};

	// Does just the merge job with no recursion.
	auto MergeRoundAndIndex = [TaskCount, Num, BatchSize, &Predicate, First](int32 Round, int32 IndexInRound, TMergeBuffer<T>& MergeBuffer)
		{
			const int32 NumMergesForRound = TaskCount >> Round;
			const IndexType CurrentLevelBatchSize = BatchSize << Round;
			const IndexType PreviousRoundBatchSize = BatchSize << (Round - 1);
			const IndexType LowerStart = IndexInRound * CurrentLevelBatchSize;
			const IndexType UpperStart = IndexInRound * CurrentLevelBatchSize + PreviousRoundBatchSize;
			const IndexType MergeNum = IndexInRound < NumMergesForRound - 1 ? CurrentLevelBatchSize : Num - LowerStart;
			Merge(First + LowerStart, MergeNum, UpperStart - LowerStart, Predicate, MergeBuffer);
		};

	// The base case of recursion; call single-threaded mergesort, which does its own recursion but without any
	// threading.
	auto RecursiveMergeRound0 = [TaskCount, Num, BatchSize, &Predicate, First](int32 IndexInRound, TMergeBuffer<T>& MergeBuffer)
		{
			const IndexType StartIndex = IndexInRound * BatchSize;
			const IndexType MergeNum = IndexInRound < TaskCount - 1 ? BatchSize : Num - StartIndex;
			MergeSortRecursive(First + StartIndex, MergeNum, Predicate, MergeBuffer);
		};

	// Our recursive function that forks itself to handle the upper recursive segment, works itself recursively on the
	// lower recursive segment, and then merges the two segments. The arguments are the round number, and the index
	// within the round of the segment it is creating.
	TFunction<void(int32, int32, TMergeBuffer<T>&)> RecursiveMergeRoundAndIndexPtr;
	auto RecursiveMergeRoundAndIndex =
		[&MergeRoundAndIndex, &RecursiveMergeRoundAndIndexPtr, &RecursiveMergeRound0, &GetMergeBufferForRound]
		(int32 Round, int32 IndexInRound, TMergeBuffer<T>& MergeBuffer)
		{
			if (Round > 0)
			{
				FTask Upper = Launch(TEXT("ParallelMergeSort"),
					[&RecursiveMergeRoundAndIndexPtr, &GetMergeBufferForRound, Round, IndexInRound]()
					{
						TMergeBuffer<T> MergeBuffer = GetMergeBufferForRound(Round - 1, 100);
						RecursiveMergeRoundAndIndexPtr(Round - 1, 2 * IndexInRound + 1, MergeBuffer);
					});

				RecursiveMergeRoundAndIndexPtr(Round - 1, 2 * IndexInRound, MergeBuffer);
				Upper.Wait();
				MergeRoundAndIndex(Round, IndexInRound, MergeBuffer);
			}
			else
			{
				RecursiveMergeRound0(IndexInRound, MergeBuffer);
			}
		};
	RecursiveMergeRoundAndIndexPtr = RecursiveMergeRoundAndIndex;

	// Kick off RecursiveMergeRoundAndIndexPtr for the last round, at its only segment index: index 0.
	TMergeBuffer<T> MergeBuffer = GetMergeBufferForRound(NumRounds - 1, 200);
	RecursiveMergeRoundAndIndexPtr(NumRounds - 1, 0, MergeBuffer);
}

template <typename T, typename IndexType, typename PredicateType>
void MergeSortPointerAndNum(T* First, IndexType Num, PredicateType Predicate)
{
	TMergeBuffer<T> MergeBuffer;
	IndexType BufferSize = (Num + 1) / 2 + 1;
	MergeBuffer.Buffer.Reserve(BufferSize);
	MergeBuffer.AllocatedSize = BufferSize;
	MergeBuffer.ConstructorSite = 300;
	MergeBuffer.InputNum = Num;
	MergeBuffer.Round = -1;
	MergeBuffer.BatchSize = -1;
	MergeSortRecursive(First, Num, Predicate, MergeBuffer);
}

template <typename T, typename IndexType, typename PredicateType>
void MergeSortRecursive(T* First, IndexType Num, PredicateType Predicate, TMergeBuffer<T>& MergeBuffer)
{
	if (Num <= 3)
	{
		SmallSort(First, Num, Predicate);
		return;
	}

	const IndexType FloorMidIndex = (Num + 1) / 2;
	MergeSortRecursive(First, FloorMidIndex, Predicate, MergeBuffer);
	MergeSortRecursive(First + FloorMidIndex, Num - FloorMidIndex, Predicate, MergeBuffer);

	Merge(First, Num, FloorMidIndex, Predicate, MergeBuffer);
}

template <typename T, typename IndexType, typename PredicateType>
void Merge(T* First, IndexType Num, IndexType FloorMidIndex, PredicateType Predicate, TMergeBuffer<T>& InMergeBuffer)
{
	IndexType WriteHead = 0;
	IndexType UpperReadHead = FloorMidIndex;

	// Skip past however many leading elements of Lower come before the first element of Upper
	while (WriteHead < FloorMidIndex)
	{
		if (!Predicate(First[UpperReadHead], First[WriteHead]))
		{
			++WriteHead;
		}
		else
		{
			break;
		}
	}

	if (WriteHead == FloorMidIndex)
	{
		// All elements of Lower came before Upper, the range is already sorted
		return;
	}

	TRingBuffer<T>& MergeBuffer = InMergeBuffer.Buffer;
	MergeBuffer.Reset();
	// Our caller is supposed to reserve the MergeBuffer for us. The maximum size of MergeBuffer that we use is
	// the length of the Upper half, plus 1. Assert that we have that size so that we don't hit the performance
	// cost of growing the buffer here.
	checkf(MergeBuffer.Max() >= Num - FloorMidIndex + 1,
		TEXT("ParallelSort bug: MergeBuffer not preallocated large enough. %d < %d. ")
		TEXT("AllocatedSize=%d, ConstructorSite=%d, InputNum=%d, Round=%d, BatchSize=%d."),
		MergeBuffer.Max(), Num - FloorMidIndex + 1,
		InMergeBuffer.AllocatedSize, InMergeBuffer.ConstructorSite, InMergeBuffer.InputNum, InMergeBuffer.Round, InMergeBuffer.BatchSize);

	// Move the UpperReadHead into WriteHead, we already compared it above and know it is lower
	MergeBuffer.Add(MoveTemp(First[WriteHead]));
	First[WriteHead] = MoveTemp(First[UpperReadHead]);
	++UpperReadHead;
	++WriteHead;

	while (UpperReadHead < Num && WriteHead < FloorMidIndex)
	{
		check(WriteHead < UpperReadHead);
		MergeBuffer.Add(MoveTemp(First[WriteHead]));
		if (!Predicate(First[UpperReadHead], MergeBuffer[0]))
		{
			First[WriteHead] = MergeBuffer.PopFrontValue();
		}
		else
		{
			First[WriteHead] = MoveTemp(First[UpperReadHead]);
			++UpperReadHead;
		}
		++WriteHead;
	}
	while (UpperReadHead < Num && !MergeBuffer.IsEmpty())
	{
		check(WriteHead < UpperReadHead);
		if (!Predicate(First[UpperReadHead], MergeBuffer[0]))
		{
			First[WriteHead] = MergeBuffer.PopFrontValue();
		}
		else
		{
			First[WriteHead] = MoveTemp(First[UpperReadHead]);
			++UpperReadHead;
		}
		++WriteHead;
	}
	while (WriteHead < FloorMidIndex)
	{
		MergeBuffer.Add(MoveTemp(First[WriteHead]));
		First[WriteHead] = MergeBuffer.PopFrontValue();
		++WriteHead;
	}
	while (!MergeBuffer.IsEmpty())
	{
		check(WriteHead < UpperReadHead);
		First[WriteHead] = MergeBuffer.PopFrontValue();
		++WriteHead;
	}

	// We have finished all lower, and so the remaining elements to write (if any) all come from the
	// remainder of UpperReadHead. And we don't need to move them because they're already in their destination
	// position. So the list of remaining writes should be identical to the list of remaining uppers. If we're
	// done with Uppers as well, then WriteHead == UpperReadHead == Num.
	check(WriteHead == UpperReadHead);
}

template <typename T, typename IndexType, typename PredicateType>
void SmallSort(T* First, IndexType Num, PredicateType Predicate)
{
	switch (Num)
	{
	case 0:
		return;
	case 1:
		return;
	case 2:
		if (Predicate(First[1], First[0]))
		{
			Swap(First[1], First[0]);
		}
		return;
	case 3:
		if (!Predicate(First[1], First[0]))
		{
			// 0 < 1
			if (Predicate(First[2], First[1]))
			{
				// 0 < 1 and 2 < 1
				if (Predicate(First[2], First[0]))
				{
					// 2 < 0 < 1
					T Temp = MoveTemp(First[0]);
					First[0] = MoveTemp(First[2]);
					First[2] = MoveTemp(First[1]);
					First[1] = MoveTemp(Temp);
				}
				else
				{
					// 0 < 2 < 1
					Swap(First[2], First[1]);
				}
			}
			// else Already sorted 0 < 1 < 2
		}
		else
		{
			// 1 < 0
			if (Predicate(First[2], First[0]))
			{
				// 1 < 0 and 2 < 0
				if (Predicate(First[2], First[1]))
				{
					// 2 < 1 < 0
					Swap(First[2], First[0]);
				}
				else
				{
					// 1 < 2 < 0
					T Temp = MoveTemp(First[0]);
					First[0] = MoveTemp(First[1]);
					First[1] = MoveTemp(First[2]);
					First[2] = MoveTemp(Temp);
				}
			}
			else
			{
				// 1 < 0 < 2
				Swap(First[1], First[0]);
			}
		}
		return;
	default:
		checkf(false, TEXT("ParallelSort bug: SmallSort was passed Num > 3"));
		return;
	}
}

} // namespace Algo