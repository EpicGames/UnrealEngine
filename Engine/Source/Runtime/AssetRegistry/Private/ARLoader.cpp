// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARLoader.h"

#include "AssetRegistry.h"
#include "AssetRegistryImpl.h"
#include "AssetRegistryPrivate.h"
#include "Async/Async.h"
#include "Async/TaskGraphInterfaces.h"
#include "CoreGlobals.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/Event.h"
#include "HAL/FileManager.h"
#include "HAL/FileManagerGeneric.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformProperties.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CoreMisc.h"
#include "Misc/DelayedAutoRegister.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "Misc/StringBuilder.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Serialization/LargeMemoryReader.h"
#include "Templates/UnrealTemplate.h"

namespace UE::AssetRegistry::Premade
{

FPreloader GPreloader;

bool IsEnabled()
{
	bool PlatformRequiresCookedData = FPlatformProperties::RequiresCookedData() && (IsRunningGame() || IsRunningDedicatedServer());

#if WITH_EDITOR && !ASSETREGISTRY_FORCE_PREMADE_REGISTRY_IN_EDITOR
	bool bUsePremadeInEditor = false;
	if (FCommandLine::IsInitialized())
	{
		bUsePremadeInEditor = FParse::Param(FCommandLine::Get(), TEXT("EnablePremadeAssetRegistry"));
	}
#else
	constexpr bool bUsePremadeInEditor = WITH_EDITOR;
#endif

	return PlatformRequiresCookedData || bUsePremadeInEditor;
}

bool CanLoadAsync()
{
	// TaskGraphSystemReady callback doesn't really mean it's running
	return FPlatformProcess::SupportsMultithreading() && FTaskGraphInterface::IsRunning();
}

FAssetRegistryLoadOptions GetLoadOptions()
{
	FAssetRegistryLoadOptions Options;
	// If ActiveMounts are enabled we need to be able to unload all Assets in a mountpoint at a time, so 
	// set bMountPointsRequireSeparateAllocationBlocks=true. Otherwise allocate all assets in a single block.
	Options.bMountPointsRequireSeparateAllocationBlocks =
		UE::AssetRegistry::Impl::FActiveMountsLoaderWrapper::IsEnabledByConfig();

	return Options;
}

/** Returns the paths to possible Premade AssetRegistry files, ordered from highest priority to lowest. */
static TArray<FString, TInlineAllocator<2>> GetPriorityPaths()
{
	TArray<FString, TInlineAllocator<2>> Paths;
#if WITH_EDITOR
	Paths.Add(FPaths::Combine(FPaths::ProjectDir(), TEXT("EditorClientAssetRegistry.bin")));
#endif
	Paths.Add(FPaths::Combine(FPaths::ProjectDir(), TEXT("AssetRegistry.bin")));
	return Paths;
}

FPreloader::FPreloader()
{
	//In the editor premade Asset Registry can be enabled by a command line argument so we need to wait until the task graph is ready 
	//before we rely on UE::AssetRegistry::Premade::IsEnabled() to return the correct result
	bool PremadeCanBeEnabled =
#if WITH_EDITOR
		true;
#else
		UE::AssetRegistry::Premade::IsEnabled();
#endif

	if (PremadeCanBeEnabled)
	{
		// run DelayedInitialize when TaskGraph system is ready
		OnTaskGraphReady.Emplace(STATS ? EDelayedRegisterRunPhase::StatSystemReady :
			EDelayedRegisterRunPhase::TaskGraphSystemReady,
			[this]()
			{
				if (UE::AssetRegistry::Premade::IsEnabled())
				{
					LoadState = EState::NotFound;
					DelayedInitialize();
				}
			});
	}
}

FPreloader::~FPreloader()
{
	// We are destructed after Main exits, which means that our AsyncThread was either never called
	// or it was waited on to complete by TaskGraph. Therefore we do not need to handle waiting for it ourselves.
	Shutdown(true /* bFromGlobalDestructor */);
}

bool FPreloader::Consume(FConsumeFunction&& ConsumeFunction)
{
	EConsumeResult Result = ConsumeInternal(MoveTemp(ConsumeFunction), FConsumeFunction());
	check(Result != EConsumeResult::Deferred);
	return Result == EConsumeResult::Succeeded;
}

FPreloader::EConsumeResult FPreloader::ConsumeOrDefer(FConsumeFunction&& ConsumeSynchronous,
	FConsumeFunction&& ConsumeAsynchronous)
{
	return ConsumeInternal(MoveTemp(ConsumeSynchronous), MoveTemp(ConsumeAsynchronous));
}

bool FPreloader::TrySetPath()
{
	for (FString& LocalPath : GetPriorityPaths())
	{
		if (IFileManager::Get().FileExists(*LocalPath))
		{
			ARPath = MoveTemp(LocalPath);
			return true;
		}
	}
	return false;
}

bool FPreloader::TrySetPath(const IPakFile& Pak)
{
	for (FString& LocalPath : GetPriorityPaths())
	{
		if (Pak.PakContains(LocalPath))
		{
			ARPath = MoveTemp(LocalPath);
			return true;
		}
	}
	return false;
}

ELoadResult FPreloader::TryLoad()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCookedAssetRegistryPreloader::TryLoad);
	LLM_SCOPE(ELLMTag::AssetRegistry);
	checkf(!ARPath.IsEmpty(), TEXT("TryLoad must not be called until after TrySetPath has succeeded."));

	FAssetRegistryLoadOptions Options = UE::AssetRegistry::Premade::GetLoadOptions();
	const int32 ThreadReduction = 2; // This thread + main thread already has work to do 
	int32 MaxWorkers = CanLoadAsync() ? FPlatformMisc::NumberOfCoresIncludingHyperthreads() - ThreadReduction : 0;
	Options.ParallelWorkers = FMath::Clamp(MaxWorkers, 0, 16);

	TUniquePtr<FArchive> FileArchive = UE::AssetRegistry::Impl::OpenReadStateFileArchive(ARPath);
	if (!FileArchive)
	{
		UE_LOGF(LogAssetRegistry, Warning, "Premade AssetRegistry path %ls existed but was unreadable.", *ARPath);
		LoadResult = ELoadResult::FailedToLoad;
		return LoadResult;
	}
	FAssetRegistryLoadResults LoadResults;
	if (!Payload.ARState.Load(*FileArchive, Options, LoadResults))
	{
		UE_LOGF(LogAssetRegistry, Warning, "Premade AssetRegistry path %ls existed but failed to load.", *ARPath);
		LoadResult = ELoadResult::FailedToLoad;
		return LoadResult;
	}
	Payload.StateFilePath = ARPath;
	Payload.StoreUsedByState = MoveTemp(LoadResults.Store);

	UE_LOGF(LogAssetRegistry, Log, "Premade AssetRegistry loaded from '%ls'.", *ARPath);
	LoadResult = ELoadResult::Succeeded;
	return LoadResult;
}

void FPreloader::DelayedInitialize()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCookedAssetRegistryPreloader::DelayedInitialize);
	// This function will run before any UObject (ie UAssetRegistryImpl) code can run, so we don't need to do any thread safety
	// CanLoadAsync - we have to check this after the task graph is ready
	if (!CanLoadAsync())
	{
		LoadState = EState::LoadSynchronous;
		return;
	}

	// PreloadReady is in Triggered state until the Async thread is created. It is Reset in KickPreload.
	PreloadReady = FPlatformProcess::GetSynchEventFromPool(true /* bIsManualReset */);
	PreloadReady->Trigger();

	if (TrySetPath())
	{
		FScopeLock Lock(&StateLock);
		KickPreloadOrDelayIfConfigNotReady();
	}
	else
	{
		// set to NotFound, although PakMounted may set it to found later
		LoadState = EState::NotFound;

		// The PAK with the main registry isn't mounted yet
		PakMountedDelegate = FCoreDelegates::GetOnPakFileMounted2().AddLambda([this](const IPakFile& Pak)
			{
				FScopeLock Lock(&StateLock);
				if (LoadState == EState::NotFound && TrySetPath(Pak))
				{
					KickPreloadOrDelayIfConfigNotReady();
					// Remove the callback from OnPakFileMounted2 to avoid wasting time in all future PakFile mounts
					// Do not access any of the lambda captures after the call to Remove, because deallocating the 
					// DelegateHandle also deallocates our lambda captures
					FDelegateHandle LocalPakMountedDelegate = PakMountedDelegate;
					PakMountedDelegate.Reset();
					FCoreDelegates::GetOnPakFileMounted2().Remove(LocalPakMountedDelegate);
				}
			});
	}
}

void FPreloader::KickPreloadOrDelayIfConfigNotReady()
{
	// Called within StateLock.
	// Only called from DelayedInitialize or its PakMountedDelegate, and only called if ARPath has been found.
	// KickPreload requires ARPath has been found.
	check(!ARPath.IsEmpty());
	if (GConfig)
	{
		KickPreload();
	}
	else
	{
		LoadState = EState::ConfigNotReady;
		FCoreDelegates::TSConfigReadyForUse().AddLambda([this]()
			{
				FScopeLock Lock(&StateLock);
				if (LoadState == EState::ConfigNotReady)
				{
					KickPreload();
				}
			});
	}
}

void FPreloader::KickPreload()
{
	CSV_TRACE_REGION_BEGIN(TEXT("Preload"), TEXT("AssetRegistry"))
	TRACE_CPUPROFILER_EVENT_SCOPE(FCookedAssetRegistryPreloader::KickPreload);
	// Called from Within the Lock
	check((LoadState == EState::NotFound || LoadState == EState::ConfigNotReady) && !ARPath.IsEmpty());
	LoadState = EState::Loading;
	PreloadReady->Reset();
	Async(EAsyncExecution::TaskGraph, [this]() { TryLoadAsync(); });
}

void FPreloader::TryLoadAsync()
{
	// This function is active only after State has been set to Loading and PreloadReady has been Reset
	// Until this function triggers PreloadReady, it has exclusive ownership of bLoadSucceeded and Payload
	// Load outside the lock so that ConsumeOrDefer does not have to wait for the Load before it can defer and exit
	ELoadResult LocalResult = TryLoad();
	// Trigger outside the lock so that a locked Consume function that is waiting on PreloadReady can wait inside the lock.
	PreloadReady->Trigger();

	FConsumeFunction LocalConsumeCallback;
	{
		FScopeLock Lock(&StateLock);
		// The consume function may have woken up after the trigger and already consumed and changed LoadState to Consumed
		if (LoadState == EState::Loading)
		{
			LoadState = EState::Loaded;
			if (ConsumeCallback)
			{
				LocalConsumeCallback = MoveTemp(ConsumeCallback);
				ConsumeCallback.Reset();
				LoadState = EState::Consumed;
			}
		}
	}

	if (LocalConsumeCallback)
	{
		// No further threads will read/write payload at this point (until destructor, which is called after all async threads are complete
		// so we can use it outside the lock
		LocalConsumeCallback(LocalResult, MoveTemp(Payload));
		Shutdown();
	}
}

FPreloader::EConsumeResult FPreloader::ConsumeInternal(FConsumeFunction&& ConsumeSynchronous, 
	FConsumeFunction&& ConsumeAsynchronous)
{
	SCOPED_BOOT_TIMING("FCookedAssetRegistryPreloader::Consume");

	FScopeLock Lock(&StateLock);
	// Report failure if constructor decided not to preload or this has already been Consumed
	if (LoadState == EState::WillNeverPreload || LoadState == EState::Consumed || ConsumeCallback)
	{
		Lock.Unlock(); // Unlock before calling external code in Consume callback
		ELoadResult LocalResult = (LoadState == EState::Consumed || ConsumeCallback) ? ELoadResult::AlreadyConsumed : ELoadResult::Inactive;
		ConsumeSynchronous(LocalResult, UE::AssetRegistry::Impl::FAssetProviderInitData());
		return EConsumeResult::Failed;
	}

	if (LoadState == EState::LoadSynchronous)
	{
		ELoadResult LocalResult = TrySetPath() ? TryLoad() : ELoadResult::NotFound;
		LoadState = EState::Consumed;
		Lock.Unlock(); // Unlock before calling external code in Consume callback
		ConsumeSynchronous(LocalResult, MoveTemp(Payload));
		Shutdown(); // Shutdown can be called outside the lock since AsyncThread doesn't exist
		return LocalResult == ELoadResult::Succeeded ? EConsumeResult::Succeeded : EConsumeResult::Failed;
	}

	// Cancel any further searching in Paks since we will no longer accept preloads starting after this point
	FCoreDelegates::GetOnPakFileMounted2().Remove(PakMountedDelegate);
	PakMountedDelegate.Reset();

	if (ConsumeAsynchronous && LoadState == EState::Loading)
	{
		// The load might have completed and the TryAsyncLoad thread is waiting to enter the lock, but we will still defer since Consume won the race
		ConsumeCallback = MoveTemp(ConsumeAsynchronous);
		return EConsumeResult::Deferred;
	}

	{
		SCOPED_BOOT_TIMING("BlockingConsume");
		// If the load is in progress, wait for it to finish (which it does outside the lock)
		PreloadReady->Wait();
	}

	// TryAsyncLoad might not yet have set state to Loaded
	check(LoadState == EState::Loaded || LoadState == EState::Loading || LoadState == EState::NotFound || LoadState == EState::ConfigNotReady);
	ELoadResult LocalResult = (LoadState == EState::NotFound || LoadState == EState::ConfigNotReady) ? ELoadResult::NotFound : LoadResult;
	LoadState = EState::Consumed;

	// No further async threads exist that will read/write payload at this point so we can use it outside the lock
	Lock.Unlock(); // Unlock before calling external code in Consume callback
	ConsumeSynchronous(LocalResult, MoveTemp(Payload));
	Shutdown(); // Shutdown can be called outside the lock since we have set state to Consumed and the Async thread will notice and exit
	return LocalResult == ELoadResult::Succeeded ? EConsumeResult::Succeeded : EConsumeResult::Failed;
}

void FPreloader::Shutdown(bool bFromGlobalDestructor)
{
	OnTaskGraphReady.Reset();
	if (PreloadReady)
	{
		// If we are exiting the process early while PreloadReady is still allocated, the event
		// system has already been torn down and there is nothing for us to free for PreloadReady.
		if (!bFromGlobalDestructor)
		{
			FPlatformProcess::ReturnSynchEventToPool(PreloadReady);
		}
		PreloadReady = nullptr;
	}
	ARPath.Reset();
	Payload.Reset();
	CSV_TRACE_REGION_END(TEXT("Preload"), TEXT("AssetRegistry"))
}

FAsyncConsumer::~FAsyncConsumer()
{
	if (Consumed)
	{
		FPlatformProcess::ReturnSynchEventToPool(Consumed);
		Consumed = nullptr;
	}
}

void FAsyncConsumer::PrepareForConsume()
{
	// Called within the lock
	check(!Consumed);
	Consumed = FPlatformProcess::GetSynchEventFromPool(true /* bIsManualReset */);
	++ReferenceCount;
};

void FAsyncConsumer::Wait(UE::AssetRegistry::FInterfaceWriteScopeLock& ScopeLock)
{
	// Called within the lock
	if (ReferenceCount == 0)
	{
		return;
	}
	++ReferenceCount;

	// Wait outside of the lock so that the AsyncThread can enter the lock to call Consume
	{
		ScopeLock.Lock.WriteUnlock();
		ON_SCOPE_EXIT{ ScopeLock.Lock.WriteLock(); };
		check(Consumed != nullptr);
		Consumed->Wait();
	}

	--ReferenceCount;
	if (ReferenceCount == 0)
	{
		// We're the last one to drop the refcount, so delete Consumed
		check(Consumed != nullptr);
		FPlatformProcess::ReturnSynchEventToPool(Consumed);
		Consumed = nullptr;
	}
}

void FAsyncConsumer::Consume(UAssetRegistryImpl& UARI, UE::AssetRegistry::Impl::FEventContext& EventContext,
	UE::AssetRegistry::Impl::FActiveMountEvents& ActiveMountEvents,
	UE::AssetRegistry::FInterfaceWriteScopeLock& ProofOfWriteLock, ELoadResult LoadResult,
	UE::AssetRegistry::Impl::FAssetProviderInitData&& StateInitData)
{
	// Called within the lock
	UARI.GuardedData.LoadPremadeAssetRegistry(UARI, EventContext, ActiveMountEvents, ProofOfWriteLock, LoadResult,
		MoveTemp(StateInitData));
	check(ReferenceCount >= 1);
	check(Consumed != nullptr);
	Consumed->Trigger();
	--ReferenceCount;
	if (ReferenceCount == 0)
	{
		// We're the last one to drop the refcount, so delete Consumed
		FPlatformProcess::ReturnSynchEventToPool(Consumed);
		Consumed = nullptr;
	}
}

} // namespace UE::AssetRegistry::Premade

namespace UE::AssetRegistry::Impl
{

TUniquePtr<FArchive> OpenReadStateFileArchive(FStringView FilePath, uint32 BufferSize)
{
	TStringBuilder<512> FilePathSZ(InPlace, FilePath);
	TUniquePtr<IFileHandle> FileHandle(FPlatformFileManager::Get().GetPlatformFile().OpenRead(*FilePathSZ));
	if (!FileHandle)
	{
		return TUniquePtr<FArchive>();
	}
	int64 FileSize = FileHandle->Size();
	if (FileSize <= 0)
	{
		return TUniquePtr<FArchive>();
	}
	if (BufferSize == 0 || FileSize <= static_cast<int64>(BufferSize))
	{
		// Read the entire file into memory. We set BufferSize=0 and take this route by default because
		// AssetRegistry serialization is on the critical path for starting the game in many cases, and its
		// faster to read all at once rather than serializing repeatedly from a FileReader.

		// FLargeMemoryReader can take ownership of the Data buffer; it requires that we allocate with malloc.
		uint8* Data = static_cast<uint8*>(FMemory::Malloc(FileSize));
		ON_SCOPE_EXIT
		{
			if (Data)
			{
				FMemory::Free(Data);
			}
		};
		if (!FileHandle->Read(Data, FileSize))
		{
			return TUniquePtr<FArchive>();
		}
		TUniquePtr<FArchive> Result(new FLargeMemoryReader(Data, FileSize, ELargeMemoryReaderFlags::TakeOwnership));
		Data = nullptr; // detach Data from our call to FMemory::Free
		return Result;
	}
	else
	{
		return MakeUnique<FArchiveFileReaderGeneric>(FileHandle.Release(), *FilePathSZ, FileSize, BufferSize);
	}
}

FAssetProviderInitData::FAssetProviderInitData(FAssetProviderInitData&& Other)
{
	*this = MoveTemp(Other);
}

FAssetProviderInitData& FAssetProviderInitData::operator=(FAssetProviderInitData&& Other)
{
	ARState = MoveTemp(Other.ARState);
	StateFilePath = MoveTemp(Other.StateFilePath);
	StoreUsedByState = MoveTemp(Other.StoreUsedByState);
	Other.Reset();
	return *this;
}

void FAssetProviderInitData::Reset()
{
	ARState.Reset();
	StateFilePath.Empty();
	StoreUsedByState = FixedTagPrivate::FWeakStorePtr();
}

} // namespace UE::AssetRegistry::Impl