// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARTypes.h"
#include "Async/RecursiveMutex.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "Containers/ArrayView.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/CriticalSection.h"
#include "Misc/Optional.h"
#include "Misc/PackageName.h"
#include "Serialization/Archive.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

#include <atomic>

// Preloading a premade AssetRegistry is disabled in editor unless the target specifies it in UBT, because the editor
// usually gathers to create an up to date AssetRegistry rather than using a previously cooked AssetRegistry.bin.
#ifndef ASSETREGISTRY_FORCE_PREMADE_REGISTRY_IN_EDITOR
#define ASSETREGISTRY_FORCE_PREMADE_REGISTRY_IN_EDITOR 0
#endif

class FEvent;
class FPakPlatformFile;
class IPakFile;
class UAssetRegistryImpl;
namespace UE::AssetRegistry { class FAssetRegistryImpl; }
namespace UE::AssetRegistry::Impl { class FActiveMountEvents; }
namespace UE::AssetRegistry::Impl { class FActiveMountsLoader; }
namespace UE::AssetRegistry::Impl { class FAssetProvider; }
namespace UE::AssetRegistry::Impl { class FMountPointData; }
namespace UE::AssetRegistry::Impl { struct FEventContext; }
struct FDelayedAutoRegisterHelper;

namespace UE::AssetRegistry::Impl
{

/**
 * Data about the premade AssetRegistryState passed through from its load into LoadPremadeAssetRegistry and the
 * ActiveMountsLoader 
 */
struct FAssetProviderInitData
{
	FAssetProviderInitData() = default;
	FAssetProviderInitData(const FAssetProviderInitData& Other) = delete;
	FAssetProviderInitData(FAssetProviderInitData&& Other);
	FAssetProviderInitData& operator=(const FAssetProviderInitData& Other) = delete;
	FAssetProviderInitData& operator=(FAssetProviderInitData&& Other);
	void Reset();

	FAssetRegistryState ARState;
	FString StateFilePath;
	FixedTagPrivate::FWeakStorePtr StoreUsedByState;
};

}

namespace UE::AssetRegistry::Premade
{

enum class ELoadResult : uint8
{
	Succeeded = 0,
	NotFound = 1,
	FailedToLoad = 2,
	Inactive = 3,
	AlreadyConsumed = 4,
	UninitializedMemberLoadResult = 5,
};

/** Loads AssetRegistry.bin during engine startup, using an async preload task if available and sync otherwise. */
class FPreloader
{
public:
	enum class EConsumeResult
	{
		Succeeded,
		Failed,
		Deferred
	};
	using FConsumeFunction = TFunction<void(ELoadResult LoadResult,
		UE::AssetRegistry::Impl::FAssetProviderInitData&& StateInitData)>;

	FPreloader();
	~FPreloader();

	/**
	 * Block on any pending async load, load if synchronous, and call ConsumeFunction with the results before returning.
	 * If Consume has been called previously, the current ConsumeFunction is ignored and this call returns false.
	 *
	 * @return Whether the load succeeded (this information is also passed to the ConsumeFunction).
	 */
	bool Consume(FConsumeFunction&& ConsumeFunction);

	/**
	 * If a load is pending, store ConsumeAsynchronous for later calling and return EConsumeResult::Deferred.
	 * If load is complete, or failed, or needs to run synchronously, load if necessary and call ConsumeSynchronous with results before returning.
	 * Note if this function returns EConsumeResult::Deferred, the ConsumeAsynchronous will be called from another thread,
	 * possibly before this call returns.
	 * If Consume has been called previously, this call is ignored and returns EConsumeResult::Failed.
	 *
	 * @return Whether the load succeeded (this information is also passed to the ConsumeFunction).
	 */
	EConsumeResult ConsumeOrDefer(FConsumeFunction&& ConsumeSynchronous, FConsumeFunction&& ConsumeAsynchronous);

private:
	enum class EState : uint8
	{
		WillNeverPreload,
		LoadSynchronous,
		NotFound,
		ConfigNotReady,
		Loading,
		Loaded,
		Consumed,
	};


	bool TrySetPath();
	bool TrySetPath(const IPakFile& Pak);
	ELoadResult TryLoad();
	void DelayedInitialize();
	void KickPreloadOrDelayIfConfigNotReady();
	void KickPreload();
	void TryLoadAsync();
	EConsumeResult ConsumeInternal(FConsumeFunction&& ConsumeSynchronous, FConsumeFunction&& ConsumeAsynchronous);
	/** Called when the Preloader has no further work to do, to free resources early since destruction occurs at end of process. */
	void Shutdown(bool bFromGlobalDestructor = false);

	/** simple way to trigger a callback at a specific time that TaskGraph is usable. */
	TOptional<FDelayedAutoRegisterHelper> OnTaskGraphReady;

	/** Lock that guards members on this (see notes on each member). */
	FCriticalSection StateLock;
	/** Trigger for blocking Consume to wait upon TryLoadAsync. This Trigger is only allocated when in the states NotFound, Loaded, Loading. */
	FEvent* PreloadReady = nullptr;

	/** Path discovered for the AssetRegistry; Read/Write only within the Lock. */
	FString ARPath;

	/**
	 * The set of (ARState, AssetRegistryLoadResults, and other metadata) that was loaded from disk.
	 * Owned exclusively by either the first Consume or by TryAsyncLoad.
	 * If LoadState is never set to Loading, this state is read/written only by the first thread to call Consume.
	 * If LoadState is set to Loading (which happens before threading starts), the thread running TryAsyncLoad
	 * owns this payload until it triggers PreloadReady, after which ownership returns to the first thread to call Consume.
	 */
	UE::AssetRegistry::Impl::FAssetProviderInitData Payload;

	FDelegateHandle PakMountedDelegate;

	/** Callback from ConsumeOrDefer that is set so TryLoadAsync can trigger the Consume when it completes.Read / Write only within the lock. */
	FConsumeFunction ConsumeCallback;

	/** State machine state. Read/Write only within the lock (or before threading starts). */
	EState LoadState = EState::WillNeverPreload;

	/** Result of TryLoad.Thread ownership rules are the same as the rules for Payload. */
	ELoadResult LoadResult = ELoadResult::UninitializedMemberLoadResult;
};

/** Returns whether the given executable configuration supports AssetRegistry Preloading. Called before Main. */
extern bool IsEnabled();
extern bool CanLoadAsync();
/** Returns the LoadOptions used to load the premade AssetRegistry.bin. */
FAssetRegistryLoadOptions GetLoadOptions();

extern FPreloader GPreloader;

/**
* A struct to consume a Premade asset registry on an async thread. It supports a cheap Wait() call so that it can be Waited on
* by frequent AssetRegistry interface calls for the rest of the process.
*/
struct FAsyncConsumer
{
	~FAsyncConsumer();
	/** Sets the Consumer into need-to-wait mode; Wait will block until Consume is called. */
	void PrepareForConsume();
	/**
	* Does not return until after the Premade AssetRegistryState has been added into the target AssetRegistry, or it has
	* been decided not to be used. For performance reasons, this must be called within the WriteScopeLock, and the caller
	* must handle the possibility that it leaves and reenters the lock.
	*/
	void Wait(FInterfaceWriteScopeLock& ScopeLock);
	/** Callback from the async thread to consume the Premade ARState */
	void Consume(UAssetRegistryImpl& UARI, UE::AssetRegistry::Impl::FEventContext& EventContext,
		UE::AssetRegistry::Impl::FActiveMountEvents& ActiveMountEventContext,
		UE::AssetRegistry::FInterfaceWriteScopeLock& ProofOfWriteLock,
		ELoadResult LoadResult, UE::AssetRegistry::Impl::FAssetProviderInitData&& StateInitData);

private:
	/**
	* ReferenceCounter for the Consumed event. Also used to decide whether Waiting is necessary. Read/Write only inside the lock.
	*/
	int32 ReferenceCount = 0;
	/** Event used to Wait for Consume. Allocated/deallocated within the lock. Can be waited on outside the lock if RefCount is held. */
	FEvent* Consumed = nullptr;
};

} // namespace UE::AssetRegistry::Premade

namespace UE::AssetRegistry::Impl
{

/**
 * Open the given AssetRegistry.bin file using the performant archive we use for loading it.
 * If BufferSize == 0 or is greater than filesize, reads the entire file into an in-memory archive.
 */
TUniquePtr<FArchive> OpenReadStateFileArchive(FStringView FilePath, uint32 BufferSize=0);

/** Which event triggered our need to register the premade AssetRegistry with the ActiveMountsLoading. */
enum class EInitialProviderRegisterReason
{
	/* Expected case, OnActiveMountsInitialize was called. */
	ActiveMountsInitialize,
	/** Allowed case, AppendState was called before OnActiveMountsInitialize. */
	AppendState,
	/** Error case, some other data was added to global AssetRegistry before the premade. */
	EarlierState,
};

/**
 * Holds callbacks and events triggered during ActiveMountsLoading functions that need to be broadcast outside of the
 * AssetRegistry's locks.
 */
class FActiveMountEvents
{
public:
	/**
	 * Add a completion callback from ActiveMountsLoadAsync or ActiveMountsUnloadAsync, and the value of its
	 * bCanceled parameter.
	 */
	void AddCompletion(TFunction<void(bool bCanceled)> InOnComplete, bool bInCanceled);
	/** Add an OnIdle callback from ActiveMountsReportWhenIdle. */
	void AddIdle(TFunction<void()> InOnIdle);
	/** Send all of the added events. Must be called outside of the AssetRegistry's locks. */
	void Broadcast();

private:
	struct FCompletionData
	{
		TFunction<void(bool bCanceled)> OnComplete;
		bool bCanceled = false;
	};
	TArray<FCompletionData> CompletionEvents;
	TArray<TFunction<void()>> IdleCallbacks;
};

/**
 * The interface into the ActiveMountsLoader used by the rest of the AssetRegistry. The ActiveMountsLoader stores
 * filepaths for AssetRegistry.bin files that provide assets to the AssetRegistry, and it manages a state machine to
 * unload and reload the AssetDatas for mountpoints (such as plugin Content directories) when those mountpoints are
 * loaded and unloaded. @see the comment on ActiveMountsLoading in IAssetRegistry.h.
 *
 * The ActiveMountsLoader has some minor performance costs and so should only be enabled in configurations and projects
 * that need the memory savings and that can behave correctly when AssetDatas are unloaded for inactive plugins.
 * 
 * This interface object exists in all configurations, but it only creates the full ActiveMountsLoader when enabled.
 * When not enabled it implements only the side effects that are required by the rest of the AssetRegistry, such
 * as calling AppendState for filenames passed to TryRegisterAndAppendState. This wrapper also manages the separate
 * critical section(s) used by the ActiveMountsLoader; ActiveMountsLoader functions are always called inside of its
 * lock.
 */
class FActiveMountsLoaderWrapper
{
public:
	/**
	 * Called during AssetRegistry initialization. Reads configuration settings such as enable flags and creates the
	 * full FActiveMountsLoader if enabled.
	 */
	void InitializeConfig();
	/**
	 * Called during AssetRegistry shutdown. Signals any active ActiveMountsLoader threads to terminate, waits for
	 * them, and shuts down the ActiveMountsLoader.
	 */
	void Shutdown(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext);
	/** Report whether the full ActiveMountsLoader is currently enabled. */
	bool IsEnabled();

	/** @see IAssetRegistry::ActiveMountsLoadAsync. Calls OnComplete(bCanceled=true) immediately if disabled. */
	void LoadAsync(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> InMountPoints,
		TFunction<void(bool bCanceled)>&& OnComplete);
	/** @see IAssetRegistry::ActiveMountsReloadAllAsync. Calls OnComplete(bCanceled=true) immediately if disabled. */
	void ReloadAllAsync(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		TFunction<void(bool bCanceled)>&& OnComplete);
	/** @see IAssetRegistry::ActiveMountsUnloadAsync. Calls OnComplete(bCanceled=true) immediately if disabled. */
	void UnloadAsync(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> InMountPoints,
		TFunction<void(bool bCanceled)>&& OnComplete);
	/** @see IAssetRegistry::ActiveMountsUnloadAllUnmountedAsync. Calls OnComplete(bCanceled=true) immediately if disabled. */
	void UnloadAllUnmountedAsync(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		TFunction<void(bool bCanceled)>&& OnComplete);
	/** @see IAssetRegistry::ActiveMountsIsLoaded. */
	bool IsLoaded(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint);
	/** @see IAssetRegistry::ActiveMountsIsInProgress. */
	bool IsInProgress(UAssetRegistryImpl& UARI);
	/** @see IAssetRegistry::ActiveMountsReportWhenIdle. */
	void ReportWhenIdle(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext, TFunction<void()> OnIdle);

	/** @see IAssetRegistry::ActiveMountsInitialize. */
	void OnActiveMountsInitialize(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		FInterfaceWriteScopeLock& ProofOfWriteLock);
	/** @see IAssetRegistry::ActiveMountsRegisterAndLoadFilePathSynchronous. */
	bool TryRegisterAssetProviderAndAppendStateOutsideARLock(UAssetRegistryImpl& UARI, FEventContext& AREventContext,
		FActiveMountEvents& EventContext, FStringView InStateFilePath);
	/**
	 * Called by internal AssetRegistry code to load and register the AssetRegistry files for built-in plugins, called
	 * immediately after the initial premade AssetRegistry is loaded. If successful, this function also triggers
	 * the immediate registration of the premade AssetRegistry.
	 */
	bool TryLoadStateAndRegisterAssetProvider(UAssetRegistryImpl& UARI,	FActiveMountEvents& EventContext,
		FInterfaceWriteScopeLock& ProofOfWriteLock, FPakPlatformFile* PakPlatformFile, FStringView InStateFilePath,
		FAssetRegistryState& OutLoadedState);
	/**
	 * Deactivate a file as an AssetProvider, clearing the data about it from memory. AssetDatas in MountPoints
	 * provided by the AssetProvider can no longer be unloaded or reloaded (until the AssetProvider is registered
	 * again). Invalid to call on the premade initial provider.
	 */
	void UnregisterAssetProvider(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		FStringView InStateFilePath);
	/**
	 * Called when the premade AssetRegistryState has finished loading and is added to the AssetRegistry. We do not
	 * yet try to identify the MountPoints used by the AssetDatas within the AssetRegistryState, because identifying
	 * them might rely on the registration of MountPoints with FPackageName and some projects might not be immediately
	 * ready for that. This function just stores the filename and marks that the AssetProvider exists.
	 */
	void InitialProviderReport(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		FAssetProviderInitData& StateInitData);
	/**
	 * Called when ActiveMountsInitialize or some other event signals that we are ready to identify MountPoints within
	 * the initial provider. Just sets a flag, takes no other action.
	 */
	void InitialProviderOnReadyForRegister(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		EInitialProviderRegisterReason Reason);
	/**
	 * Checks whether InitialProviderReport and InitialProviderOnReadyForRegister have both been called and if so
	 * parses the initial provider's AssetRegistry state to identify its MountPoints.
	 */
	void InitialProviderConditionalRegister(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		FAssetRegistryState& InitialProviderState);

	/**
	 * Called when FPackageName reports one or more OnMountPointMounted; triggers the automatic async batched reloading
	 * of AssetDatas for the MountPoint, if autounload is enabled.
	 */
	void OnMountPointsMounted(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> InMountPoints);
	/**
	 * Called when FPackageName reports one or more OnMountPointDismounted; triggers the automatic async batched
	 * unloading of AssetDatas for the MountPoint, if autounload is enabled.
	 */
	void OnMountPointsDismounted(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		FInterfaceWriteScopeLock& ProofOfWriteLock,
		TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> InMountPoints);

	/**
	 * Report whether the configuration settings specify that the full ActiveMountsLoader should be enabled. Should only
	 * be used during InitializeConfig or by functions that need to execute before it, such as
	 * UE::AssetRegistry::Premade::FPreloader.
	 */
	static bool IsEnabledByConfig();
private:
	/**
	 * Called by all public functions, even when the full ActiveMountsLoader is enabled. When disabled, operates as
	 * cheaply as possible, does not lock, and returns nullptr. When enabled, locks and returns a pointer to
	 * this->Inner.
	 */
	FActiveMountsLoader* LockIfEnabled();
	/** Lock ActiveMountsLock, the lock that must be entered around any full ActiveMountsLoader functions. */
	void Lock();
	/** Unlock ActiveMountsLock. */
	void Unlock();
	/**
	 * Return the lock that must be entered around AssetDataProvider register/deregister functions. Returns
	 * nullptr if full ActiveMountsLoader is not enabled.
	 */
	TSharedPtr<UE::FRecursiveMutex> GetRegistrationLock();

	/**
	 * Lock that guards the ActiveMountsLoader's data, separate from the AssetRegistry's data so that we do not block
	 * AssetRegistry queries when doing some long operations inside the lock.
	 * This lock is allowed to be held at the same time as the AssetRegistry's interface lock (avoid when possible).
	 * When both are held, the AssetRegistry's interface lock must be entered first.
	 */
	UE::FRecursiveMutex ActiveMountsLock;
	/**
	 * Null if ActiveMounts are disabled. This is an atomic only because it can be set to null during Shutdown and
	 * the reader might be checking it for null outside the lock when that happens. The reader must always enter the
	 * lock and check for null again before dereferencing it.
	 */
	std::atomic<FActiveMountsLoader*> Inner{ nullptr };

	friend FActiveMountsLoader;
	friend FAssetProvider;
	friend FMountPointData;
};

} // namespace UE::AssetRegistry::Impl
