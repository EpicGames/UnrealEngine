// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM.h"
#include "HAL/Platform.h"
#include "Misc/Optional.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"

#include <atomic>

namespace UE::AssetRegistry::Private { class FRWLockWithPriority; }
namespace UE::AssetRegistry::Private { using FInterfaceRWLock = FRWLockWithPriority; }
namespace UE::AssetRegistry::Private { template <typename TMutexType> class TTracingWriteScopeLock; }
namespace UE::AssetRegistry::Private { template <typename TMutexType> class TTracingReadScopeLock; }
namespace UE::AssetRegistry { template <typename TScopeLockType> class TRWScopeLockWithPriority; }
namespace UE::AssetRegistry { using FInterfaceReadScopeLock = TRWScopeLockWithPriority<Private::TTracingReadScopeLock<Private::FRWLockWithPriority>>; }
namespace UE::AssetRegistry { using FInterfaceWriteScopeLock = TRWScopeLockWithPriority<Private::TTracingWriteScopeLock<Private::FRWLockWithPriority>>; }
namespace UE::AssetRegistry { using FInterfaceRWScopeLock = class FRWScopeLockWithPriority; }

namespace UE::AssetRegistry::Private
{

/**
 * Custom lock class that can prioritize waiters upon request. This is used to allow threads with critical requests
 * to request that the gatherer thread pause its work and allow the higher pri threads to jump in. Note that for this
 * to work correctly it must be used with the associated TPriorityScopeLock types
 * (TScopeLockWithPriority and FRWScopeLockWithPriority)
*/
class FRWLockWithPriority
{
public:
	FORCEINLINE void WriteLock()
	{
		Mutex.WriteLock();
	}

	FORCEINLINE bool TryWriteLock()
	{
		return Mutex.TryWriteLock();
	}

	FORCEINLINE void WriteUnlock()
	{
		Mutex.WriteUnlock();
	}

	FORCEINLINE void ReadLock()
	{
		Mutex.ReadLock();
	}

	FORCEINLINE bool TryReadLock()
	{
		return Mutex.TryReadLock();
	}

	FORCEINLINE void ReadUnlock()
	{
		Mutex.ReadUnlock();
	}

	bool HasWaiters() const
	{
		bool Result;
		UE_AUTORTFM_OPEN
		{
			Result = HighPriorityWaitersCount.load(std::memory_order_relaxed) > 0;
		};
		return Result;
	}

private:
	friend UE::AssetRegistry::FInterfaceReadScopeLock;
	friend UE::AssetRegistry::FInterfaceWriteScopeLock;
	friend UE::AssetRegistry::FInterfaceRWScopeLock;
	FTransactionallySafeRWLock Mutex;
	std::atomic<int32> HighPriorityWaitersCount { 0 };
};

// Custom lock scopes to allow measuring lock contention
template<typename MutexType>
class TTracingReadScopeLock
{
public:
	UE_NONCOPYABLE(TTracingReadScopeLock);

	UE_NODISCARD_CTOR TTracingReadScopeLock(MutexType& InMutex)
		: Mutex(&InMutex)
	{
		check(Mutex);
		if (!Mutex->TryReadLock())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(InterfaceReadLockContention);
			Mutex->ReadLock();
		}
	}

	~TTracingReadScopeLock()
	{
		ReadUnlock();
	}

	void ReadUnlock()
	{
		if (Mutex)
		{
			Mutex->ReadUnlock();
			Mutex = nullptr;
		}
	}

private:
	MutexType* Mutex;
};

template<typename MutexType>
class TTracingWriteScopeLock
{
public:
	UE_NONCOPYABLE(TTracingWriteScopeLock);

	UE_NODISCARD_CTOR TTracingWriteScopeLock(MutexType& InMutex)
		: Mutex(&InMutex)
	{
		check(Mutex);
		if (!Mutex->TryWriteLock())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(InterfaceWriteLockContention);
			Mutex->WriteLock();
		}
	}

	~TTracingWriteScopeLock()
	{
		WriteUnlock();
	}

	void WriteUnlock()
	{
		if (Mutex)
		{
			Mutex->WriteUnlock();
			Mutex = nullptr;
		}
	}

private:
	MutexType* Mutex;
};

enum ELockPriority : uint8
{
	PriorityLow,
	PriorityHigh
};

} // namespace UE::AssetRegistry::Private

namespace UE::AssetRegistry
{

/** Tracks high-priority waiters on a lock, in an thread- and transactionally-safe manner. */
class FScopedHighPriorityWaitTracker
{
public:
	FScopedHighPriorityWaitTracker(std::atomic<int32>* InCounter, Private::ELockPriority InPriority)
		: Counter((InPriority == Private::PriorityHigh) ? InCounter : nullptr)
	{
		if (Counter)
		{
			UE_AUTORTFM_OPEN
			{
				Counter->fetch_add(1, std::memory_order_relaxed);
			};
			AutoRTFM::PushOnAbortHandler(this, [this]
				{
					Counter->fetch_sub(1, std::memory_order_relaxed);
				});
		}
	}

	~FScopedHighPriorityWaitTracker()
	{
		if (Counter)
		{
			UE_AUTORTFM_OPEN
			{
				Counter->fetch_sub(1, std::memory_order_relaxed);
			};
			AutoRTFM::PopOnAbortHandler(this);
		}
	}

private:
	std::atomic<int32>* Counter = nullptr;
};

/**
 * Keeps an FRWLock read-locked while this scope lives.
 * This is almost a clone of the existing FReadScopeLock and similar types
 * however this adds an extra flag to help the background processing thread
 * know when a higher priority thread would like to gain access to the protected
 * data
 */
template <typename TScopeLockType> class TRWScopeLockWithPriority
{
public:
	UE_NODISCARD_CTOR explicit TRWScopeLockWithPriority(Private::FRWLockWithPriority& InLock,
		Private::ELockPriority InPriority = Private::PriorityHigh)
		: Lock(InLock)
		, Priority(InPriority)
	{
		FScopedHighPriorityWaitTracker Tracker(&Lock.HighPriorityWaitersCount, Priority);
		GuardWrapper.Emplace(Lock);
	}

	TOptional<TScopeLockType> GuardWrapper;
	UE::AssetRegistry::Private::FRWLockWithPriority& Lock;
	UE::AssetRegistry::Private::ELockPriority Priority;
};

class FRWScopeLockWithPriority
{
public:
	UE_NODISCARD_CTOR explicit FRWScopeLockWithPriority(Private::FRWLockWithPriority& InLockObject,
		FRWScopeLockType InLockType, Private::ELockPriority InPriority = Private::PriorityHigh)
		: Lock(InLockObject)
		, Priority(InPriority)
		, LockType(InLockType)
	{
		FScopedHighPriorityWaitTracker Tracker(&Lock.HighPriorityWaitersCount, Priority);
		GuardWrapper.Emplace(Lock, LockType);
	}

	// NOTE: As the name suggests, this function should be used with caution. 
	// It releases the read lock _before_ acquiring a new write lock. This is not an atomic operation and the caller should 
	// not treat it as such. 
	// E.g. Pointers read from protected data structures prior to this call may be invalid after the function is called. 
	void ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION()
	{
		if (LockType == SLT_ReadOnly)
		{
			GuardWrapper.Reset();
			{
				FScopedHighPriorityWaitTracker Tracker(&Lock.HighPriorityWaitersCount, Priority);
				GuardWrapper.Emplace(Lock, SLT_Write);
			}
			LockType = SLT_Write;
		}
	}

	Private::FRWLockWithPriority& Lock;
	TOptional<TRWScopeLock<Private::FRWLockWithPriority>> GuardWrapper;
	Private::ELockPriority Priority;
	FRWScopeLockType LockType;
};

} // namespace UE::AssetRegistry