// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActiveMountsLoader.h"

#include "Algo/Count.h"
#include "AssetRegistry.h"
#include "AssetRegistryPrivate.h"
#include "Async/Async.h"
#include "Misc/ConfigCacheIni.h"
#include "Containers/DirectoryTree.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProperties.h"
#include "Interfaces/IPluginManager.h"
#include "IPlatformFilePak.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreMisc.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/FileHelper.h"
#include "Misc/Parse.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "Misc/StringBuilder.h"
#include "Stats/Stats.h"

namespace UE::AssetRegistry::Impl
{

template <typename ContainerType>
void ConditionalShrinkUsuallyEmptyContainer(ContainerType& Container)
{
	if (Container.Num() == 0 ||
		(Container.Max() > 8 && Container.Max() > 2 * Container.Num()))
	{
		Container.Shrink();
	}
}

void FActiveMountEvents::AddCompletion(TFunction<void(bool bCanceled)> InOnComplete, bool bInCanceled)
{
	if (!InOnComplete)
	{
		return;
	}
	CompletionEvents.Add(FCompletionData{ MoveTemp(InOnComplete), bInCanceled });
}

void FActiveMountEvents::AddIdle(TFunction<void()> InOnIdle)
{
	if (!InOnIdle)
	{
		return;
	}
	IdleCallbacks.Add(MoveTemp(InOnIdle));
}

void FActiveMountEvents::Broadcast()
{
	for (FCompletionData& Data : CompletionEvents)
	{
		check(Data.OnComplete);
		Data.OnComplete(Data.bCanceled);
	}
	CompletionEvents.Empty();
	for (TFunction<void()>& OnIdle : IdleCallbacks)
	{
		check(OnIdle);
		OnIdle();
	}
	IdleCallbacks.Empty();
}

bool FActiveMountsLoaderWrapper::IsEnabled()
{
	return Inner.load(std::memory_order_relaxed) != nullptr;
}

bool FActiveMountsLoaderWrapper::IsEnabledByConfig()
{
	static bool bEnabled = []()
		{
			bool bLocalEnabled = false;
			if (FPlatformProperties::RequiresCookedData() && IsRunningGame() && !IsRunningDedicatedServer())
			{
				if (!FParse::Bool(FCommandLine::Get(), TEXT("-ActiveMountsLoading="), bLocalEnabled))
				{
					// We need the value from Config to calculate the proper value, so assert that GConfig is
					// available by the time we first call IsEnabledByConfig.
					check(GConfig);
					GConfig->GetBool(TEXT("AssetRegistry"), TEXT("bActiveMountsLoadingEnabled_Client"),
						bLocalEnabled, GEngineIni);
				}
			}
			return bLocalEnabled;
		}();
	return bEnabled;
}

void FActiveMountsLoaderWrapper::InitializeConfig()
{
	if (!IsEnabledByConfig())
	{
		UE_LOGFMT(LogAssetRegistry, Log, "ActiveMountsLoading is disabled.");
		return;
	}

	bool bAutoUnload = false;
	if (!FParse::Bool(FCommandLine::Get(), TEXT("-ActiveMountsLoadingAuto="), bAutoUnload))
	{
		check(GConfig); // It was already required by IsEnabledByConfig
		GConfig->GetBool(TEXT("AssetRegistry"), TEXT("bActiveMountsLoadingAuto_Client"),
			bAutoUnload, GEngineIni);
	}

	Lock();
	ON_SCOPE_EXIT
	{
		Unlock();
	};
	// Races during initialization are unexpected but are also not a problem.
	if (!Inner.load(std::memory_order_relaxed))
	{
		UE_LOGFMT(LogAssetRegistry, Log, "ActiveMountsLoading is enabled.");
		FActiveMountsLoader* LocalInner = new FActiveMountsLoader();
		LocalInner->RegistrationLock = MakeShared<UE::FRecursiveMutex>();
		LocalInner->bAutoUnload = bAutoUnload;
		Inner.store(LocalInner, std::memory_order_relaxed);
	}
}

void FActiveMountsLoaderWrapper::Shutdown(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext)
{
	TArray<UE::Tasks::FTask> WaitOnAsync;
	if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
	{
		ON_SCOPE_EXIT
		{
			Unlock();
		};
		LocalInner->StartShutdown(UARI, EventContext, WaitOnAsync);
	}
	else
	{
		return;
	}

	// Waiting has to be done outside the lock because the task threads have to enter the lock
	// before they can exit.
	UE::Tasks::Wait(WaitOnAsync);

	if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
	{
		ON_SCOPE_EXIT
		{
			Unlock();
		};

		LocalInner->FinishShutdown(UARI, EventContext);
		delete LocalInner;
		Inner.store(nullptr, std::memory_order_relaxed);
	}
}

FActiveMountsLoader* FActiveMountsLoaderWrapper::LockIfEnabled()
{
	if (!Inner.load(std::memory_order_relaxed))
	{
		return nullptr;
	}
	ActiveMountsLock.Lock();
	FActiveMountsLoader* LocalInner = Inner.load(std::memory_order_relaxed);
	if (!LocalInner)
	{
		ActiveMountsLock.Unlock();
		return nullptr;
	}
	// Caller is responsible for calling Unlock.
	return LocalInner;
}

void FActiveMountsLoaderWrapper::Lock()
{
	ActiveMountsLock.Lock();
}

void FActiveMountsLoaderWrapper::Unlock()
{
	ActiveMountsLock.Unlock();
}

TSharedPtr<UE::FRecursiveMutex> FActiveMountsLoaderWrapper::GetRegistrationLock()
{
	if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
	{
		ON_SCOPE_EXIT
		{
			Unlock();
		};
		if (!LocalInner->bShutdown)
		{
			return LocalInner->RegistrationLock;
		}
	}
	return TSharedPtr<UE::FRecursiveMutex>();
}

void FActiveMountsLoaderWrapper::LoadAsync(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> InMountPoints,
	TFunction<void(bool bCanceled)>&& OnComplete)
{
	if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
	{
		ON_SCOPE_EXIT
		{
			Unlock();
		};
		if (!LocalInner->bShutdown)
		{
			LocalInner->LoadAsync(UARI, EventContext, InMountPoints, MoveTemp(OnComplete));
			return;
		}
	}
	EventContext.AddCompletion(MoveTemp(OnComplete), true /* bCanceled */);
}

void FActiveMountsLoaderWrapper::ReloadAllAsync(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	TFunction<void(bool bCanceled)>&& OnComplete)
{
	if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
	{
		ON_SCOPE_EXIT
		{
			Unlock();
		};
		if (!LocalInner->bShutdown)
		{
			LocalInner->ReloadAllAsync(UARI, EventContext, MoveTemp(OnComplete));
			return;
		}
	}
	EventContext.AddCompletion(MoveTemp(OnComplete), true /* bCanceled */);
}

void FActiveMountsLoaderWrapper::UnloadAsync(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> InMountPoints,
	TFunction<void(bool bCanceled)>&& OnComplete)
{
	if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
	{
		ON_SCOPE_EXIT
		{
			Unlock();
		};
		if (!LocalInner->bShutdown)
		{
			LocalInner->UnloadAsync(UARI, EventContext, InMountPoints, MoveTemp(OnComplete));
			return;
		}
	}
	EventContext.AddCompletion(MoveTemp(OnComplete), true /* bCanceled */);
}

void FActiveMountsLoaderWrapper::UnloadAllUnmountedAsync(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	TFunction<void(bool bCanceled)>&& OnComplete)
{
	if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
	{
		ON_SCOPE_EXIT
		{
			Unlock();
		};
		if (!LocalInner->bShutdown)
		{
			LocalInner->UnloadAllUnmountedAsync(UARI, EventContext, MoveTemp(OnComplete));
			return;
		}
	}
	EventContext.AddCompletion(MoveTemp(OnComplete), true /* bCanceled */);
}

bool FActiveMountsLoaderWrapper::IsLoaded(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint)
{
	if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
	{
		ON_SCOPE_EXIT
		{
			Unlock();
		};
		if (!LocalInner->bShutdown)
		{
			return LocalInner->IsLoaded(UARI, EventContext, MountPoint);
		}
	}
	return false;
}

bool FActiveMountsLoaderWrapper::IsInProgress(UAssetRegistryImpl& UARI)
{
	if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
	{
		ON_SCOPE_EXIT
		{
			Unlock();
		};
		if (!LocalInner->bShutdown)
		{
			return LocalInner->IsInProgress(UARI);
		}
	}
	return false;
}

void FActiveMountsLoaderWrapper::ReportWhenIdle(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	TFunction<void()> OnIdle)
{
	if (!OnIdle)
	{
		return;
	}
	if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
	{
		ON_SCOPE_EXIT
		{
			Unlock();
		};
		if (!LocalInner->bShutdown)
		{
			return LocalInner->ReportWhenIdle(UARI, EventContext, MoveTemp(OnIdle));
		}
	}
	EventContext.AddIdle(MoveTemp(OnIdle));
}

void FActiveMountsLoaderWrapper::OnActiveMountsInitialize(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	FInterfaceWriteScopeLock& ProofOfWriteLock)
{
	if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
	{
		ON_SCOPE_EXIT
		{
			Unlock();
		};
		if (!LocalInner->bShutdown)
		{
			LocalInner->OnActiveMountsInitialize(UARI, EventContext, ProofOfWriteLock);
		}
	}
}

bool FActiveMountsLoaderWrapper::TryRegisterAssetProviderAndAppendStateOutsideARLock(UAssetRegistryImpl& UARI,
	FEventContext& AREventContext, FActiveMountEvents& EventContext, FStringView InStateFilePath)
{
	FString StateFilePath = UE::AssetRegistry::CreateStandardFilename(InStateFilePath);
	TSharedPtr<UE::FRecursiveMutex> RegistrationLock = GetRegistrationLock();
	if (RegistrationLock)
	{
		UE::TUniqueLock RegistrationScopeLock(*RegistrationLock);
		if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
		{
			ON_SCOPE_EXIT
			{
				Unlock();
			};
			if (!LocalInner->bShutdown)
			{
				return LocalInner->TryRegisterAssetProviderAndAppendStateOutsideARLock(UARI, AREventContext,
					EventContext, StateFilePath);
			}
		}
	}

	TUniquePtr<FArchive> FileArchive = OpenReadStateFileArchive(StateFilePath);
	if (!FileArchive)
	{
		return false;
	}
	FAssetRegistryState StateToAppend;
	if (!StateToAppend.Load(*FileArchive, UE::AssetRegistry::Premade::GetLoadOptions()))
	{
		return false;
	}
	UARI.AppendState(StateToAppend);
	return true;
}

bool FActiveMountsLoaderWrapper::TryLoadStateAndRegisterAssetProvider(UAssetRegistryImpl& UARI,
	FActiveMountEvents& EventContext, FInterfaceWriteScopeLock& ProofOfWriteLock, FPakPlatformFile* PakPlatformFile,
	FStringView InStateFilePath, FAssetRegistryState& OutLoadedState)
{
	FString StateFilePath = UE::AssetRegistry::CreateStandardFilename(InStateFilePath);
	TSharedPtr<UE::FRecursiveMutex> RegistrationLock = GetRegistrationLock();
	if (RegistrationLock)
	{
		UE::TUniqueLock RegistrationScopeLock(*RegistrationLock);
		if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
		{
			ON_SCOPE_EXIT
			{
				Unlock();
			};
			if (!LocalInner->bShutdown)
			{
				return LocalInner->TryLoadStateAndRegisterAssetProvider(UARI, EventContext, ProofOfWriteLock,
					PakPlatformFile, StateFilePath, OutLoadedState);
			}
		}
	}

	// Optimization: if we're using pak files then only search paks (avoid unnecessary fallback to loose)
	bool bFileExists = PakPlatformFile != nullptr
		? PakPlatformFile->FindFileInPakFiles(*StateFilePath)
		: IFileManager::Get().FileExists(*StateFilePath);
	if (!bFileExists)
	{
		return false;
	}
	TUniquePtr<FArchive> FileArchive = OpenReadStateFileArchive(StateFilePath);
	if (!FileArchive)
	{
		return false;
	}
	return OutLoadedState.Load(*FileArchive, UE::AssetRegistry::Premade::GetLoadOptions());
}

void FActiveMountsLoaderWrapper::UnregisterAssetProvider(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	FStringView InStateFilePath)
{
	FString StateFilePath = UE::AssetRegistry::CreateStandardFilename(InStateFilePath);
	TSharedPtr<UE::FRecursiveMutex> RegistrationLock = GetRegistrationLock();
	if (RegistrationLock)
	{
		UE::TUniqueLock RegistrationScopeLock(*RegistrationLock);
		TArray<UE::Tasks::FTask> WaitOnAsync;
		if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
		{
			ON_SCOPE_EXIT
			{
				Unlock();
			};
			if (!LocalInner->bShutdown)
			{
				if (!LocalInner->TryStartUnregisterAssetProvider(UARI, EventContext, WaitOnAsync, StateFilePath))
				{
					return;
				}
			}
		}

		// Waiting has to be done outside the normal lock because the task threads have to enter the lock
		// before they can exit. It does not have to be done outside the RegistrationLock, because the
		// task threads do not use that lock.
		UE::Tasks::Wait(WaitOnAsync);

		if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
		{
			ON_SCOPE_EXIT
			{
				Unlock();
			};

			if (!LocalInner->bShutdown)
			{
				LocalInner->FinishUnregisterAssetProvider(UARI, EventContext, StateFilePath);
			}
		}
	}
}

void FActiveMountsLoaderWrapper::InitialProviderReport(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	FAssetProviderInitData& StateInitData)
{
	if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
	{
		ON_SCOPE_EXIT
		{
			Unlock();
		};
		if (!LocalInner->bShutdown)
		{
			LocalInner->InitialProviderReport(UARI, EventContext, StateInitData);
		}
	}
}

void FActiveMountsLoaderWrapper::InitialProviderOnReadyForRegister(UAssetRegistryImpl& UARI,
	FActiveMountEvents& EventContext, EInitialProviderRegisterReason Reason)
{
	if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
	{
		ON_SCOPE_EXIT
		{
			Unlock();
		};
		if (!LocalInner->bShutdown)
		{
			LocalInner->InitialProviderOnReadyForRegister(UARI, EventContext, Reason);
		}
	}
}

void FActiveMountsLoaderWrapper::InitialProviderConditionalRegister(UAssetRegistryImpl& UARI,
	FActiveMountEvents& EventContext, FAssetRegistryState& InitialProviderState)
{
	if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
	{
		ON_SCOPE_EXIT
		{
			Unlock();
		};
		if (!LocalInner->bShutdown)
		{
			LocalInner->InitialProviderConditionalRegister(UARI, EventContext, InitialProviderState);
		}
	}
}

void FActiveMountsLoaderWrapper::OnMountPointsMounted(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> InMountPoints)
{
	if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
	{
		ON_SCOPE_EXIT
		{
			Unlock();
		};
		if (!LocalInner->bShutdown)
		{
			LocalInner->OnMountPointsMounted(UARI, EventContext, InMountPoints);
		}
	}
}

void FActiveMountsLoaderWrapper::OnMountPointsDismounted(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	FInterfaceWriteScopeLock& ProofOfWriteLock,
	TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> InMountPoints)
{
	if (FActiveMountsLoader* LocalInner = LockIfEnabled(); LocalInner)
	{
		ON_SCOPE_EXIT
		{
			Unlock();
		};
		if (!LocalInner->bShutdown)
		{
			LocalInner->OnMountPointsDismounted(UARI, EventContext, ProofOfWriteLock, InMountPoints);
		}
	}
}

FAssetProvider::FAssetProvider(FStringView InFilePath)
	: FilePath(InFilePath)
{
}

FStringView FAssetProvider::GetFilePath() const
{
	return FilePath;
}

bool FAssetProvider::IsInitialProvider() const
{
	return bInitialProvider;
}

void FAssetProvider::SetIsInitialProvider(bool bValue)
{
	bInitialProvider = bValue;
}

const FixedTagPrivate::FWeakStorePtr& FAssetProvider::GetStore() const
{
	return Store;
}

void FAssetProvider::SetStore(FixedTagPrivate::FWeakStorePtr InStore)
{
	Store = MoveTemp(InStore);
}

uint32& FAssetProvider::GetLoadRefCountPtr()
{
	return LoadRefCount;
}

UE::Tasks::FTask& FAssetProvider::GetLoadTask()
{
	return LoadTask;
}

bool FAssetProvider::IsShutdown() const
{
	return bShutdown;
}

void FAssetProvider::SetIsShutdown(bool bValue)
{
	bShutdown = bValue;
}

bool FAssetProvider::HasMissingMountPoints() const
{
	return bHasMissingMountPoints;
}

void FAssetProvider::SetHasMissingMountPoints(bool bValue)
{
	bHasMissingMountPoints = bValue;
}

bool FAssetProvider::IsErrorNoUnload() const
{
	return bErrorNoUnload;
}

void FAssetProvider::SetIsErrorNoUnload(bool bValue)
{
	bErrorNoUnload = bValue;
}

FAssetProviderSetKey::FAssetProviderSetKey(TUniquePtr<FAssetProvider>&& InAssetProvider)
	: AssetProvider(MoveTemp(InAssetProvider))
{
}

bool FAssetProviderSetKey::operator==(const FAssetProviderSetKey& Other) const
{
	return AssetProvider.Get() == Other.AssetProvider.Get();
}

bool FAssetProviderSetKey::operator==(FStringView OtherProviderFilePath) const
{
	return AssetProvider->GetFilePath().Equals(OtherProviderFilePath, ESearchCase::IgnoreCase);
}

FAssetProvider& FAssetProviderSetKey::GetAssetProvider() const
{
	return *AssetProvider;
}

uint32 GetTypeHash(const FAssetProviderSetKey& Key)
{
	return GetTypeHash(Key.GetAssetProvider().GetFilePath());
}

FMountPointData::FMountPointData(TRefCountPtr<UE::PackageName::IMountPoint> InMountPoint, EMountState InInitialState)
	: MountPoint(MoveTemp(InMountPoint))
{
	SetCurrentState(InInitialState);
	SetTargetState(InInitialState);
}

bool FMountPointData::operator==(const FMountPointData& Other) const
{
	return MountPoint.GetReference() == Other.MountPoint.GetReference();
}

bool FMountPointData::operator==(UE::PackageName::IMountPoint* InMountPoint) const
{
	return MountPoint.GetReference() == InMountPoint;
}

uint32 GetTypeHash(const FMountPointData& Key)
{
	return ::GetTypeHash(Key.GetMountPoint().GetReference());
}

const TRefCountPtr<UE::PackageName::IMountPoint>& FMountPointData::GetMountPoint() const
{
	return MountPoint;
}

FAssetProvider* FMountPointData::GetAssetProvider() const
{
	return AssetProvider;
}

void FMountPointData::SetAssetProvider(FAssetProvider* InAssetProvider)
{
	AssetProvider = InAssetProvider;
}

EMountState FMountPointData::GetCurrentState() const
{
	return static_cast<EMountState>(CurrentStateBits);
}

void FMountPointData::SetCurrentState(EMountState InState)
{
	CurrentStateBits = static_cast<uint32>(InState);
}

EMountState FMountPointData::GetTargetState() const
{
	return static_cast<EMountState>(TargetStateBits);
}

void FMountPointData::SetTargetState(EMountState InState)
{
	TargetStateBits = static_cast<uint32>(InState);
}

bool FMountPointData::IsErrorNoUnload() const
{
	return ErrorNoUnloadBit != 0;
}

void FMountPointData::SetIsErrorNoUnload(bool bValue)
{
	ErrorNoUnloadBit = bValue ? 1 : 0;
}

bool FMountPointData::IsMultiSegmentName() const
{
	return MultiSegmentNameBit != 0;
}

void FMountPointData::SetIsMultiSegmentName(bool bValue)
{
	MultiSegmentNameBit = bValue ? 1 : 0;
}

bool FMountPointData::IsParentOrChild() const
{
	return IsParentOrChildBit != 0;
}

void FMountPointData::SetIsParentOrChild(bool bValue)
{
	IsParentOrChildBit = bValue ? 1 : 0;
}

bool FMountPointData::HasMultipleProviders() const
{
	return HasMultipleProvidersBit != 0;
}

void FMountPointData::SetHasMultipleProviders(bool bValue)
{
	HasMultipleProvidersBit = bValue ? 1 : 0;
}

void FActiveMountsLoader::StartShutdown(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext, TArray<UE::Tasks::FTask>& OutWaitOnAsync)
{
	// If bShutdown is already true, then another thread has already started shutdown. We can both race to see who
	// gets to reenter the lock after waiting on async and call FinishShutdown. We still need to wait on this thread
	// for the tasks to complete before this (or any) thread is allowed to call FinishShutdown.
	if (!bShutdown)
	{
		// Set bShutdown=true to signal to all taskthreads that they should shutdown.
		bShutdown = true;
	}
	// Tell our caller to wait for tasks to complete (outside our lock) before we call FinishShutdown to delete data
	// out from under them.
	if (UnloadTask.IsValid())
	{
		OutWaitOnAsync.Add(UnloadTask);
	}
	for (FAssetProviderSetKey& SetKey : AssetProviders)
	{
		UE::Tasks::FTask& LoadTask = SetKey.GetAssetProvider().GetLoadTask();
		if (LoadTask.IsValid())
		{
			OutWaitOnAsync.Add(LoadTask);
		}
	}
}

void FActiveMountsLoader::FinishShutdown(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext)
{
	using namespace UE::PackageName;
	using FSubscriberArray = TArray<TRefCountPtr<FMountLoadEventSubscriber>>;

	// Caller should have cleared UnloadTask via StartShutdown and WaitOnAsync.
	check(!UnloadTask.IsValid());
	for (TPair<TRefCountPtr<IMountPoint>, FSubscriberArray>& LoadPair : LoadSubscribers)
	{
		for (TRefCountPtr<FMountLoadEventSubscriber>& Subscriber : LoadPair.Value)
		{
			EventContext.AddCompletion(MoveTemp(Subscriber->OnComplete), true /* bWasCanceled */);
			Subscriber.SafeRelease();
		}
	}
	LoadSubscribers.Empty();
	for (TPair<TRefCountPtr<IMountPoint>, FSubscriberArray>& UnloadPair : UnloadSubscribers)
	{
		for (TRefCountPtr<FMountLoadEventSubscriber>& Subscriber : UnloadPair.Value)
		{
			EventContext.AddCompletion(MoveTemp(Subscriber->OnComplete), true /* bWasCanceled */);
			Subscriber.SafeRelease();
		}
	}
	UnloadSubscribers.Empty();
	for (TFunction<void()>& OnIdle : OnIdleSubscribers)
	{
		EventContext.AddIdle(MoveTemp(OnIdle));
	}
	OnIdleSubscribers.Empty();

	// MountPoints have a raw pointer to AssetProviders, AssetProviders do not have a pointer to MountPoints.
	// MountPoints do not currently access their AssetProvider during their destructor, but in case that changes,
	// clear them first.
	MountPoints.Empty();
#if DO_CHECK
	for (FAssetProviderSetKey& SetKey : AssetProviders)
	{
		FAssetProvider& AssetProvider = SetKey.GetAssetProvider();
		// Caller should have cleared LoadTask via StartShutdown and WaitOnAsync.
		check(!AssetProvider.GetLoadTask().IsValid());
	}
#endif
	AssetProviders.Empty(); // TUniquePtr within the FAssetProviderSetKey deletes the AssetProviders.
}

void FActiveMountsLoader::LoadAsync(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> InMountPoints,
	TFunction<void(bool bCanceled)>&& OnComplete)
{
	using namespace UE::PackageName;

	if (!bLoaderIsActive)
	{
		// We assume that the initial provider has been registered for a few reasons, such as being able to assume
		// that no MountPointData implies no AssetProvider has assets in that MountPoint.
		// To support that assumption we have to early exit here. We know that we can report completion in this case
		// because we do not unload anything until that point either.
		EventContext.AddCompletion(MoveTemp(OnComplete), false /* bCanceled */);
		return;
	}

	TArray<TRefCountPtr<IMountPoint>> WaitForCompletionMounts;
	TArray<TRefCountPtr<IMountPoint>> TriggerMounts;
	for (const TRefCountPtr<IMountPoint>& MountPoint : InMountPoints)
	{
		if (MountPoint == nullptr)
		{
			continue;
		}

		FMountPointData* MountPointData = FindMountPointData(MountPoint.GetReference());
		if (!MountPointData)
		{
			continue;
		}
		FAssetProvider* AssetProvider = MountPointData->GetAssetProvider();
		if (!AssetProvider || AssetProvider->IsShutdown())
		{
			continue;
		}

		if (MountPointData->GetCurrentState() != EMountState::Loaded)
		{
			if (MountPointData->GetTargetState() != EMountState::Loaded)
			{
				TriggerMounts.Add(MountPoint);
			}
			if (OnComplete)
			{
				WaitForCompletionMounts.Add(MountPoint);
			}
		}
	}

	if (OnComplete)
	{
		// Create the subscriber before triggering any loads, because we may find that we can
		// complete a Load immediately and we will need to decrement the subscriber's RefCount,
		// so we need to have set it up already.
		if (WaitForCompletionMounts.IsEmpty())
		{
			// We only added to TriggerMounts in the case where we also added to WaitForCompletionMounts, so assert empty.
			check(TriggerMounts.IsEmpty());
			EventContext.AddCompletion(MoveTemp(OnComplete), false /* bCanceled */);
		}
		else
		{
			TRefCountPtr<FMountLoadEventSubscriber> Subscriber = MakeRefCount<FMountLoadEventSubscriber>();
			Subscriber->OnComplete = MoveTemp(OnComplete);
			Subscriber->RemainingCount = WaitForCompletionMounts.Num();
			Subscriber->bIsLoadSubscriber = true;
			for (TRefCountPtr<IMountPoint>& WaitMount : WaitForCompletionMounts)
			{
				LoadSubscribers.FindOrAdd(MoveTemp(WaitMount)).Add(Subscriber);
			}
			WaitForCompletionMounts.Empty();
		}
	}

	for (const TRefCountPtr<IMountPoint>& TriggerMount : TriggerMounts)
	{
		FMountPointData* TriggerData = FindMountPointData(TriggerMount.GetReference());
		check(TriggerData); // We only added it to TriggerMounts if it existed
		// We only added the mount to TriggerMounts if GetTargetState() != Loaded, and no other mount states
		// (in particular, no transition mount states like Loading) are allowed in GetTargetState fields,
		// so we know the target state must be Unloaded.
		check(TriggerData->GetTargetState() == EMountState::Unloaded);
		switch (TriggerData->GetCurrentState())
		{
		case EMountState::Unloaded:
			TriggerData->SetTargetState(EMountState::Loaded);
			TriggerData->SetCurrentState(EMountState::Loading);
			check(TriggerData->GetAssetProvider()); // We only added it to TriggerMounts if it had an AssetProvider.
			AddRefMountPointLoad(UARI, TriggerData);
			break;
		case EMountState::Loading:
			// We previously received a load request that we're still working on, and after that but before now we
			// received an unload request that superseded it, but that had to wait until the load finishes before
			// it could operate. And now we have received a load request that supersedes the superseder.
			// Mark the unload as completed but canceled, and set targetstate back to loaded.
			TriggerData->SetTargetState(EMountState::Loaded);
			NotifyMountPointUnloadComplete(EventContext, TriggerMount, true /* bCanceled */);
			break;
		case EMountState::Loaded:
			check(false); // We only added it to TriggerMounts if it was not in Loaded
			break;
		case EMountState::Unloading:
			// We previously received an unload request that we are still working on, but now we have a load
			// request that supersedes it, but we have to wait for the unload to finish before we can start
			// working on the load. Mark the unload as completed but canceled, and set targetstate to loaded.
			TriggerData->SetTargetState(EMountState::Loaded);
			NotifyMountPointUnloadComplete(EventContext, TriggerMount, true /* bCanceled */);
			break;
		default:
			checkNoEntry();
			break;
		}
	}
}

void FActiveMountsLoader::ReloadAllAsync(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	TFunction<void(bool bCanceled)>&& OnComplete)
{
	if (!bLoaderIsActive)
	{
		// See the comment in LoadAsync. We set bCanceled=false because we know all mountpoints are loaded.
		EventContext.AddCompletion(MoveTemp(OnComplete), false /* bCanceled */);
		return;
	}

	TArray<TRefCountPtr<UE::PackageName::IMountPoint>> MountPointsToLoad;
	for (FMountPointData& MountData : MountPoints)
	{
		FAssetProvider* AssetProvider = MountData.GetAssetProvider();
		if (!AssetProvider || AssetProvider->IsShutdown())
		{
			continue;
		}

		// Reload from AssetRegistry the MountPoints that we have previously unmounted.
		// Include MountPoints that are in the process of loading but have not finished loading yet, because we need
		// to wait on those before reporting to our caller.
		// Include MountPoints that are loaded but that have been requested to unload, because we need to cancel those
		// unloads.
		if (MountData.GetCurrentState() != EMountState::Loaded ||
			MountData.GetTargetState() != EMountState::Loaded)
		{
			const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint = MountData.GetMountPoint();
			MountPointsToLoad.Add(MountPoint);
		}
	}
	LoadAsync(UARI, EventContext, MountPointsToLoad, MoveTemp(OnComplete));
}

void FActiveMountsLoader::UnloadAsync(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> InMountPoints,
	TFunction<void(bool bCanceled)>&& OnComplete)
{
	using namespace UE::PackageName;

	if (!bLoaderIsActive)
	{
		ensureMsgf(false, TEXT("UnloadAsync called before ActiveMountsInitialized. The unload call will be ignored."));
		EventContext.AddCompletion(MoveTemp(OnComplete), true /* bCanceled */);
		return;
	}

	TArray<TRefCountPtr<IMountPoint>> WaitForCompletionMounts;
	TArray<TRefCountPtr<IMountPoint>> TriggerMounts;
	for (const TRefCountPtr<IMountPoint>& MountPoint : InMountPoints)
	{
		if (MountPoint == nullptr)
		{
			continue;
		}

		FMountPointData* MountPointData = FindMountPointData(MountPoint.GetReference());
		if (!MountPointData)
		{
			continue;
		}
		FAssetProvider* AssetProvider = MountPointData->GetAssetProvider();
		if (!AssetProvider || AssetProvider->IsShutdown())
		{
			continue;
		}
		if (MountPointData->IsErrorNoUnload() || AssetProvider->IsErrorNoUnload())
		{
			continue;
		}

		if (MountPointData->GetCurrentState() != EMountState::Unloaded)
		{
			if (MountPointData->GetTargetState() != EMountState::Unloaded)
			{
				TriggerMounts.Add(MountPoint);
			}
			if (OnComplete)
			{
				WaitForCompletionMounts.Add(MountPoint);
			}
		}
	}

	if (OnComplete)
	{
		// Create the subscriber before triggering any loads, because we may find that we can
		// complete an unload immediately and we will need to decrement the subscriber's RefCount,
		// so we need to have set it up already.
		if (WaitForCompletionMounts.IsEmpty())
		{
			// We only added to TriggerMounts in the case where we also added to WaitForCompletionMounts, so assert empty.
			check(TriggerMounts.IsEmpty());
			EventContext.AddCompletion(MoveTemp(OnComplete), false /* bCanceled */);
		}
		else
		{
			TRefCountPtr<FMountLoadEventSubscriber> Subscriber = MakeRefCount<FMountLoadEventSubscriber>();
			Subscriber->OnComplete = MoveTemp(OnComplete);
			Subscriber->RemainingCount = WaitForCompletionMounts.Num();
			Subscriber->bIsLoadSubscriber = false;
			for (TRefCountPtr<IMountPoint>& WaitMount : WaitForCompletionMounts)
			{
				UnloadSubscribers.FindOrAdd(MoveTemp(WaitMount)).Add(Subscriber);
			}
			WaitForCompletionMounts.Empty();
		}
	}

	for (const TRefCountPtr<IMountPoint>& TriggerMount : TriggerMounts)
	{
		FMountPointData* TriggerData = FindMountPointData(TriggerMount.GetReference());
		check(TriggerData); // We only added it to TriggerMounts if it existed
		// We only added the mount to TriggerMounts if GetTargetState() != Unloaded, and no other mount states
		// (in particular, no transition mount states like Loading) are allowed in GetTargetState fields,
		// so we know the target state must be Loaded.
		check(TriggerData->GetTargetState() == EMountState::Loaded);
		switch (TriggerData->GetCurrentState())
		{
		case EMountState::Unloaded:
			check(false); // We only added it to TriggerMounts if it was not in Unloaded
			break;
		case EMountState::Loading:
			// We previously received a load request that we are still working on, but now we have an unload
			// request that supersedes it, but we have to wait for the load to finish before we can start working
			// on the unload. Mark the load as completed but canceled, and set targetstate to unloaded.
			TriggerData->SetTargetState(EMountState::Unloaded);
			NotifyMountPointLoadComplete(EventContext, TriggerMount, true /* bCanceled */);
			break;
		case EMountState::Loaded:
			TriggerData->SetTargetState(EMountState::Unloaded);
			TriggerData->SetCurrentState(EMountState::Unloading);
			check(TriggerData->GetAssetProvider()); // We only added it to TriggerMounts if it had an AssetProvider.
			AddRefMountPointUnload(UARI, TriggerData);
			break;
		case EMountState::Unloading:
			// We previously received an unload request that we're still working on, and after that but before now
			// we received a load request that superseded it, but that had to wait until the unload finishes before
			// it could operate. And now we have received an unload request that supersedes the superseder.
			// Mark the load as completed but canceled, and set targetstate back to unloaded.
			TriggerData->SetTargetState(EMountState::Unloaded);
			NotifyMountPointLoadComplete(EventContext, TriggerMount, true /* bCanceled */);
			break;
		default:
			checkNoEntry();
			break;
		}
	}
}

void FActiveMountsLoader::UnloadAllUnmountedAsync(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	TFunction<void(bool bCanceled)>&& OnComplete)
{
	if (!bLoaderIsActive)
	{
		ensureMsgf(false, TEXT("UnloadAllUnmountedAsync called before ActiveMountsInitialized. The unload call will be ignored."));
		EventContext.AddCompletion(MoveTemp(OnComplete), true /* bCanceled */);
		return;
	}

	TArray<TRefCountPtr<UE::PackageName::IMountPoint>> MountPointsToUnload;
	for (FMountPointData& MountData : MountPoints)
	{
		FAssetProvider* AssetProvider = MountData.GetAssetProvider();
		if (!AssetProvider || AssetProvider->IsShutdown())
		{
			continue;
		}
		if (MountData.IsErrorNoUnload() || AssetProvider->IsErrorNoUnload())
		{
			continue;
		}
		const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint = MountData.GetMountPoint();

		// Unload from AssetRegistry the MountPoints that PackageName reports are not currently mounted.
		if (!MountPoint->IsMounted())
		{
			MountPointsToUnload.Add(MountPoint);
		}
	}
	UnloadAsync(UARI, EventContext, MountPointsToUnload, MoveTemp(OnComplete));
}

bool FActiveMountsLoader::IsLoaded(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint)
{
	FMountPointData* MountData = MountPoints.FindByHash(
		GetTypeHash(MountPoint.GetReference()), MountPoint.GetReference());
	if (!MountData)
	{
		return false;
	}
	return MountData->GetCurrentState() == EMountState::Loaded;
}

bool FActiveMountsLoader::IsInProgress(UAssetRegistryImpl& UARI)
{
	if (UnloadTask.IsValid())
	{
		return true;
	}
	for (FAssetProviderSetKey& SetKey : AssetProviders)
	{
		FAssetProvider& AssetProvider = SetKey.GetAssetProvider();
		if (AssetProvider.GetLoadTask().IsValid())
		{
			return true;
		}
	}
	return false;
}

void FActiveMountsLoader::ReportWhenIdle(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	TFunction<void()> OnIdle)
{
	if (!IsInProgress(UARI))
	{
		EventContext.AddIdle(OnIdle);
		return;
	}
	OnIdleSubscribers.Add(MoveTemp(OnIdle));
}

void FActiveMountsLoader::OnActiveMountsInitialize(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	FInterfaceWriteScopeLock& ProofOfWriteLock)
{
	if (bLoaderIsActive)
	{
		return;
	}

	InitialProviderOnReadyForRegister(UARI, EventContext, EInitialProviderRegisterReason::ActiveMountsInitialize);
	// Note that we require the ProofOfWriteLock around the read of UARI.GuardedData.State.
	InitialProviderConditionalRegister(UARI, EventContext, UARI.GuardedData.State);
	bLoaderIsActive = true;
	UE_LOGFMT(LogAssetRegistry, Log, "ActiveMountsInitialize called. AssetRegistry data unloading is now available.");
}

bool FActiveMountsLoader::TryRegisterAssetProviderAndAppendStateOutsideARLock(UAssetRegistryImpl& UARI,
	FEventContext& AREventContext, FActiveMountEvents& EventContext, FStringView StateFilePath)
{
	FAssetProviderSetKey* ExistingSetKey = AssetProviders.FindByHash(GetTypeHash(StateFilePath), StateFilePath);
	if (ExistingSetKey)
	{
		// Handling register/unregister races between different threads without a separate critical section would
		// be complicated, so we require that our caller enter the registration critical section that is also entered
		// when unregistering. AssetProvider.IsShutdown will only ever be set when the AssetProvider is also atomically
		// cleared within that critical section. Therefore we cannot find an IsShutdown() == true in this function.
		check(!ExistingSetKey->GetAssetProvider().IsShutdown());
		return true;
	}

	// Load from disk outside of the lock
	FAssetRegistryState StateToAppend;
	FAssetRegistryLoadResults LoadResults;
	FActiveMountsLoader* pNewValueOfThis;
	{
		UARI.ActiveMountsLoader.Unlock();
		ON_SCOPE_EXIT
		{
			UARI.ActiveMountsLoader.Lock();
		};

		TUniquePtr<FArchive> FileArchive = OpenReadStateFileArchive(StateFilePath);
		if (!FileArchive)
		{
			return false;
		}

		FAssetRegistryLoadOptions LoadOptions = UE::AssetRegistry::Premade::GetLoadOptions();
		check(LoadOptions.bMountPointsRequireSeparateAllocationBlocks == true);
		if (!StateToAppend.Load(*FileArchive, LoadOptions, LoadResults))
		{
			return false;
		}

		{
			// We have to enter the AssetRegistry lock around the read of UARI.GuardedData.State. Note that
			// we also have to enter the AssetRegistry lock before entering ActiveMountsLoader lock, to match required
			// lock order and avoid deadlocks, so it is important that we have already exited the ActiveMountsLoader
			// lock above.
			UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(UARI.InterfaceLock);

			// Reenter the ActiveMountsLoader lock to be able to access ActiveMounts data during InitialProviderOnReadyForRegister
			{
				UE::TScopeLock InnerScopeLockOfActiveMountsLock(UARI.ActiveMountsLoader.ActiveMountsLock);
				// After reentering the lock, check again for any early exits or earlier calculations that may have changed.
				// DELETED THIS-POINTER WARNING: Check pNewValueOfThis!=nullptr first before reading any data, this
				// may have been deleted while outside the lock.
				pNewValueOfThis = UARI.ActiveMountsLoader.Inner.load(std::memory_order_relaxed);
				if (!pNewValueOfThis || pNewValueOfThis->bShutdown)
				{
					return false;
				}
				if (ExistingSetKey = AssetProviders.FindByHash(GetTypeHash(StateFilePath), StateFilePath); ExistingSetKey)
				{
					check(!ExistingSetKey->GetAssetProvider().IsShutdown());
					return true;
				}

				// Do the work we can only do while inside both locks
				InitialProviderOnReadyForRegister(UARI, EventContext, EInitialProviderRegisterReason::AppendState);
				InitialProviderConditionalRegister(UARI, EventContext, UARI.GuardedData.State);

				// Call AppendState below outside of the ActiveMountsLoader lock for better performance.
			}

			// Our contract states that we AppendState as well, so do that now. We have to do it only after reading data
			// from UARI.GuardedData.State above.
			UARI.GuardedData.AppendState(AREventContext, StateToAppend, EAppendMode::Append, /*bEmitAssetEvents*/ true);
		}
	}
	// After reentering the lock, check again for any early exits or earlier calculations that may have changed.
	// DELETED THIS-POINTER WARNING: Check pNewValueOfThis!=nullptr first before reading any data, this
	// may have been deleted while outside the lock.
	pNewValueOfThis = UARI.ActiveMountsLoader.Inner.load(std::memory_order_relaxed);
	if (!pNewValueOfThis || pNewValueOfThis->bShutdown)
	{
		return false;
	}
	if (ExistingSetKey = AssetProviders.FindByHash(GetTypeHash(StateFilePath), StateFilePath); ExistingSetKey)
	{
		check(!ExistingSetKey->GetAssetProvider().IsShutdown());
		return true;
	}

	TUniquePtr<FAssetProvider> AssetProviderPtr(new FAssetProvider(StateFilePath));
	FAssetProvider& AssetProvider = *AssetProviderPtr;
	AssetProvider.SetStore(LoadResults.Store);
	bool bAlreadyExisted;
	AssetProviders.Add(FAssetProviderSetKey(MoveTemp(AssetProviderPtr)), &bAlreadyExisted);
	// Assert !bAlreadyExisted because we would expensively destroy and recreate if it did and would need to not call
	// ParseInitialState. bAlreadyExisted cannot occur because we tested AssetProvider.FindByHash above.
	check(!bAlreadyExisted);

	ParseInitialState(AssetProvider, StateToAppend);
	return true;
}

bool FActiveMountsLoader::TryLoadStateAndRegisterAssetProvider(UAssetRegistryImpl& UARI,
	FActiveMountEvents& EventContext, FInterfaceWriteScopeLock& ProofOfWriteLock, FPakPlatformFile* PakPlatformFile,
	FStringView StateFilePath, FAssetRegistryState& OutLoadedState)
{
	// This function uses a legacy behavior that causes poor performance: load the file off disk inside
	// the AssetRegistry lock (and to make it even easier, also inside the ActiveMounts lock). It is used only for
	// the load of cooked AssetRegistry.bin for built-in plugins during the startup LoadPremadeAssetRegistry.
	// If it becomes a performance issue, we will instead need to trigger these loads asynchronously and delay the
	// AssetRegistry's reporting of bPreloadingComplete=true that happens at the end of LoadPremadeAssetRegistry
	// until after the asynchronous loads have completed.

	// Optimization: if we're using pak files then only search paks (avoid unnecessary fallback to loose)
	TStringBuilder<256> FilePathStr(InPlace, StateFilePath);
	bool bFileExists = PakPlatformFile != nullptr
		? PakPlatformFile->FindFileInPakFiles(*FilePathStr)
		: IFileManager::Get().FileExists(*FilePathStr);
	if (!bFileExists)
	{
		return false;
	}
	TUniquePtr<FArchive> FileArchive = OpenReadStateFileArchive(StateFilePath);
	if (!FileArchive)
	{
		return false;
	}

	FAssetRegistryLoadOptions LoadOptions = UE::AssetRegistry::Premade::GetLoadOptions();
	check(LoadOptions.bMountPointsRequireSeparateAllocationBlocks == true);
	FAssetRegistryLoadResults LoadResults;
	if (!OutLoadedState.Load(*FileArchive, LoadOptions, LoadResults))
	{
		return false;
	}

	FAssetProviderSetKey* ExistingSetKey = AssetProviders.FindByHash(GetTypeHash(StateFilePath), StateFilePath);
	if (ExistingSetKey)
	{
		FAssetProvider& ExistingAssetProvider = ExistingSetKey->GetAssetProvider();
		// Handling register/unregister races between different threads without a separate critical section would
		// be complicated, so we require that our caller enter the registration critical section that is also entered
		// when unregistering. AssetProvider.IsShutdown will only ever be set when the AssetProvider is also atomically
		// cleared within that critical section. Therefore we cannot find an IsShutdown() == true in this function.
		check(!ExistingAssetProvider.IsShutdown());
		return true;
	}

	TUniquePtr<FAssetProvider> AssetProviderPtr(new FAssetProvider(StateFilePath));
	FAssetProvider& AssetProvider = *AssetProviderPtr;
	AssetProvider.SetStore(LoadResults.Store);
	bool bAlreadyExisted;
	AssetProviders.Add(FAssetProviderSetKey(MoveTemp(AssetProviderPtr)), &bAlreadyExisted);
	// Assert !bAlreadyExisted because we would expensively destroy and recreate if it did and would need to not call
	// ParseInitialState. bAlreadyExisted cannot occur because we tested AssetProviders.FindByHash above.
	check(!bAlreadyExisted);

	ParseInitialState(AssetProvider, OutLoadedState);

	// Since we successfully loaded the AssetRegistry file, we expect our caller to call AppendState on it, so we have
	// to trigger the registration of the initial provider now to satisfy the requirement that GuardedData.State is
	// composed only of AssetDatas from the initial provider.
	InitialProviderOnReadyForRegister(UARI, EventContext, EInitialProviderRegisterReason::AppendState);
	InitialProviderConditionalRegister(UARI, EventContext, UARI.GuardedData.State);

	return true;
}

bool FActiveMountsLoader::TryStartUnregisterAssetProvider(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	TArray<UE::Tasks::FTask>& OutWaitOnAsync, FStringView StateFilePath)
{
	FAssetProviderSetKey* SetKey = AssetProviders.FindByHash(GetTypeHash(StateFilePath), StateFilePath);
	if (!SetKey)
	{
		return false;
	}
	FAssetProvider& AssetProvider = SetKey->GetAssetProvider();
	if (AssetProvider.IsInitialProvider())
	{
		ensureMsgf(false, TEXT("ActiveMounts received a request to unregister an AssetProvider, but the filepath is the global premade AssetRegistry, which is not unregisterable. ")
			TEXT("Ignoring the unregister request. FilePath = %s"),
			*FString(StateFilePath));
		return false;
	}

	// Handling register/unregister races between different threads without a separate critical section would
	// be complicated, so we require that our caller enter the registration critical section.
	// AssetProvider.IsShutdown will only ever be set when the AssetProvider is also atomically cleared within that
	// critical section. Therefore we cannot find an IsShutdown() already true in this function.
	check(!AssetProvider.IsShutdown());
	AssetProvider.SetIsShutdown(true);

	if (AssetProvider.GetLoadTask().IsValid())
	{
		OutWaitOnAsync.Add(AssetProvider.GetLoadTask());
	}
	return true;
}

void FActiveMountsLoader::FinishUnregisterAssetProvider(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	FStringView StateFilePath)
{
	FAssetProviderSetKey* SetKey = AssetProviders.FindByHash(GetTypeHash(StateFilePath), StateFilePath);
	// Handling register/unregister races between different threads without a separate critical section would
	// be complicated, so we require that our caller enter the registration critical section.
	// The AssetProvider was previously found in StartUnregisterAssetProvider within the registration critical section,
	// so it must still exist now.
	check(SetKey);
	FAssetProvider& AssetProvider = SetKey->GetAssetProvider();

	for (TSet<FMountPointData>::TIterator Iter(MountPoints); Iter; ++Iter)
	{
		FMountPointData& MountData = *Iter;
		if (MountData.GetAssetProvider() == &AssetProvider)
		{
			NotifyMountPointLoadComplete(EventContext, MountData.GetMountPoint(), true /* bCanceled */);
			NotifyMountPointUnloadComplete(EventContext, MountData.GetMountPoint(), true /* bCanceled */);

			if (MountData.HasMultipleProviders())
			{
				MountData.SetAssetProvider(nullptr);
			}
			else
			{
				Iter.RemoveCurrent();
				continue;
			}
		}
	}
	MountPoints.Shrink();
	LoadSubscribers.Shrink();
	UnloadSubscribers.Shrink();

	AssetProviders.Remove(*SetKey);
	AssetProviders.Shrink();
}

void FActiveMountsLoader::InitialProviderReport(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	FAssetProviderInitData& StateInitData)
{
	if (bInitialProviderReported)
	{
		return;
	}
	bInitialProviderReported = true;
	FString StateFilePath = UE::AssetRegistry::CreateStandardFilename(StateInitData.StateFilePath);
	TUniquePtr<FAssetProvider> AssetProviderPtr(new FAssetProvider(StateFilePath));
	FAssetProvider& AssetProvider = *AssetProviderPtr;
	AssetProvider.SetStore(StateInitData.StoreUsedByState);

	bool bAlreadyExisted = false;
	AssetProviders.Add(FAssetProviderSetKey(MoveTemp(AssetProviderPtr)), &bAlreadyExisted);
	if (bAlreadyExisted)
	{
		UE_LOGFMT(LogAssetRegistry, Warning, "Premade Assetregistry uses a filename that was already used in another call to AppendState. "
			"The earlier call prematurely registered it, and AssetRegistry data for assets without MountPoints currently known "
			"in FPackageName at the time of its earlier registration will remain loaded for the entire process.");
		return;
	}
	AssetProvider.SetIsInitialProvider(true);
}

void FActiveMountsLoader::InitialProviderOnReadyForRegister(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	EInitialProviderRegisterReason Reason)
{
	if (bInitialProviderReadyForRegister)
	{
		return;
	}

	switch (Reason)
	{
	case EInitialProviderRegisterReason::ActiveMountsInitialize:
		// Expected flow, no notes
		break;
	case EInitialProviderRegisterReason::EarlierState:
		UE_LOGFMT(LogAssetRegistry, Error,
			"Assets were added to the AssetRegistry before LoadPremadeAssetRegistry was called. "
			"We will prematurely register the premade AssetRegistry with ActiveMountsLoader. "
			"AssetRegistry data for assets without MountPoints currently known in FPackageName will remain loaded for the entire process.");
		break;
	case EInitialProviderRegisterReason::AppendState:
		UE_LOGFMT(LogAssetRegistry, Log,
			"AppendState was called with another AssetRegistry before ActiveMountsInitialize was called. Triggering ActiveMountsInitialize.");
		break;
	default:
		checkNoEntry();
		break;
	}
	bInitialProviderReadyForRegister = true;
}

void FActiveMountsLoader::InitialProviderConditionalRegister(UAssetRegistryImpl& UARI,
	FActiveMountEvents& EventContext, FAssetRegistryState& InitialProviderState)
{
	if (bInitialProviderRegistered
		|| !bInitialProviderReported || !bInitialProviderReadyForRegister)
	{
		return;
	}

	for (FAssetProviderSetKey& SetKey : AssetProviders)
	{
		FAssetProvider& AssetProvider = SetKey.GetAssetProvider();
		if (AssetProvider.IsInitialProvider())
		{
			ParseInitialState(AssetProvider, InitialProviderState);
			break;
		}
	}
	bInitialProviderRegistered = true;
}

void FActiveMountsLoader::OnMountPointsMounted(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> InMountPoints)
{
	if (InMountPoints.IsEmpty())
	{
		return;
	}
	if (bLoaderIsActive && bAutoUnload && bAutoUnloadInitialUnloadTriggered)
	{
		LoadAsync(UARI, EventContext, InMountPoints, TFunction<void(bool bCanceled)>());
	}
}

void FActiveMountsLoader::OnMountPointsDismounted(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
	FInterfaceWriteScopeLock& ProofOfWriteLock,
	TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> InMountPoints)
{
	if (InMountPoints.IsEmpty())
	{
		return;
	}
	if (bAutoUnload)
	{
		if (!bAutoUnloadInitialUnloadTriggered)
		{
			// Automatically call OnActiveMountsInitialize if not already called.
			OnActiveMountsInitialize(UARI, EventContext, ProofOfWriteLock);
			// Automatically unload AssetRegistry data for unmounted MountPoints.
			UnloadAllUnmountedAsync(UARI, EventContext, TFunction<void(bool bCanceled)>());
			bAutoUnloadInitialUnloadTriggered = true;
		}
	}
	if (bLoaderIsActive && bAutoUnload)
	{
		UnloadAsync(UARI, EventContext, InMountPoints, TFunction<void(bool bCanceled)>());
	}
}

void FActiveMountsLoader::ParseInitialState(FAssetProvider& AssetProvider, const FAssetRegistryState& State)
{
	using namespace UE::PackageName;

	// If this parsing is too expensive, we can instead do the parsing during cook, and save the list of MountPoints
	// into the serialized AssetRegistry and pass them from FAssetRegistryState::Load rather than calculating them here.
	TDirectoryTree<TRefCountPtr<IMountPoint>> UsedMountPoints;
	TStringBuilder<256> PathStr;
	TSet<FString> UnmountedRoots;
	IPluginManager& PluginManager = IPluginManager::Get();

	State.EnumerateAllPaths([&PathStr, &UsedMountPoints, &UnmountedRoots, &AssetProvider, &PluginManager](FName PathName)
		{
			PathStr.Reset();
			PathStr << PathName;
			if (PathStr.Len() > 0 && !FStringView(PathStr).EndsWith('/'))
			{
				PathStr << '/';
			}
			FStringView MountPointRoot = FPackageName::SplitPackageNameRoot(PathStr, nullptr,
				FPackageName::EPathFormatFlags::MountPointSlashes);
			if (MountPointRoot.IsEmpty())
			{
				// Invalid PathName, we don't allow unloading these and don't allow adding them to mount points.
				UE_LOGFMT(LogAssetRegistry, Warning,
					"ActiveMountsLoader found an invalid path in a cooked AssetRegistry. AssetRegistry data for assets in this path will always remain loaded. "
					"AssetRegistryFile: '{AssetRegistryFile}', Path: '{Path}'.",
					*FString(AssetProvider.GetFilePath()), *PathStr);
				return;
			}
			TRefCountPtr<IMountPoint>* MountPointPtr = UsedMountPoints.FindClosestValue(PathStr);
			if (!MountPointPtr)
			{
				TRefCountPtr<IMountPoint> MountPoint = FPackageName::FindMountPointByChildLongPackageName(PathStr);

				if (!MountPoint)
				{
					// Look for a plugin matching the MountPointRoot, if one exists, add it as an unmounted mountpoint.
					// Get "PluginName" from "/PluginName/" by trimming 1 from begin and end.
					FStringView PluginName = MountPointRoot.Mid(1, MountPointRoot.Len() - 2);
					TSharedPtr<IPlugin> Plugin = PluginManager.FindPlugin(PluginName);
					if (Plugin)
					{
						MountPoint = FPackageName::FindOrAddMountPoint(MountPointRoot, Plugin->GetContentDir());
					}
				}

				if (MountPoint)
				{
					UsedMountPoints.FindOrAdd(MountPoint->GetLongPackageName()) = MoveTemp(MountPoint);
				}
				else
				{
					if (!UnmountedRoots.FindByHash(GetTypeHash(MountPointRoot), MountPointRoot))
					{
						if (!AssetProvider.HasMissingMountPoints())
						{
							UE_LOGFMT(LogAssetRegistry, Warning,
								"ActiveMountsLoader found paths in an unrecognized mountpoint in a cooked AssetRegistry. AssetRegistry data for assets in this path will always remain loaded. "
								"AssetRegistryFile: '{AssetRegistryFile}', Path: '{Path}'.",
								*FString(AssetProvider.GetFilePath()), *PathStr);
						}
						UnmountedRoots.Add(FString(MountPointRoot));
					}
				}
			}
		});

	if (!UnmountedRoots.IsEmpty())
	{
		// We do not treat this as an error for the entire AssetProvider; we still allow unloading AssetDatas in
		// recognized mountpoints in the AssetProvider.
		// TODO: We could eliminate this warning and allow unloading Assets whenever they are registered if we
		// added support for placeholder MountPoints with known packagename but unknown contentpath.
		AssetProvider.SetHasMissingMountPoints(true);
	}

	// Record the MountPoints
	for (const TPair<FStringView, TRefCountPtr<IMountPoint>>& Pair : UsedMountPoints)
	{
		IMountPoint* MountPoint = Pair.Value.GetReference();
		FMountPointData& MountData = FindOrAddMountPointData(MountPoint, EMountState::Loaded);

		if (MountData.GetAssetProvider() != nullptr || MountData.HasMultipleProviders())
		{
			if (!MountData.HasMultipleProviders())
			{
				MountData.SetHasMultipleProviders(true);
				MountData.SetIsErrorNoUnload(true);
				UE_LOGFMT(LogAssetRegistry, Error,
					"ActiveMountsLoader found assets under MountPoint present in multiple cooked AssetRegistries. "
					"Unloading is not yet supported when assets are present in multiple registries. "
					"Assets from this MountPoint will no longer be unloadable. "
					"MountPoint: '{MountPoint}', AssetRegistryFile1: '{AssetRegistryFile1}', AssetRegistryFile2: '{AssetRegistryFile2}'.",
					*FString(MountData.GetMountPoint()->GetLongPackageName()),
					*FString(MountData.GetAssetProvider()->GetFilePath()),
					*FString(AssetProvider.GetFilePath()));
			}
			continue;
		}
		MountData.SetAssetProvider(&AssetProvider);

		// Mount points have a starting slash and terminating slash: "/MountPoint/"
		// If they have more than one path segment - "/Path/Leaf/" we need to use a slower method to find them.
		SIZE_T NumSlashes = Algo::Count(MountPoint->GetLongPackageName(), '/');
		if (NumSlashes > 2)
		{
			MountData.SetIsMultiSegmentName(true);
		}
	}

	// Look for parent/child mounts
	for (const TPair<FStringView, TRefCountPtr<IMountPoint>>& Pair : UsedMountPoints)
	{
		IMountPoint* Parent = Pair.Value.GetReference();
		TRefCountPtr<IMountPoint>* Closest = UsedMountPoints.FindClosestValue(Parent->GetLongPackageName());
		if (Closest && Closest->GetReference() != Parent)
		{
			IMountPoint* Child = Closest->GetReference();
			// Don't call FindOrAdd because that could invalidate our FMountPointData pointers, but do assert that
			// they exist from our FindOrAdd call above.
			FMountPointData* ParentData = FindMountPointData(Parent);
			FMountPointData* ChildData = FindMountPointData(Child);
			check(ParentData);
			check(ChildData);
			if (!ParentData->IsParentOrChild() || !ChildData->IsParentOrChild())
			{
				ParentData->SetIsParentOrChild(true);
				ParentData->SetIsErrorNoUnload(true);
				ChildData->SetIsParentOrChild(true);
				ChildData->SetIsErrorNoUnload(true);
				UE_LOGFMT(LogAssetRegistry, Warning,
					"ActiveMountsLoader found assets in cooked AssetRegistry that use a parent and child MountPoint. "
					"We do not yet efficiently support unloading and reloading assets when parent and child mount points are present, so we are disabling unload of any assets in these mountpoints. "
					"AssetRegistryFile: '{AssetRegistryFile}', MountPoint1: '{MountPoint1}', MountPoint2: '{MountPoint2}'.",
					*FString(AssetProvider.GetFilePath()),
					*FString(Parent->GetLongPackageName()),
					*FString(Child->GetLongPackageName()));
			}
		}
	}
}

FMountPointData& FActiveMountsLoader::FindOrAddMountPointData(UE::PackageName::IMountPoint* MountPoint,
	EMountState InitialState)
{
	using namespace UE::PackageName;

	// We guarantee returning a FMountPointData& and don't have a mechanism for reporting invalid argument,
	// so caller must guarantee MountPoint != nullptr.
	check(MountPoint != nullptr);

	FMountPointData* MountPointData = MountPoints.FindByHash(GetTypeHash(MountPoint), MountPoint);
	if (MountPointData)
	{
		return *MountPointData;
	}

	FSetElementId Id = MountPoints.Add(FMountPointData(TRefCountPtr<IMountPoint>(MountPoint), InitialState));
	return MountPoints[Id];
}

FMountPointData* FActiveMountsLoader::FindMountPointData(UE::PackageName::IMountPoint* MountPoint)
{
	return MountPoints.FindByHash(GetTypeHash(MountPoint), MountPoint);
}

void FActiveMountsLoader::NotifyMountPointLoadComplete(FActiveMountEvents& EventContext,
	const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint, bool bCanceled)
{
	TArray<TRefCountPtr<FMountLoadEventSubscriber>> Subscribers;
	if (LoadSubscribers.RemoveAndCopyValue(MountPoint, Subscribers))
	{
		for (TRefCountPtr<FMountLoadEventSubscriber>& Subscriber : Subscribers)
		{
			check(Subscriber); // nullptrs are never added to LoadSubscribers
			if (Subscriber->RemainingCount <= 0 || !Subscriber->bIsLoadSubscriber)
			{
				ensureMsgf(false,
					TEXT("ActiveMounts found an invalid MountLoadEventSubscriber registered in LoadSubscribers. ")
					TEXT("We will not send the notification and this may cause a softlock. ")
					TEXT("MountPoint = %s, RemainingCount = %d, bIsLoadSubscriber = %s, bWasCanceled = %s"),
					*FString(MountPoint->GetLongPackageName()), Subscriber->RemainingCount,
					*::LexToString(Subscriber->bIsLoadSubscriber), *::LexToString(Subscriber->bWasCanceled));
				continue;
			}
			Subscriber->bWasCanceled |= bCanceled;
			Subscriber->RemainingCount--;
			if (Subscriber->RemainingCount == 0)
			{
				EventContext.AddCompletion(MoveTemp(Subscriber->OnComplete), Subscriber->bWasCanceled);
			}
		}
		ConditionalShrinkUsuallyEmptyContainer(LoadSubscribers);
	}
}

void FActiveMountsLoader::NotifyMountPointUnloadComplete(FActiveMountEvents& EventContext,
	const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint, bool bCanceled)
{
	TArray<TRefCountPtr<FMountLoadEventSubscriber>> Subscribers;
	if (UnloadSubscribers.RemoveAndCopyValue(MountPoint, Subscribers))
	{
		for (TRefCountPtr<FMountLoadEventSubscriber>& Subscriber : Subscribers)
		{
			check(Subscriber); // nullptrs are never added to UnloadSubscribers
			if (Subscriber->RemainingCount <= 0 || Subscriber->bIsLoadSubscriber)
			{
				ensureMsgf(false,
					TEXT("ActiveMounts found an invalid MountLoadEventSubscriber registered in UnloadSubscribers. ")
					TEXT("We will not send the notification and this may cause a softlock. ")
					TEXT("MountPoint = %s, RemainingCount = %d, bIsLoadSubscriber = %s, bWasCanceled = %s"),
					*FString(MountPoint->GetLongPackageName()), Subscriber->RemainingCount,
					*::LexToString(Subscriber->bIsLoadSubscriber), *::LexToString(Subscriber->bWasCanceled));
				continue;
			}
			Subscriber->bWasCanceled |= bCanceled;
			Subscriber->RemainingCount--;
			if (Subscriber->RemainingCount == 0)
			{
				EventContext.AddCompletion(MoveTemp(Subscriber->OnComplete), Subscriber->bWasCanceled);
			}
		}
		ConditionalShrinkUsuallyEmptyContainer(UnloadSubscribers);
	}
}

void FActiveMountsLoader::ConditionalNotifyIdle(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext)
{
	if (OnIdleSubscribers.IsEmpty())
	{
		return;
	}
	if (IsInProgress(UARI))
	{
		return;
	}
	for (TFunction<void()>& OnIdle : OnIdleSubscribers)
	{
		EventContext.AddIdle(MoveTemp(OnIdle));
	}
	OnIdleSubscribers.Empty();
}

void FActiveMountsLoader::AddRefMountPointLoad(UAssetRegistryImpl& UARI, FMountPointData* MountPointData)
{
	// Caller must not call AddRef if MountPointData is null or does not have an AssetProvider.
	check(MountPointData && MountPointData->GetAssetProvider());
	FAssetProvider* AssetProvider = MountPointData->GetAssetProvider();
	AssetProvider->GetLoadRefCountPtr()++;
	UE::Tasks::FTask& LoadTask = AssetProvider->GetLoadTask();
	if (LoadTask.IsValid())
	{
		// Nothing to do, the task is still running its loop and will pick up the new MountPoint.
		return;
	}

	LoadTask = UE::Tasks::Launch(TEXT("ActiveMountsLoad"), [&UARI, this, AssetProvider]()
		{
			RunAssetProviderLoadTask(UARI, *AssetProvider);
		});
}

void FActiveMountsLoader::ReleaseMountPointLoad(UAssetRegistryImpl& UARI, FMountPointData* MountPointData)
{
	// Caller must not call Release if MountPointData is null or does not have an AssetProvider.
	check(MountPointData && MountPointData->GetAssetProvider());
	MountPointData->GetAssetProvider()->GetLoadRefCountPtr()--;
	// Release is not responsible for shutting down the LoadTask; RunAssetProviderLoadTask polls the
	// refcount and shuts itself down when it sees 0.
}

void FActiveMountsLoader::AddRefMountPointUnload(UAssetRegistryImpl& UARI, FMountPointData* MountPointData)
{
	// Caller must not call AddRef if MountPointData is null or does not have an AssetProvider.
	check(MountPointData && MountPointData->GetAssetProvider());
	MountPointUnloadRefCount++;
	if (UnloadTask.IsValid())
	{
		// Nothing to do, the task is still running its loop and will pick up the new MountPoint.
		return;
	}
	UnloadTask = UE::Tasks::Launch(TEXT("ActiveMountsUnload"), [&UARI, this]()
		{
			RunMountPointUnloadTask(UARI);
		});
}

void FActiveMountsLoader::ReleaseMountPointUnload(UAssetRegistryImpl& UARI, FMountPointData* MountPointData)
{
	// Caller must not call Release if MountPointData is null.
	check(MountPointData);
	MountPointUnloadRefCount--;
	// Release is not responsible for shutting down the UnloadTask; RunMountPointUnloadTaskRunMountPointUnloadTask polls the
	// refcount and shuts itself down when it sees 0.
}

void FActiveMountsLoader::RunAssetProviderLoadTask(UAssetRegistryImpl& UARI, FAssetProvider& AssetProvider)
{
	LLM_SCOPE(ELLMTag::AssetRegistry);
	FActiveMountEvents OnIdleEvents;
	ON_SCOPE_EXIT
	{
		OnIdleEvents.Broadcast();
	};

	auto MarkAsyncThreadComplete = [this, &AssetProvider, &UARI, &OnIdleEvents]()
		{
			// Caller must call only while within the ActiveMounts critical section, to avoid a softlock race
			// where the main thread sees that the AsyncTask still exists and does not schedule another one,
			// but the AsyncTask is shutting down and does not plan to check for further work to do.
			AssetProvider.GetLoadTask() = UE::Tasks::FTask();
			ConditionalNotifyIdle(UARI, OnIdleEvents);
		};

	// Requests might come in unbatched, or in batches that race with our load and come in after we
	// decide which MountPoints we will load from the disk. Keep repeatedly loading the AssetRegistry.bin file
	// and creating AssetDatas for the current batch of MountPoints out of it until we have no further load requests.
	for (int32 NumIterations = 0; /* Loop is terminated by break statements inside a lock */; ++NumIterations)
	{
		FString StateFilePath;
		TArray<TRefCountPtr<UE::PackageName::IMountPoint>> MountPointsToLoad;
		bool bHasMultiSegmentPath = false;
		{
			FActiveMountsLoader* pNewThis = UARI.ActiveMountsLoader.LockIfEnabled();
			// ActiveMounts were previously enabled otherwise this function would not be called, and Shutdown is not
			// supposed to disable ActiveMounts until after the thread running this function shuts down. So we
			// should still be enabled and pNewThis should be non-null and we don't have to handle exiting.
			check(pNewThis);
			ON_SCOPE_EXIT
			{
				UARI.ActiveMountsLoader.Unlock();
			};

			// We do however have to handle the shutdown signal, either global or for the AssetProvider, and leave the thread.
			if (bShutdown || AssetProvider.IsShutdown())
			{
				MarkAsyncThreadComplete();
				return;
			}

			// If no further MountPoints to load, leave the thread.
			if (AssetProvider.GetLoadRefCountPtr() == 0)
			{
				MarkAsyncThreadComplete();
				return;
			}

			// If we have already run once, reschedule our task rather than continuing to run, to give any other
			// high priority tasks that have come in a chance to take our WorkerThread.
			if (NumIterations > 0)
			{
				AssetProvider.GetLoadTask() = UE::Tasks::Launch(TEXT("ActiveMountsLoad"), [&UARI, this, AssetProvider=&AssetProvider]()
					{
						RunAssetProviderLoadTask(UARI, *AssetProvider);
					});
				// Do not call MarkAsyncThreadComplete; we are still running the task.
				return;
			}

			StateFilePath = AssetProvider.GetFilePath();
			// Leave the lock to load the file off of disk.
		}

		// Open the file and preload it, but do not deserialize it yet; we need to know the list of mountpoints first,
		// and we want to capture those as late as possible in case of stragglers while we are preloading the file.
		constexpr uint32 BufferSize = 10 * 1024 * 1024;
		TUniquePtr<FArchive> FileArchive = OpenReadStateFileArchive(StateFilePath, BufferSize);
		FixedTagPrivate::FWeakStorePtr Store;

		// Reenter the lock to get the list of mountpoints to load from the statefile.
		{
			FActiveMountsLoader* pNewThis = UARI.ActiveMountsLoader.LockIfEnabled();
			// Per the comment above, we should still be enabled and pNewThis should be non-null and we don't have to handle exiting.
			check(pNewThis);
			ON_SCOPE_EXIT
			{
				UARI.ActiveMountsLoader.Unlock();
			};

			// We do however have to handle the shutdown signal, either global or for the AssetProvider, and leave the thread.
			if (bShutdown || AssetProvider.IsShutdown())
			{
				MarkAsyncThreadComplete();
				return;
			}

			// Check whether LoadRefCountPtr is still non-zero.
			if (AssetProvider.GetLoadRefCountPtr() == 0)
			{
				MarkAsyncThreadComplete();
				return;
			}

			MountPointsToLoad.Reserve(AssetProvider.GetLoadRefCountPtr());
			for (FMountPointData& MountData : MountPoints)
			{
				if (MountData.GetCurrentState() == EMountState::Loading
					&& MountData.GetAssetProvider() == &AssetProvider)
				{
					MountPointsToLoad.Add(MountData.GetMountPoint());
					bHasMultiSegmentPath |= MountData.IsMultiSegmentName();
				}
			}
			if (MountPointsToLoad.Num() == 0)
			{
				// Unexpected, because MountPoints are supposed to drop their refcount before leaving the Loading
				// state.
				UE_LOGFMT(LogAssetRegistry, Error,
					"ActiveMounts: Some MountPoint left the loading state without dropping its refcount on the AssetProvider. "
					"AssetRegistryFile: {AssetRegistryFile}.",
					*StateFilePath);
				MarkAsyncThreadComplete();
				return;
			}
			Store = AssetProvider.GetStore();
			// Leave the lock to deserialize the AssetRegistry state
		}

		TFunction<bool(FName PackageName)> KeepFunction;
		TDirectoryTree<bool> KeepTree;
		TSet<FString> KeepSet;
		if (!bHasMultiSegmentPath)
		{
			for (TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint : MountPointsToLoad)
			{
				KeepSet.Add(FString(MountPoint->GetLongPackageName()));
			}
			KeepFunction = [&KeepSet](FName PackageName) -> bool
				{
					TStringBuilder<256> PathStr(InPlace, PackageName);
					// We have put MountPoint with slashes - /MountPoint/ - in our set, so get slashes for lookup.
					FStringView MountPointRoot = FPackageName::SplitPackageNameRoot(PathStr, nullptr,
						FPackageName::EPathFormatFlags::MountPointSlashes);
					if (MountPointRoot.IsEmpty())
					{
						// We never unload these (we log an error during initial parsing that we keep them loaded)
						// so we do not need to reload them now.
						return false;
					}
					return KeepSet.ContainsByHash(GetTypeHash(MountPointRoot), MountPointRoot);
				};
		}
		else
		{
			// In the case of mountpoint with more than one segment, we have to do a prefix lookup in our list of
			// mountpoints rather than a faster TSet lookup, because we don't know how much of the packagename to use
			// for the TSet lookup.
			for (TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint : MountPointsToLoad)
			{
				KeepTree.FindOrAdd(MountPoint->GetLongPackageName()) = true;
			}
			KeepFunction = [&KeepTree](FName PackageName) -> bool
				{
					TStringBuilder<256> PathStr(InPlace, PackageName);
					FStringView ParentDir = FPathViews::GetPath(PathStr);
					if (ParentDir.IsEmpty())
					{
						// We never unload these (we log an error during initial parsing that we keep them loaded)
						// so we do not need to reload them now.
						return false;
					}
					return KeepTree.ContainsPathOrParent(ParentDir);
				};
		}

		bool bLoadSuccessful = false;
		FAssetRegistryState StateToAppend;
		FAssetRegistryLoadOptions LoadOptions = UE::AssetRegistry::Premade::GetLoadOptions();
		check(LoadOptions.bMountPointsRequireSeparateAllocationBlocks == true);
		LoadOptions.ShouldKeepPackage = MoveTemp(KeepFunction);
		LoadOptions.Store = MoveTemp(Store);
		FAssetRegistryLoadResults LoadResults;

		if (FileArchive)
		{
			bLoadSuccessful = StateToAppend.Load(*FileArchive, LoadOptions, LoadResults);
		}

		if (bLoadSuccessful)
		{
			FEventContext AREventContext;
			{
				UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(UARI.InterfaceLock);
				// Use ConsumeState rather than AppendState so that we don't create a new FStore for the
				// AssetDatas moved over from the StateToAppend, and we don't wastefully reallocate the AssetDatas.
				// ConsumeState also uses EAppendMode::OnlyUpdateNew, which is fine for our purpose which is
				// just reloading data that we previously unloaded.
				UARI.GuardedData.ConsumeState(AREventContext, MoveTemp(StateToAppend), /*bEmitAssetEvents*/ true);
			}
			UARI.Broadcast(AREventContext);
		}

		// Update state data and send events for all of the MountPointsToLoad
		FActiveMountEvents EventContext;
		{
			FActiveMountsLoader* pNewThis = UARI.ActiveMountsLoader.LockIfEnabled();
			// Per the comment above, we should still be enabled and pNewThis should be non-null and we don't have to handle exiting.
			check(pNewThis);
			ON_SCOPE_EXIT
			{
				UARI.ActiveMountsLoader.Unlock();
			};

			// We do however have to handle the shutdown signal, either global or for the AssetProvider, and leave the thread.
			if (bShutdown || AssetProvider.IsShutdown())
			{
				MarkAsyncThreadComplete();
				return;
			}

			if (bLoadSuccessful)
			{
				// Refresh the Store pointer on the AssetProvider, in case the old store had been garbage collected and the load
				// allocated a new store.
				AssetProvider.SetStore(LoadResults.Store);
			}
			else
			{
				UE_LOGFMT(LogAssetRegistry, Error,
					"ActiveMounts: Failed to reload AssetRegistry data from a cooked AssetRegistry after some of its mountpoints were unloaded and then reloaded. "
					"The assets will be missing from the AssetRegistry, for all of the MountPoints in the cooked AssetRegistry that were previously unloaded. "
					"AssetRegistryFile: '{AssetRegistryFile}', example missing MountPoint: '{MountPoint}'.",
					*StateFilePath, *FString(MountPointsToLoad[0]->GetLongPackageName()));
			}

			UE_LOGFMT(LogAssetRegistry, Log, "ActiveMountsLoading reloaded AssetRegistry data for {NumMountPoints} MountPoints.",
				MountPointsToLoad.Num());

			for (TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint : MountPointsToLoad)
			{
				// The MountPoints should still be in the Loading state, because nothing is allowed to take them out
				// of the loading state other than this function running on their AssetProvider, and we do not allow
				// a second instance of this function on the same AssetProvider at a time.
				FMountPointData* MountData = MountPoints.FindByHash(GetTypeHash(MountPoint), MountPoint);
				if (!MountData || MountData->GetAssetProvider() == nullptr)
				{
					// AssetProvider containing the MountData was unregistered.
					continue;
				}

				if (MountData->GetCurrentState() != EMountState::Loading)
				{
					UE_LOGFMT(LogAssetRegistry, Error,
						"ActiveMounts: Failed to reload AssetRegistry data for mountpoint because the mountpoint unexpectedly changed loading state while we were loading it. "
						"The assets in the MountPoint might now be be missing from the AssetRegistry. "
						"AssetRegistryFile: '{StateFilePath}', MountPoint: '{MountPoint}', CurrentState: '{CurrentState}.'",
						*StateFilePath, *FString(MountPoint->GetLongPackageName()), int(MountData->GetCurrentState()));
					continue;
				}

				// Even if the load failed, we still have to mark all the MountDatas as loaded, to avoid having this thread
				// repeatedly continue trying to load them and kill performance.
				MountData->SetCurrentState(EMountState::Loaded);
				ReleaseMountPointLoad(UARI, MountData);
				NotifyMountPointLoadComplete(EventContext, MountPoint, false /* bCanceled */);

				// It is possible that a request came in to unload the MountPoint while we were still working on the
				// load. The unload requested was recorded in the targetstate if so, and no other action taken. Check
				// for that now and kick off the unload if so.
				if (MountData->GetTargetState() != EMountState::Loaded)
				{
					// TargetState can only be Loaded or Unloaded, transition states are not possible for it.
					check(MountData->GetTargetState() == EMountState::Unloaded);
					MountData->SetCurrentState(EMountState::Unloading);
					check(MountData->GetAssetProvider()); // We checked above for GetAssetProvider()
					AddRefMountPointUnload(UARI, MountData);
				}
			}
		}
		EventContext.Broadcast();
	}
}

void FActiveMountsLoader::RunMountPointUnloadTask(UAssetRegistryImpl& UARI)
{
	LLM_SCOPE(ELLMTag::AssetRegistry);
	FActiveMountEvents OnIdleEvents;
	ON_SCOPE_EXIT
	{
		OnIdleEvents.Broadcast();
	};

	auto MarkAsyncThreadComplete = [this, &UARI, &OnIdleEvents]()
		{
			// Caller must call only while within the ActiveMounts critical section, to avoid a softlock race
			// where the main thread sees that the AsyncTask still exists and does not schedule another one,
			// but the AsyncTask is shutting down and does not plan to check for further work to do.
			UnloadTask = UE::Tasks::FTask();
			ConditionalNotifyIdle(UARI, OnIdleEvents);
		};

	// Requests might come in unbatched, or in batches that race with our unload and come in after we
	// decide which MountPoints we will unload. Keep repeatedly unloading the current batch until we have
	// no further unload requests.
	for (int32 NumIterations = 0; /* Loop is terminated by break statements inside a lock */ ; ++NumIterations)
	{
		TArray<TRefCountPtr<UE::PackageName::IMountPoint>> MountPointsToUnload;
		{
			FActiveMountsLoader* pNewThis = UARI.ActiveMountsLoader.LockIfEnabled();
			// ActiveMounts were previously enabled otherwise this function would not be called, and Shutdown is not
			// supposed to disable ActiveMounts until after the thread running this function shuts down. So we
			// should still be enabled and pNewThis should be non-null and we don't have to handle exiting.
			check(pNewThis);
			ON_SCOPE_EXIT
			{
				UARI.ActiveMountsLoader.Unlock();
			};

			// We do however have to handle the shutdown signal, and leave the thread.
			if (bShutdown)
			{
				MarkAsyncThreadComplete();
				return;
			}

			// If no further MountPoints to load, leave the thread.
			if (MountPointUnloadRefCount == 0)
			{
				MarkAsyncThreadComplete();
				return;
			}

			// If we have already run once, reschedule our task rather than continuing to run, to give any other
			// high priority tasks that have come in a chance to take our WorkerThread.
			if (NumIterations > 0)
			{
				UnloadTask = UE::Tasks::Launch(TEXT("ActiveMountsUnload"), [&UARI, this]()
					{
						RunMountPointUnloadTask(UARI);
					});
				// Do not call MarkAsyncThreadComplete; we are still running the task.
				return;
			}

			MountPointsToUnload.Reserve(MountPointUnloadRefCount);
			for (FMountPointData& MountData : MountPoints)
			{
				if (MountData.GetCurrentState() == EMountState::Unloading
					&& MountData.GetAssetProvider() != nullptr)
				{
					MountPointsToUnload.Add(MountData.GetMountPoint());
				}
			}
			if (MountPointsToUnload.Num() == 0)
			{
				// Unexpected, because MountPoints are supposed to drop their refcount before leaving the Unloading
				// state.
				UE_LOGFMT(LogAssetRegistry, Error,
					"ActiveMounts: Some MountPoint left the unloading state without dropping its MountPointUnloadRefCount.");
				MarkAsyncThreadComplete();
				return;
			}
			// Leave the ActiveMounts lock to modify the AssetRegistry state.
		}

		FEventContext AREventContext;
		{
			UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(UARI.InterfaceLock);
			UARI.GuardedData.UnloadMountPoints(AREventContext, MountPointsToUnload);
		}
		UARI.Broadcast(AREventContext);

		// Update state data and send events for all of the unloaded MountPoints.
		FActiveMountEvents EventContext;
		{
			FActiveMountsLoader* pNewThis = UARI.ActiveMountsLoader.LockIfEnabled();
			// Per the comment above, we should still be enabled and pNewThis should be non-null and we don't have to handle exiting.
			check(pNewThis);
			ON_SCOPE_EXIT
			{
				UARI.ActiveMountsLoader.Unlock();
			};

			// We do however have to handle the shutdown signal, either global or for the AssetProvider, and leave the thread.
			if (bShutdown)
			{
				MarkAsyncThreadComplete();
				return;
			}

			UE_LOGFMT(LogAssetRegistry, Log, "ActiveMountsLoading unloaded AssetRegistry data for {NumMountPoints} MountPoints.",
				MountPointsToUnload.Num());

			for (TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint : MountPointsToUnload)
			{
				// The MountPoints should still be in the Unloading state, because nothing is allowed to take them out
				// of the Unloading state other than this function, and we do not allow a second instance of this
				// function at a time.
				FMountPointData* MountData = MountPoints.FindByHash(GetTypeHash(MountPoint), MountPoint);
				if (!MountData || MountData->GetAssetProvider() == nullptr)
				{
					// AssetProvider owning the MountData was unregistered
					continue;
				}
				if (MountData->GetCurrentState() != EMountState::Unloading)
				{
					UE_LOGFMT(LogAssetRegistry, Error,
						"ActiveMounts: Failed to unload AssetRegistry data for mountpoint because the mountpoint unexpectedly changed unloading state while we were unloading it. "
						"The assets in the MountPoint might now be be missing from the AssetRegistry. "
						"MountPoint: '{MountPoint}', CurrentState: '{CurrentState}'.",
						*FString(MountPoint->GetLongPackageName()), int(MountData->GetCurrentState()));
					continue;
				}

				MountData->SetCurrentState(EMountState::Unloaded);
				ReleaseMountPointUnload(UARI, MountData);
				NotifyMountPointUnloadComplete(EventContext, MountPoint, false /* bCanceled */);

				// It is possible that a request came in to load the MountPoint while we were still working on the
				// unload. The load requested was recorded in the targetstate if so, and no other action taken. Check
				// for that now and kick off the load if so.
				if (MountData->GetTargetState() != EMountState::Unloaded)
				{
					// TargetState can only be Loaded or Unloaded, transition states are not possible for it.
					check(MountData->GetTargetState() == EMountState::Loaded);
					MountData->SetCurrentState(EMountState::Loading);
					check(MountData->GetAssetProvider()); // We checked above for GetAssetProvider
					AddRefMountPointLoad(UARI, MountData);
				}
			}
		}
		EventContext.Broadcast();
	}
}

} // namespace UE::AssetRegistry::Impl
