// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARLoader.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "HAL/PlatformMath.h"
#include "Misc/Optional.h"
#include "Tasks/Task.h"

namespace UE::AssetRegistry::Impl
{

/** Loading states of MountPoints managed by the ActiveMountsLoader state machine. */
enum class EMountState : uint8
{
	Unloaded,
	Loading,
	Loaded,
	Unloading,
	Count,
	NumBits = FPlatformMath::ConstExprCeilLogTwo(EMountState::Count),
};

/**
 * A source of AssetDatas for the ActiveMountsLoader to apply to the AssetRegistry. Currently the only such source is
 * a cooked AssetRegistry.bin file on disk. The filepath for a registered AssetRegistry.bin file is stored in an
 * AssetProvider, and a record of the MountPoints that contain the AssetData paths within the AssetProvider are stored
 * in the ActiveMountsLoader and associated with the AssetProvider.  The ActiveMountsLoader launches an async task to
 * serialize the file from disk whenever it needs to reload AssetDatas under any of the MountPoints that contain the
 * AssetData paths within the AssetProvider.
 */
class FAssetProvider
{
public:
	explicit FAssetProvider(FStringView InFilePath);

	/** Return this provider's AssetRegistry.bin filepath that can be deserialized from disk to load the AssetDatas. */
	FStringView GetFilePath() const;
	/**
	 * Report whether this provider is global AssetRegistry's premade AssetRegistry. That AssetRegistry is notable
	 * because it is the first and the largest, and we have special handling to manage it performantly. It can not be
	 * unregistered.
	 */
	bool IsInitialProvider() const;
	void SetIsInitialProvider(bool bValue);

	/**
	 * Return a weak pointer to the opaque FixedTagPrivate::FStore that was created or referenced by
	 * FAssetRegistryState::Load when this provider was last deserialized from disk. FStore holds deduplicated
	 * FAssetData.TagsAndValues data for all AssetDatas in the AssetRegistryState. We record this FStore and pass it
	 * into the FAssetRegistryState::Load when we reload, which then reuses it rather than creating a new one. Other
	 * than passing it into FAssetRegistryState::Load, we do not interpret it. The pointer is weak and will be
	 * set to null if all AssetDatas using it are dismounted, in which case it will be recreated during Load.
	 */
	const FixedTagPrivate::FWeakStorePtr& GetStore() const;
	void SetStore(FixedTagPrivate::FWeakStorePtr InStore);
	/**
	 * A count of how many FMountPoints have been requested to reload and not yet reloaded. Used to manage the
	 * async task that loads them. This value contractually equals the number of FMountPointDatas associated with this
	 * AssetProvider that are in the loading state.
	 */
	uint32& GetLoadRefCountPtr();
	/**
	 * Return a mutable reference to the LoadTask; we create it on demand if not yet created and it will keep running
	 * until it finds no more work to do and exits. When the task exits it clears this reference to itself.
	 */
	UE::Tasks::FTask& GetLoadTask();

	/**
	 * Whether this AssetProvider is shutting down, which happens when it is unregistered by the game project
	 * to save on memory, or when the ActiveMountsLoader is shutting down. This field is used to block new activity while
	 * we wait for async tasks to shutdown.
	 */
	bool IsShutdown() const;
	void SetIsShutdown(bool bValue);
	/**
	 * Information discovered during parsing of this provider's AssetRegistryState: true if there are
	 * AssetDatas in this provider for which we could not find the FPackageName::IMountPoint and therefore cannot
	 * unload/reload.
	 */
	bool HasMissingMountPoints() const;
	void SetHasMissingMountPoints(bool bValue);
	/**
	 * Whether this AssetProvider encountered an error on load of its AssetRegistryState and therefore we
	 * should not unload AssetDatas in any of its MountPoints.
	 */
	bool IsErrorNoUnload() const;
	void SetIsErrorNoUnload(bool bValue);

private:
	const FString FilePath;
	UE::Tasks::FTask LoadTask;
	FixedTagPrivate::FWeakStorePtr Store;
	uint32 LoadRefCount = 0;
	bool bInitialProvider : 1 = false;
	bool bShutdown : 1 = false;
	bool bHasMissingMountPoints : 1 = false;
	bool bErrorNoUnload : 1 = false;
};

/**
 * A wrapper structure for FAssetProviders in a TSet that allow them to be looked up by FilePath. This structure
 * also maintains the TUniquePtr for the FAssetProvider, which need to be at a fixed pointer over their lifetime
 * and therefore must be TUniquePtr rather than direct elements in the TSet.
 */
class FAssetProviderSetKey
{
public:
	explicit FAssetProviderSetKey(TUniquePtr<FAssetProvider>&& InAssetProvider);

	bool operator==(const FAssetProviderSetKey& Other) const;
	bool operator==(FStringView OtherProviderFilePath) const;

	FAssetProvider& GetAssetProvider() const;

private:
	TUniquePtr<FAssetProvider> AssetProvider;
};
extern uint32 GetTypeHash(const FAssetProviderSetKey& Key);

/**
 * Data held by the ActiveMountsLoader about a MountPoint that contains AssetDatas in one of its registered
 * AssetProviders. This data includes the associated AssetProvider for the MountPoint and the state machine data for
 * the unload/reload of the assets in the MountPoint.
 */
class FMountPointData
{
public:
	explicit FMountPointData(TRefCountPtr<UE::PackageName::IMountPoint> InMountPoint, EMountState InInitialState);

	bool operator==(const FMountPointData& Other) const;
	bool operator==(UE::PackageName::IMountPoint* MountPoint) const;

	const TRefCountPtr<UE::PackageName::IMountPoint>& GetMountPoint() const;
	FAssetProvider* GetAssetProvider() const;
	void SetAssetProvider(FAssetProvider* InAssetProvider);
	/**
	 * The current state of the MountPoints unload/reload. This only changes when a request changes it into a transition
	 * state or when a completed unload/reload changes it back to a stable state.
	 */
	EMountState GetCurrentState() const;
	void SetCurrentState(EMountState InState);
	/**
	 * The currently requested unloaded or loaded state of the MountPoint. We track this requested state so that we
	 * can handle new requests that override old requests that are still in progress.
	 */
	EMountState GetTargetState() const;
	void SetTargetState(EMountState InState);

	/**
	 * Whether this MountPoint encountered an error on load of (any of) its associated AssetProviders and therefore we
	 * should not unload AssetDatas under it. Notably, being associated with multiple AssetProviders is an error
	 * itself; @see HasMultipleProviders.
	 */
	bool IsErrorNoUnload() const;
	void SetIsErrorNoUnload(bool bValue);
	/**
	 * Whether this MountPoint has a multisegment IMountPoint->GetLongPackageName(), such as /Game/Weapons/, and so
	 * requires more expensive calculations to determine whether a given AssetData LongPackageName is under it.
	 */
	bool IsMultiSegmentName() const;
	void SetIsMultiSegmentName(bool bValue);
	/**
	 * Whether this MountPoint is a either a child or parent of another existing IMountPoint (e.g. /Game/Weapons/ is
	 * a child of /Game/, and /Game/ is a parent of /Game/Weapons/) and so we don't currently support unload AssetDatas
	 * under it.
	 */
	bool IsParentOrChild() const;
	void SetIsParentOrChild(bool bValue);
	/**
	 * Whether AssetDatas under this MountPoint exist in multiple AssetProviders, and therefore we should not unload
	 * AssetDatas under it. We do not currently support MountPoints in multiple AssetProviders because we want to save
	 * memory by avoiding tracking multiple providers.
	 */
	bool HasMultipleProviders() const;
	void SetHasMultipleProviders(bool bValue);

private:
	TRefCountPtr<UE::PackageName::IMountPoint> MountPoint;
	FAssetProvider* AssetProvider = nullptr;
	uint32 ErrorNoUnloadBit : 1 = false;
	uint32 MultiSegmentNameBit : 1 = false;
	uint32 IsParentOrChildBit : 1 = false;
	uint32 HasMultipleProvidersBit : 1 = false;
	uint32 CurrentStateBits : static_cast<uint8>(EMountState::NumBits) = static_cast<uint32>(EMountState::Unloaded);
	uint32 TargetStateBits : static_cast<uint8>(EMountState::NumBits) = static_cast<uint32>(EMountState::Unloaded);
};
extern uint32 GetTypeHash(const FMountPointData& Key);

/**
 * Storage of an OnComplete function that was passed into LoadAsync or UnloadAsync. We hold this storage until
 * all requests in the batch have completed and we can therefore call the OnComplete function. This struct is updated
 * every time we complete a request for one of its MountPoints.
 */
struct FMountLoadEventSubscriber : public FRefCountedObject
{
	TFunction<void(bool bCanceled)> OnComplete;
	// We don't store the list of remaining MountPoints to save memory/time on storing it, and we
	// can do that but still be able to debug if it ever gets stuck at non-zero value because that
	// information already is present (stored in the other direction) in FActiveMountsLoader.Subscribers.
	// If RemainingCount > 0, there should be a pointer to this somewhere in LoadSubscribers or UnloadSubscribers.
	int32 RemainingCount = 0;
	bool bWasCanceled = false;
	/**
	 * True if listening to loads (and referenced from LoadSubscribers), false if listening to unloads
	 * (and referenced from UnloadSubscribers).
	 */
	bool bIsLoadSubscriber = true;
};

/**
 * The implementation of ActiveMountsLoading. This implementation class is created only when the feature is enabled
 * in the current process. It holds data about how to reload each mountpoint and manages their state in an
 * unload/reload state machine.
 */
class FActiveMountsLoader
{
public:
	// Documentation for public functions on this class is on FActiveMountsLoaderWrapper functions of the same name
	// (with the possible addition of Start and Finish prefixes for functions that require WaitOnAsync outside the
	// lock). @see FActiveMountsLoaderWrapper.

	/**
	 * Called by FActiveMountsLoaderWrapper::Shutdown. Signals threads to shutdown and reports all EventRefs that
	 * must be waited on for those threads to terminate. Followed by a call to FinishShutdown after waiting.
	 */
	void StartShutdown(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		TArray<UE::Tasks::FTask>& OutWaitOnAsync);
	/**
	 * Called by FActiveMountsLoaderWrapper::Shutdown after waiting on EventRefs from StartShutdown. Clears all
	 * internal data, FActiveMountsLoader can now be deleted.
	 */
	void FinishShutdown(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext);

	void LoadAsync(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> InMountPoints,
		TFunction<void(bool bCanceled)>&& OnComplete);
	void ReloadAllAsync(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		TFunction<void(bool bCanceled)>&& OnComplete);
	void UnloadAsync(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> InMountPoints,
		TFunction<void(bool bCanceled)>&& OnComplete);
	void UnloadAllUnmountedAsync(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		TFunction<void(bool bCanceled)>&& OnComplete);
	bool IsLoaded(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint);
	bool IsInProgress(UAssetRegistryImpl& UARI);
	void ReportWhenIdle(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext, TFunction<void()> OnIdle);

	void OnActiveMountsInitialize(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		FInterfaceWriteScopeLock& ProofOfWriteLock);
	bool TryRegisterAssetProviderAndAppendStateOutsideARLock(UAssetRegistryImpl& UARI, FEventContext& AREventContext,
		FActiveMountEvents& EventContext, FStringView StateFilePath);
	bool TryLoadStateAndRegisterAssetProvider(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		FInterfaceWriteScopeLock& ProofOfWriteLock, FPakPlatformFile* PakPlatformFile, FStringView StateFilePath,
		FAssetRegistryState& OutLoadedState);
	/**
	 * Called by FActiveMountsLoaderWrapper::UnregisterAssetProvider. Signals the AssetProvider's load thread to
	 * shutdown if it exists and reports the EventRefs that must be waited on for it to terminate.
	 * Followed by a call to FinishUnregisterAssetProvider after waiting. Returns false if the AssetProvider does
	 * not exist; FinishUnregisterAssetProvider should not be called in that case.
	 */
	bool TryStartUnregisterAssetProvider(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		TArray<UE::Tasks::FTask>& OutWaitOnAsync, FStringView StateFilePath);
	/**
	 * Called by FActiveMountsLoaderWrapper::UnregisterAssetProvider after waiting on EventRefs from StartShutdown.
	 * Clears the AssetProvider's data and the data of any MountPoints it owned. 
	 */
	void FinishUnregisterAssetProvider(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		FStringView StateFilePath);

	void InitialProviderReport(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		FAssetProviderInitData& StateInitData);
	void InitialProviderOnReadyForRegister(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		EInitialProviderRegisterReason Reason);
	void InitialProviderConditionalRegister(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		FAssetRegistryState& AppendedState);

	void OnMountPointsMounted(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> InMountPoints);
	void OnMountPointsDismounted(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext,
		FInterfaceWriteScopeLock& ProofOfWriteLock,
		TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> InMountPoints);

private:
	/**
	 * Inspect all of the AssetDatas in the AssetRegistryState loaded by an AssetProvider to find which MountPoints
	 * should be associated with it.
	 */
	void ParseInitialState(FAssetProvider& AssetProvider, const FAssetRegistryState& State);
	FMountPointData& FindOrAddMountPointData(UE::PackageName::IMountPoint* MountPoint, EMountState InitialState);
	FMountPointData* FindMountPointData(UE::PackageName::IMountPoint* MountPoint);
	void NotifyMountPointLoadComplete(FActiveMountEvents& EventContext,
		const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint, bool bCanceled);
	void NotifyMountPointUnloadComplete(FActiveMountEvents& EventContext,
		const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint, bool bCanceled);
	void ConditionalNotifyIdle(UAssetRegistryImpl& UARI, FActiveMountEvents& EventContext);
	void AddRefMountPointLoad(UAssetRegistryImpl& UARI, FMountPointData* MountPointData);
	void ReleaseMountPointLoad(UAssetRegistryImpl& UARI, FMountPointData* MountPointData);
	void AddRefMountPointUnload(UAssetRegistryImpl& UARI, FMountPointData* MountPointData);
	void ReleaseMountPointUnload(UAssetRegistryImpl& UARI, FMountPointData* MountPointData);
	/** The DoTask function for an AssetProvider's LoadTask for any requested MountPoint loads in it. */
	void RunAssetProviderLoadTask(UAssetRegistryImpl& UARI, FAssetProvider& AssetProvider);
	/** The DoTask function for the UnloadTask for any requested MountPoint unloads. */
	void RunMountPointUnloadTask(UAssetRegistryImpl& UARI);

	TSet<FAssetProviderSetKey> AssetProviders;
	TSet<FMountPointData> MountPoints;
	/** The list of OnComplete functions that are waiting on a MountPoint to load. */
	TMap<TRefCountPtr<UE::PackageName::IMountPoint>, TArray<TRefCountPtr<FMountLoadEventSubscriber>>> LoadSubscribers;
	/** The list of OnComplete functions that are waiting on a MountPoint to unload. */
	TMap<TRefCountPtr<UE::PackageName::IMountPoint>, TArray<TRefCountPtr<FMountLoadEventSubscriber>>> UnloadSubscribers;
	/** The list of callbacks that are waiting on ReportWhenIdle. */
	TArray<TFunction<void()>> OnIdleSubscribers;
	/**
	 * An additional lock entered into by registration and unregistration functions. Unregistration functions have to
	 * wait outside of the usual ActiveMountsLoader lock on FActiveMountsLoaderWrapper::ActiveMountsLock because the
	 * tasks they wait on need to enter that lock. But we need to force only a single register/unregister to happen at
	 * once, so we have this lock. This lock must always be entered before entering the ActiveMountsLock. That creates
	 * a complex detail since it is possible the FActiveMountsLoader pointer holding this RegistrationLock is deleted
	 * during shutdown while we are about to try entering the RegistrationLock. We handle that detail by storing a
	 * shared pointer to the lock that might outlive the FActiveMountsLoader that created it.
	 */
	TSharedPtr<UE::FRecursiveMutex> RegistrationLock;
	/**
	 * The UnloadTask is created on demand if not yet created and it will keep running until it finds no more work to
	 * do and exits. When the task exits it clears this reference to itself.
	 */
	UE::Tasks::FTask UnloadTask;
	/**
	 * A count of how many FMountPoints have been requested to unload and not yet unloaded. Used to manage the
	 * async task that unloads them. This value contractually equals the number of FMountPointDatas that are in the
	 * unloading state.
	 */
	uint32 MountPointUnloadRefCount = 0;
	/**
	 * Configuration-specified variable for whether we should automatically request unload/reload when
	 * FPackageName::IMountPoints are dismounted/mounted.
	 */
	bool bAutoUnload : 1 = false;
	/** True only after ActiveMountsInitialize is called and so we are allowed to service Unload/Load requests. */
	bool bLoaderIsActive : 1 = false;
	/** InitialProviderOnReadyForRegister has been called. */
	bool bInitialProviderReadyForRegister : 1 = false;
	/** InitialProviderReport has been called. */
	bool bInitialProviderReported : 1 = false;
	/** Gate value to ensure Call_Once on the initial provider's registration. */
	bool bInitialProviderRegistered : 1 = false;
	/** Gate value to ensure Call_Once on bAutoUnload's trigger of UnloadAllUnmountedAsync. */
	bool bAutoUnloadInitialUnloadTriggered : 1 = false;
	/** Set to true when AssetRegistry is shutting down and we are waiting for async; blocks any external requests. */
	bool bShutdown : 1 = false;

	friend FActiveMountsLoaderWrapper;
};

} // namespace UE::AssetRegistry