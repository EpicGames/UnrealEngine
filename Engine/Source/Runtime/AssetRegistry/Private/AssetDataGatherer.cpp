// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDataGatherer.h"
#include "AssetDataGathererPrivate.h"
#include "AssetRegistry.h"
#include "AssetRegistry/AssetDataTagMapSerializationDetails.h"
#include "AssetRegistry/AssetRegistryTelemetry.h"
#include "AssetRegistryImpl.h"

#include "Algo/AnyOf.h"
#include "Algo/Find.h"
#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistryArchive.h"
#include "AssetRegistryPrivate.h"
#include "Async/ParallelFor.h"
#include "Containers/BinaryHeap.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformMutex.h"
#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "Hash/xxhash.h"
#include "Math/NumericLimits.h"
#include "Memory/MemoryView.h"
#include "Misc/AsciiSet.h"
#include "Misc/Char.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/CoreDelegates.h"
#include "Misc/DelayedAutoRegister.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeExit.h"
#include "Misc/TrackedActivity.h"
#include "Serialization/Archive.h"
#include "String/Find.h"
#include "String/LexFromString.h"
#include "Tasks/Task.h"
#include "Tasks/TaskConcurrencyLimiter.h"
#include "TelemetryRouter.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"
#include "Serialization/MemoryHasher.h"

namespace UE::AssetDataGather::Private
{

bool bBlockPackagesWithMarkOfTheWeb = false;
static FAutoConsoleVariableRef CVarBlockPackagesWithMarkOfTheWeb(
	TEXT("AssetRegistry.BlockPackagesWithMarkOfTheWeb"),
	bBlockPackagesWithMarkOfTheWeb,
	TEXT("Whether package files with mark of the web are blocked from the asset registry"));

bool bIgnoreEmptyDirectories = false;
static FAutoConsoleVariableRef CVarIgnoreEmptyDirectories(
	TEXT("AssetRegistry.IgnoreEmptyDirectories"),
	bIgnoreEmptyDirectories,
	TEXT("If true, completely empty leaf directories are ignored by the asset registry while scanning"));

bool bMemoryMappedCacheFiles = true;
static FAutoConsoleVariableRef CVarMemoryMappedCacheFiles(
	TEXT("AssetRegistry.MemoryMappedCacheFiles"),
	bMemoryMappedCacheFiles,
	TEXT("If true, asset registry cache file will be memory mapped, reducing memory usage"));

void LexFromString(EFeatureEnabledReadWrite& OutValue, FStringView Text)
{
	Text.TrimStartAndEndInline();

	if (Text.Equals(TEXTVIEW("NeverWriteNeverRead"), ESearchCase::IgnoreCase) ||
		Text.Equals(TEXTVIEW("Never"), ESearchCase::IgnoreCase))
	{
		OutValue = EFeatureEnabledReadWrite::NeverWriteNeverRead;
		return;
	}
	if (Text.Equals(TEXTVIEW("NeverWriteDefaultRead"), ESearchCase::IgnoreCase))
	{
		OutValue = EFeatureEnabledReadWrite::NeverWriteDefaultRead;
		return;
	}
	if (Text.Equals(TEXTVIEW("NeverWriteAlwaysRead"), ESearchCase::IgnoreCase))
	{
		OutValue = EFeatureEnabledReadWrite::NeverWriteAlwaysRead;
		return;
	}

	if (Text.Equals(TEXTVIEW("DefaultWriteNeverRead"), ESearchCase::IgnoreCase))
	{
		OutValue = EFeatureEnabledReadWrite::DefaultWriteNeverRead;
		return;
	}
	if (Text.Equals(TEXTVIEW("DefaultWriteDefaultRead"), ESearchCase::IgnoreCase) ||
		Text.Equals(TEXTVIEW("Default"), ESearchCase::IgnoreCase))
	{
		OutValue = EFeatureEnabledReadWrite::DefaultWriteDefaultRead;
		return;
	}
	if (Text.Equals(TEXTVIEW("DefaultWriteAlwaysRead"), ESearchCase::IgnoreCase))
	{
		OutValue = EFeatureEnabledReadWrite::DefaultWriteAlwaysRead;
		return;
	}

	if (Text.Equals(TEXTVIEW("AlwaysWriteNeverRead"), ESearchCase::IgnoreCase))
	{
		OutValue = EFeatureEnabledReadWrite::AlwaysWriteNeverRead;
		return;
	}
	if (Text.Equals(TEXTVIEW("AlwaysWriteDefaultRead"), ESearchCase::IgnoreCase) ||
		Text.Equals(TEXTVIEW("AlwaysWrite"), ESearchCase::IgnoreCase))
	{
		OutValue = EFeatureEnabledReadWrite::AlwaysWriteDefaultRead;
		return;
	}
	if (Text.Equals(TEXTVIEW("AlwaysWriteAlwaysRead"), ESearchCase::IgnoreCase))
	{
		OutValue = EFeatureEnabledReadWrite::AlwaysWriteAlwaysRead;
		return;
	}
	if (Text.Equals(TEXTVIEW("false"), ESearchCase::IgnoreCase) ||
		Text.Equals(TEXTVIEW("f"), ESearchCase::IgnoreCase) ||
		Text.Equals(TEXTVIEW("off"), ESearchCase::IgnoreCase) ||
		Text.Equals(TEXTVIEW("0"), ESearchCase::IgnoreCase))
	{
		OutValue = EFeatureEnabledReadWrite::NeverWriteNeverRead;
		return;
	}
	if (Text.Equals(TEXTVIEW("true"), ESearchCase::IgnoreCase) ||
		Text.Equals(TEXTVIEW("t"), ESearchCase::IgnoreCase) ||
		Text.Equals(TEXTVIEW("on"), ESearchCase::IgnoreCase))
	{
		OutValue = EFeatureEnabledReadWrite::DefaultWriteDefaultRead;
		return;
	}
	uint32 IntValue = 0;
	LexFromString(IntValue, Text);
	if (IntValue != 0)
	{
		OutValue = EFeatureEnabledReadWrite::DefaultWriteDefaultRead;
		return;
	}
	OutValue = EFeatureEnabledReadWrite::Invalid;
	return;
}

EEditorGameScanMode GetEditorGameScanModeFromConfig()
{
#if WITH_EDITOR
	FString EditorGameScansAR = "Sync";
	EEditorGameScanMode Result = EEditorGameScanMode::Sync;

	GConfig->GetString(TEXT("AssetRegistry"), TEXT("EditorGameScansAR"), EditorGameScansAR, GEngineIni);
	if (EditorGameScansAR.Equals(TEXT("Async"), ESearchCase::IgnoreCase))
	{
		Result = EEditorGameScanMode::Async;
	}
	else if (EditorGameScansAR.Equals(TEXT("False"), ESearchCase::IgnoreCase))
	{
		Result = EEditorGameScanMode::None;
	}
	else if (EditorGameScansAR.Equals(TEXT("True"), ESearchCase::IgnoreCase))
	{
		Result = EEditorGameScanMode::Sync;
	}
	else
	{
		ensureMsgf(EditorGameScansAR.Equals(TEXT("Sync"), ESearchCase::IgnoreCase),
			TEXT("Valid values for EditorGameScansAR are: true|false|sync|async. Received %s"), *EditorGameScansAR);
	}

	return Result;
#else
	return EEditorGameScanMode::None;
#endif
}

void FGatherSettings::Initialize()
{
	if (bInitialized)
	{
		return;
	}
	bInitialized = true;
	if (!FParse::Value(FCommandLine::Get(), TEXT("AssetRegistryCacheRootFolder="), AssetRegistryCacheRootFolder))
	{
		AssetRegistryCacheRootFolder = FPaths::ProjectIntermediateDir();
	}
	bForceDependsGathering = FParse::Param(FCommandLine::Get(), TEXT("ForceDependsGathering"));
#if WITH_EDITOR
	bGatherDependsData = bForceDependsGathering || !FParse::Param(FCommandLine::Get(), TEXT("NoDependsGathering"));
#else
	bGatherDependsData = bForceDependsGathering;
#endif
	bool bNoAssetRegistryCache = FParse::Param(FCommandLine::Get(), TEXT("NoAssetRegistryCache"));
	bool bNoAssetRegistryDiscoveryCache = bNoAssetRegistryCache || FParse::Param(FCommandLine::Get(), TEXT("NoAssetRegistryDiscoveryCache"));
	bool bNoAssetRegistryCacheRead = FParse::Param(FCommandLine::Get(), TEXT("NoAssetRegistryCacheRead"));
	uint32 MultiprocessId = UE::GetMultiprocessId();
	bool bMultiprocess = MultiprocessId > 0 || FParse::Param(FCommandLine::Get(), TEXT("multiprocess"));
	bool bNoAssetRegistryCacheWrite = FParse::Param(FCommandLine::Get(), TEXT("NoAssetRegistryCacheWrite"))
		// Don't write in multiprocess because we will collide writing the cache files
		|| bMultiprocess
		// Cooked game/server and editor -game do not need to write the cache; they get it from editor or cooking
		|| !GIsEditor;
	bGatherCacheReadEnabled = !bNoAssetRegistryCache && !bNoAssetRegistryCacheRead;
	bGatherCacheWriteEnabled = !bNoAssetRegistryCache && !bNoAssetRegistryCacheWrite;

	if (IsInGameThread())
	{
		// Ensure the TelemetryRouter is initialized from the game thread. It may need to load modules upon first access, so we 
		// fetch the singleton here to allow the module loading to remain an implementation detail. 
		// Modules can only be loaded from the game thread so we ensure we only attempt this during initialization on the game thread.
		static_cast<void>(FTelemetryRouter::Get());
	}

	bool bPlatformSupportsDiscoveryCacheInvalidation = FPlatformFileManager::Get().GetPlatformPhysical().FileJournalIsAvailable();

	bool bSkipInvalidate = FParse::Param(FCommandLine::Get(), TEXT("AssetRegistryCacheSkipInvalidate"));
	EFeatureEnabledReadWrite DiscoverySetting = EFeatureEnabledReadWrite::Invalid;
	FString AssetRegistryDiscoveryCacheStr = "Default";
	GConfig->GetString(TEXT("AssetRegistry"), TEXT("AssetRegistryDiscoveryCache"),
		AssetRegistryDiscoveryCacheStr, GEngineIni);
	FParse::Value(FCommandLine::Get(), TEXT("AssetRegistryDiscoveryCache="), AssetRegistryDiscoveryCacheStr);
	LexFromString(DiscoverySetting, AssetRegistryDiscoveryCacheStr);
	if (DiscoverySetting == EFeatureEnabledReadWrite::Invalid)
	{
		UE_LOGF(LogAssetRegistry, Error,
			"Invalid text \"%ls\" for Engine.ini:[AssetRegistry]:AssetRegistryDiscoveryCache. Expected \"Never\", \"Default\", or \"AlwaysWrite\".",
			*AssetRegistryDiscoveryCacheStr);
		DiscoverySetting = EFeatureEnabledReadWrite::DefaultWriteDefaultRead;
	}
	if (bNoAssetRegistryDiscoveryCache)
	{
		DiscoverySetting = EFeatureEnabledReadWrite::NeverWriteNeverRead;
	}
	if (bNoAssetRegistryCacheRead)
	{
		DiscoverySetting = (DiscoverySetting & ~EFeatureEnabledReadWrite::ReadMask) | EFeatureEnabledReadWrite::NeverRead;
	}
	if (bNoAssetRegistryCacheWrite)
	{
		DiscoverySetting = (DiscoverySetting & ~EFeatureEnabledReadWrite::WriteMask) | EFeatureEnabledReadWrite::NeverWrite;
	}
	if (bSkipInvalidate)
	{
		if ((DiscoverySetting & EFeatureEnabledReadWrite::WriteMask) == EFeatureEnabledReadWrite::DefaultWrite)
		{
			DiscoverySetting = (DiscoverySetting & ~EFeatureEnabledReadWrite::WriteMask) | EFeatureEnabledReadWrite::AlwaysWrite;
		}
		if ((DiscoverySetting & EFeatureEnabledReadWrite::ReadMask) == EFeatureEnabledReadWrite::DefaultRead)
		{
			DiscoverySetting = (DiscoverySetting & ~EFeatureEnabledReadWrite::ReadMask) | EFeatureEnabledReadWrite::AlwaysRead;
		}
	}
	else if (!bPlatformSupportsDiscoveryCacheInvalidation)
	{
		// Precalculate Default -> Never if we already know the platform doesn't support it
		if ((DiscoverySetting & EFeatureEnabledReadWrite::WriteMask) == EFeatureEnabledReadWrite::DefaultWrite)
		{
			DiscoverySetting = (DiscoverySetting& ~EFeatureEnabledReadWrite::WriteMask) | EFeatureEnabledReadWrite::NeverWrite;
		}
		if ((DiscoverySetting & EFeatureEnabledReadWrite::ReadMask) == EFeatureEnabledReadWrite::DefaultRead)
		{
			DiscoverySetting = (DiscoverySetting& ~EFeatureEnabledReadWrite::ReadMask) | EFeatureEnabledReadWrite::NeverRead;
		}
	}
	bDiscoveryCacheReadEnabled = (DiscoverySetting & EFeatureEnabledReadWrite::ReadMask) != EFeatureEnabledReadWrite::NeverRead;
	bDiscoveryCacheInvalidateEnabled = (DiscoverySetting & EFeatureEnabledReadWrite::ReadMask) != EFeatureEnabledReadWrite::AlwaysRead;
	switch (DiscoverySetting & EFeatureEnabledReadWrite::WriteMask)
	{
	case EFeatureEnabledReadWrite::NeverWrite:
		DiscoveryCacheWriteEnabled = EFeatureEnabled::Never;
		break;
	case EFeatureEnabledReadWrite::DefaultWrite:
		DiscoveryCacheWriteEnabled = EFeatureEnabled::IfPlatformSupported;
		break;
	case EFeatureEnabledReadWrite::AlwaysWrite:
		DiscoveryCacheWriteEnabled = EFeatureEnabled::Always;
		break;
	default:
		checkNoEntry();
		DiscoveryCacheWriteEnabled = EFeatureEnabled::Never;
		break;
	}

	EditorGameScanMode = GetEditorGameScanModeFromConfig();

	// If EditorGameScanMode is set to allow async then we will use async for any editor build (editor, game, or server)
	// Otherwise we will use async for commandlets and editor proper
	bAsyncEnabled = (!IsRunningGame() && !IsRunningDedicatedServer())
#if WITH_EDITOR
		|| (EditorGameScanMode == EEditorGameScanMode::Async)
#endif
		;

#if WITH_EDITOR || !UE_BUILD_SHIPPING
	bool bCommandlineSynchronous;
	if (FParse::Bool(FCommandLine::Get(), TEXT("AssetGatherSync="), bCommandlineSynchronous))
	{
		bAsyncEnabled = !bCommandlineSynchronous;
	}
#endif // WITH_EDITOR || !UE_BUILD_SHIPPING
	if (bAsyncEnabled && (!FPlatformProcess::SupportsMultithreading() || !FTaskGraphInterface::IsRunning()))
	{
		bAsyncEnabled = false;
		UE_LOGF(LogAssetRegistry, Warning, "Requested asynchronous asset data gather, but threading support is disabled. Performing a synchronous gather instead!");
	}

	const TCHAR* CacheBaseName = bGatherDependsData ? TEXT("CachedAssetRegistry") : TEXT("CachedAssetRegistryNoDeps");
	LegacyCacheBaseFilename = AssetRegistryCacheRootFolder / CacheBaseName;
	CacheBaseFilename = GetCacheDirectory() / CacheBaseName;
#if UE_EDITOR // See note on FPreloader for why we only allow preloading if UE_EDITOR
	bPreloadGatherCache = bAsyncEnabled && UE::AssetRegistry::ShouldSearchAllAssetsAtStart();
#else
	bPreloadGatherCache = false;
#endif

	FParse::Value(FCommandLine::Get(), TEXT("-ARDiscoverThreads="), FGatherSettings::GARDiscoverThreads);
	FParse::Value(FCommandLine::Get(), TEXT("-ARDiscoverMinBatchSize="), FGatherSettings::GARDiscoverMinBatchSize);
	FParse::Value(FCommandLine::Get(), TEXT("-ARGatherThreads="), FGatherSettings::GARGatherThreads);
	FParse::Value(FCommandLine::Get(), TEXT("-ARGatherCacheParallelism="), FGatherSettings::GARGatherCacheParallelism);
	FGatherSettings::GARDiscoverThreads = FMath::Max(0, FGatherSettings::GARDiscoverThreads);
	FGatherSettings::GARDiscoverMinBatchSize = FMath::Max(1, FGatherSettings::GARDiscoverMinBatchSize);
	FGatherSettings::GARGatherThreads = FMath::Max(0, FGatherSettings::GARGatherThreads);
	FGatherSettings::GARGatherCacheParallelism = FMath::Max(1, FGatherSettings::GARGatherCacheParallelism);
}

// Returns true if the filename's base (no extension) is an old-style sharded cache binary:
// CachedAssetRegistry_N (exactly one underscore, followed by a pure-digit shard number).
// New-style names have multiple underscores: CachedAssetRegistry_N_YYYYMMDDTHHMMSS_XXXX.
static bool IsOldStyleShardBinaryName(FStringView BaseName)
{
	// Old-style names have exactly one underscore (the separator before the shard number).
	// New-style names have at least three underscores (shard, timestamp, hex suffix).
	int32 FirstUnderscore = INDEX_NONE;
	if (!BaseName.FindChar(TEXT('_'), FirstUnderscore))
	{
		return false;
	}
	// If there's a second underscore after the first, it's new-style.
	FStringView AfterFirst = BaseName.Mid(FirstUnderscore + 1);
	int32 SecondUnderscore = INDEX_NONE;
	if (AfterFirst.FindChar(TEXT('_'), SecondUnderscore))
	{
		return false; // Multiple underscores -> new-style
	}
	// Single underscore: the suffix must be a non-empty sequence of digits (the shard number).
	if (AfterFirst.IsEmpty())
	{
		return false;
	}
	for (TCHAR C : AfterFirst)
	{
		if (!FChar::IsDigit(C))
		{
			return false;
		}
	}
	return true;
}

// Reads the binary filename stored in a .ref file. Returns empty string on any failure.
static FString ReadCacheRefFile(const FString& RefPath)
{
	FString Content;
	if (!FFileHelper::LoadFileToString(Content, *RefPath, FFileHelper::EHashOptions::None, FILEREAD_Silent))
	{
		return FString();
	}
	Content.TrimStartAndEndInline();
	return Content;
}

// Atomically writes a binary filename (relative) to a .ref file via a temp file + rename.
static bool WriteCacheRefFile(const FString& RefPath, const FString& BinaryRelName)
{
	const FString TempPath = RefPath + TEXT(".tmp");
	if (!FFileHelper::SaveStringToFile(BinaryRelName, *TempPath, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
	{
		UE_LOGF(LogAssetRegistry, Warning, "Failed to write cache ref file %ls", *TempPath);
		return false;
	}
	if (!IFileManager::Get().Move(*RefPath, *TempPath))
	{
		UE_LOGF(LogAssetRegistry, Warning, "Failed to rename cache ref file %ls -> %ls", *TempPath, *RefPath);
		IFileManager::Get().Delete(*TempPath, false, false, true);
		return false;
	}
	return true;
}

// Returns a unique binary filename for a given shard.
// Format: CacheBase_N_GUID.bin
static FString MakeShardBinaryFilename(const FString& CacheBaseFilename, int32 Shard)
{
	return FString::Printf(TEXT("%s_%d_%s.bin"),
		*CacheBaseFilename, Shard,
		*FGuid::NewGuid().ToString(EGuidFormats::Digits));
}

// Returns the .ref filename for a given shard: CacheBase_N.ref
static FString GetShardRefFilename(const FString& CacheBaseFilename, int32 Shard)
{
	return FString::Printf(TEXT("%s_%d.ref"), *CacheBaseFilename, Shard);
}

// Attempts to delete cache binary files that are not referenced by any .ref file.
// Delete failures are silently skipped (e.g. file still in use).
void CleanupOrphanedCacheFiles(const FString& CacheBaseFilename, ECacheCleanupContext Context)
{
	const FString Directory = FPaths::GetPath(CacheBaseFilename);

	// Collect all binary filenames (lowercase) currently referenced by .ref files
	TArray<FString> RefFilenames;
	IFileManager::Get().FindFiles(RefFilenames, *(CacheBaseFilename + TEXT("_*.ref")), true, false);

	TSet<FString> ReferencedBinaryNames;
	for (const FString& RefFilename : RefFilenames)
	{
		FString BinaryRelName = ReadCacheRefFile(Directory / RefFilename);
		if (!BinaryRelName.IsEmpty())
		{
			ReferencedBinaryNames.Add(BinaryRelName.ToLower());
		}
	}

	UE_LOGF(LogAssetRegistry, Display, "CleanupOrphanedCacheFiles (%ls): %d .ref files found, %d referenced binaries",
		Context == ECacheCleanupContext::PreLoad ? TEXT("PreLoad") : TEXT("PostWrite"),
		RefFilenames.Num(), ReferencedBinaryNames.Num());

	// Delete any leftover .tmp files from interrupted writes (always safe to remove)
	TArray<FString> TmpFiles;
	IFileManager::Get().FindFiles(TmpFiles, *(CacheBaseFilename + TEXT("_*.tmp")), true, false);
	for (const FString& TmpFilename : TmpFiles)
	{
		IFileManager::Get().Delete(*(Directory / TmpFilename), false, false, true);
	}

	// Try to delete binary files that are not referenced by any .ref
	TArray<FString> AllBinaries;
	IFileManager::Get().FindFiles(AllBinaries, *(CacheBaseFilename + TEXT("_*.bin")), true, false);

	int32 DeletedCount = 0;
	int32 SkippedReferencedCount = 0;
	int32 SkippedOldStyleCount = 0;
	int32 FailedDeleteCount = 0;

	for (const FString& BinaryFilename : AllBinaries)
	{
		// Skip if still referenced
		if (ReferencedBinaryNames.Contains(BinaryFilename.ToLower()))
		{
			UE_LOGF(LogAssetRegistry, Verbose, "  %ls: kept (referenced by .ref)", *BinaryFilename);
			++SkippedReferencedCount;
			continue;
		}

		FStringView BaseName = FPathViews::GetBaseFilename(FStringView(BinaryFilename));
		if (IsOldStyleShardBinaryName(BaseName))
		{
			if (Context == ECacheCleanupContext::PreLoad)
			{
				// PreLoad: keep old-style bins - FindShardedCacheFiles needs them for migration fallback.
				UE_LOGF(LogAssetRegistry, Verbose, "  %ls: kept (old-style, needed for migration fallback)", *BinaryFilename);
				++SkippedOldStyleCount;
				continue;
			}
			// PostWrite: all shards have .ref files now, old-style bins are fully superseded.
		}

		// Deletion may fail if the file is still in use; that is expected and fine.
		if (IFileManager::Get().Delete(*(Directory / BinaryFilename), false, false, true))
		{
			UE_LOGF(LogAssetRegistry, Display, "  %ls: deleted (orphaned, unreferenced by any .ref)", *BinaryFilename);
			++DeletedCount;
		}
		else
		{
			++FailedDeleteCount;
			UE_LOGF(LogAssetRegistry, Display,
				"  %ls: delete failed (orphaned but still in use)",
				*BinaryFilename);
		}
	}

	UE_LOGF(LogAssetRegistry, Display,
		"CleanupOrphanedCacheFiles: %d total binaries, %d referenced (kept), %d old-style pre-migration (kept), %d orphans deleted, %d orphans locked (kept)",
		AllBinaries.Num(), SkippedReferencedCount, SkippedOldStyleCount, DeletedCount, FailedDeleteCount);

	// Also clean up legacy cache files in the old location (Intermediate/CachedAssetRegistry_N.bin).
	// In PostWrite context all shards have .ref files, so legacy files are fully superseded.
	// In PreLoad context keep them if there are no .ref files (migration fallback may still need them).
	const FString& LegacyBase = GGatherSettings.GetLegacyCacheBaseFilename();
	if (!LegacyBase.IsEmpty() && LegacyBase != CacheBaseFilename)
	{
		const FString LegacyDir = FPaths::GetPath(LegacyBase);
		TArray<FString> LegacyBinaries;
		IFileManager::Get().FindFiles(LegacyBinaries, *(LegacyBase + TEXT("_*.bin")), true, false);

		if (LegacyBinaries.Num() > 0)
		{
			bool bShouldDeleteLegacy = (Context == ECacheCleanupContext::PostWrite) || (RefFilenames.Num() > 0);
			if (bShouldDeleteLegacy)
			{
				int32 LegacyDeletedCount = 0;
				int32 LegacyFailedCount = 0;
				for (const FString& LegacyBin : LegacyBinaries)
				{
					if (IFileManager::Get().Delete(*(LegacyDir / LegacyBin), false, false, true))
					{
						++LegacyDeletedCount;
					}
					else
					{
						++LegacyFailedCount;
					}
				}
				UE_LOGF(LogAssetRegistry, Display,
					"CleanupOrphanedCacheFiles: legacy location %ls: %d deleted, %d locked (kept)",
					*LegacyDir, LegacyDeletedCount, LegacyFailedCount);
			}
			else
			{
				UE_LOGF(LogAssetRegistry, Display,
					"CleanupOrphanedCacheFiles: legacy location %ls: %d files kept (no .ref files yet, needed for migration fallback)",
					*LegacyDir, LegacyBinaries.Num());
			}
		}

		// Also delete legacy monolithic cache file (CachedAssetRegistry.bin) if it exists
		IFileManager::Get().Delete(*(LegacyBase + TEXT(".bin")), false, false, true);
	}
}

TArray<FString> FGatherSettings::FindShardedCacheFiles() const
{
	const FString Directory = FPaths::GetPath(GetCacheBaseFilename());
	TArray<FString> Result;

	// Primary path: find all .ref files and resolve them to binary paths
	TArray<FString> RefFilenames;
	IFileManager::Get().FindFiles(RefFilenames, *(GetCacheBaseFilename() + TEXT("_*.ref")), true, false);

	TSet<int32> HandledShards;
	for (const FString& RefFilename : RefFilenames)
	{
		const FString RefPath = Directory / RefFilename;
		const FString BinaryRelName = ReadCacheRefFile(RefPath);
		if (BinaryRelName.IsEmpty())
		{
			UE_LOGF(LogAssetRegistry, Warning, "Cache ref file is empty or unreadable: %ls", *RefPath);
			continue;
		}
		const FString BinaryPath = Directory / BinaryRelName;
		if (!IFileManager::Get().FileExists(*BinaryPath))
		{
			UE_LOGF(LogAssetRegistry, Warning,
				"Cache ref file %ls points to missing binary %ls, shard will be regenerated",
				*RefPath, *BinaryPath);
			continue;
		}
		Result.Add(BinaryPath);

		// Track which shard number this .ref covers (for migration fallback below)
		FStringView RefBase = FPathViews::GetBaseFilename(FStringView(RefFilename));
		int32 LastUnderscore = INDEX_NONE;
		RefBase.FindLastChar(TEXT('_'), LastUnderscore);
		if (LastUnderscore != INDEX_NONE)
		{
			int32 ShardNum = 0;
			LexFromString(ShardNum, RefBase.Mid(LastUnderscore + 1));
			HandledShards.Add(ShardNum);
		}
	}

	// Migration fallback: include old-style _N.bin files for shards not yet covered by a .ref
	int32 MigrationCount = 0;
	TArray<FString> OldStyleBinaries;
	IFileManager::Get().FindFiles(OldStyleBinaries, *(GetCacheBaseFilename() + TEXT("_*.bin")), true, false);
	for (const FString& BinaryFilename : OldStyleBinaries)
	{
		FStringView BaseName = FPathViews::GetBaseFilename(FStringView(BinaryFilename));
		if (!IsOldStyleShardBinaryName(BaseName))
		{
			continue; // new-style binary; covered by the .ref lookup above
		}
		int32 LastUnderscore = INDEX_NONE;
		BaseName.FindLastChar(TEXT('_'), LastUnderscore);
		int32 ShardNum = 0;
		LexFromString(ShardNum, BaseName.Mid(LastUnderscore + 1));
		if (!HandledShards.Contains(ShardNum))
		{
			UE_LOGF(LogAssetRegistry, Display, "FindShardedCacheFiles: migration fallback for shard %d -> %ls", ShardNum, *BinaryFilename);
			Result.Add(Directory / BinaryFilename);
			++MigrationCount;
		}
	}

	UE_LOGF(LogAssetRegistry, Display, "FindShardedCacheFiles: %d via .ref, %d via migration fallback, %d total",
		Result.Num() - MigrationCount, MigrationCount, Result.Num());

	return Result;
}

TArray<FString> FGatherSettings::FindLegacyShardedCacheFiles() const
{
	TArray<FString> CachePaths;
	IFileManager::Get().FindFiles(CachePaths, *(GetLegacyCacheBaseFilename() + TEXT("_*.bin")), /* Files */ true, /* Directories */ false);
	if (CachePaths.Num())
	{
		FString Directory = FPaths::GetPath(GetLegacyCacheBaseFilename());
		for (FString& Path : CachePaths)
		{
			Path = Directory / Path;
		}
	}
	return CachePaths;
}

FGatherSettings GGatherSettings;

#define CACHEFILE_SYSTEMWIDE_WRITELOCK WITH_EDITOR

#if CACHEFILE_SYSTEMWIDE_WRITELOCK
// Attempts to lock a system-wide mutex guarding the writing of AssetRegistry cache files. If already locked, no waiting occurs.
// Users should verify the lock state after calling this function.
[[nodiscard]] UE::FPlatformSystemWideMutex TryLockSystemWideCacheFilesWrites()
{
	check(GGatherSettings.IsInitialized());
	TStringBuilder<256> Builder;
	Builder.Append(*FPaths::Combine(*GGatherSettings.GetAssetRegistryCacheRootFolder(), TEXT("AssetRegistryCacheLock")));
	return UE::FPlatformSystemWideMutex(Builder.ToString());
}
#endif

/** A structure to hold serialized cache data from async loads before adding it to the Gatherer's main cache. */
struct FCachePayload
{
	TUniquePtr<FName[]> PackageNames;
	TUniquePtr<FDiskCachedAssetData[]> AssetDatas;
	int32 NumAssets = 0;
	bool bSucceeded = false;
	void Reset()
	{
		PackageNames.Reset();
		AssetDatas.Reset();
		NumAssets = 0;
		bSucceeded = false;
	}
};

void SerializeCacheSave(FAssetRegistryWriter& Ar, const TArray<TPair<FName, FDiskCachedAssetData*>>& AssetsToSave);
FCachePayload SerializeCacheLoad(FAssetRegistryReader& Ar);
TArray<FCachePayload> LoadCacheFiles(TConstArrayView<FString> CacheFilenames, bool bCacheWriteEnabled);
void DeleteCacheFilesWithinLock();

/** InOutResult = Value, but without shrinking the string to fit. */
void AssignStringWithoutShrinking(FString& InOutResult, FStringView Value)
{
	TArray<TCHAR, FString::AllocatorType>& Result = InOutResult.GetCharArray();
	if (Value.IsEmpty())
	{
		Result.Reset();
	}
	else
	{
		Result.SetNumUninitialized(Value.Len() + 1, EAllowShrinking::No);
		FMemory::Memcpy(Result.GetData(), Value.GetData(), Value.Len() * sizeof(Value[0]));
		Result[Value.Len()] = '\0';
	}
}

FDiscoveredPathData::FDiscoveredPathData(FStringView InLocalAbsPath, FStringView InLongPackageName,
	FStringView InRelPath, const FDateTime& InPackageTimestamp, bool InIsReadOnly, EGatherableFileType InType)
	: LocalAbsPath(InLocalAbsPath)
	, LongPackageName(InLongPackageName)
	, RelPath(InRelPath)
	, PackageTimestamp(InPackageTimestamp)
	, bIsReadOnly(InIsReadOnly)
	, Type(InType)
{
}

FDiscoveredPathData::FDiscoveredPathData(FStringView InLocalAbsPath, FStringView InLongPackageName, 
	FStringView InRelPath, EGatherableFileType InType)
	: LocalAbsPath(InLocalAbsPath)
	, LongPackageName(InLongPackageName)
	, RelPath(InRelPath)
	, bIsReadOnly(false)
	, Type(InType)
{
}

void FDiscoveredPathData::Assign(FStringView InLocalAbsPath, FStringView InLongPackageName, FStringView InRelPath,
	EGatherableFileType InType)
{
	AssignStringWithoutShrinking(LocalAbsPath, InLocalAbsPath);
	AssignStringWithoutShrinking(LongPackageName, InLongPackageName);
	AssignStringWithoutShrinking(RelPath, InRelPath);
	Type = InType;
}

void FDiscoveredPathData::Assign(FStringView InLocalAbsPath, FStringView InLongPackageName, FStringView InRelPath,
	const FDateTime& InPackageTimestamp, bool InIsReadOnly, EGatherableFileType InType)
{
	Assign(InLocalAbsPath, InLongPackageName, InRelPath, InType);
	PackageTimestamp = InPackageTimestamp;
	bIsReadOnly = InIsReadOnly;
}

SIZE_T FDiscoveredPathData::GetAllocatedSize() const
{
	return LocalAbsPath.GetAllocatedSize() + LongPackageName.GetAllocatedSize() + RelPath.GetAllocatedSize();
}

FGatheredPathData::FGatheredPathData(FStringView InLocalAbsPath, FStringView InLongPackageName,
	const FDateTime& InPackageTimestamp, bool InIsReadOnly, EGatherableFileType InType)
	: LocalAbsPath(InLocalAbsPath)
	, LongPackageName(InLongPackageName)
	, PackageTimestamp(InPackageTimestamp)
	, bIsReadOnly(InIsReadOnly)
	, Type(InType)
{
}

FGatheredPathData::FGatheredPathData(const FDiscoveredPathData& DiscoveredData)
	: FGatheredPathData(DiscoveredData.LocalAbsPath, DiscoveredData.LongPackageName,
		DiscoveredData.PackageTimestamp, DiscoveredData.bIsReadOnly, DiscoveredData.Type)
{
}

FGatheredPathData::FGatheredPathData(FDiscoveredPathData&& DiscoveredData)
	: LocalAbsPath(MoveTemp(DiscoveredData.LocalAbsPath))
	, LongPackageName(MoveTemp(DiscoveredData.LongPackageName))
	, PackageTimestamp(MoveTemp(DiscoveredData.PackageTimestamp))
	, bIsReadOnly(DiscoveredData.bIsReadOnly)
	, Type(DiscoveredData.Type)
{
}

void FGatheredPathData::Assign(FStringView InLocalAbsPath, FStringView InLongPackageName,
	const FDateTime& InPackageTimestamp, bool InIsReadOnly, EGatherableFileType InType)
{
	AssignStringWithoutShrinking(LocalAbsPath, InLocalAbsPath);
	AssignStringWithoutShrinking(LongPackageName, InLongPackageName);
	PackageTimestamp = InPackageTimestamp;
	bIsReadOnly = InIsReadOnly;
	Type = InType;
}

void FGatheredPathData::Assign(const FDiscoveredPathData& DiscoveredData)
{
	Assign(DiscoveredData.LocalAbsPath, DiscoveredData.LongPackageName, DiscoveredData.PackageTimestamp,
		DiscoveredData.bIsReadOnly, DiscoveredData.Type);
}

SIZE_T FGatheredPathData::GetAllocatedSize() const
{
	return LocalAbsPath.GetAllocatedSize() + LongPackageName.GetAllocatedSize();
}

FScanDir::FScanDir(FMountDir& InMountDir, FScanDir* InParent, FStringView InRelPath)
	: MountDir(&InMountDir)
	, Parent(InParent)
	, RelPath(InRelPath)
{
	FAssetDataDiscovery& Discovery = InMountDir.GetDiscovery();
	Discovery.NumDirectoriesToScan.Increment();
}

FScanDir::~FScanDir()
{
	// Assert that Shutdown has been called to confirm that the parent no longer has a reference we need to clear.
	check(!MountDir);
}

void FScanDir::Shutdown()
{
	if (!MountDir)
	{
		// Already shutdown
		return;
	}

	// Shutdown all children
	for (TRefCountPtr<FScanDir>& ScanDir : SubDirs)
	{
		// Destruction contract for FScanDir requires that the parent calls Shutdown before dropping the reference
		ScanDir->Shutdown();
		ScanDir.SafeRelease();
	}
	SubDirs.Empty();

	// Update MountDir data that we influence
	if (!bIsComplete)
	{
		MountDir->GetDiscovery().NumDirectoriesToScan.Decrement();
	}

	// Clear backpointers (which also marks us as shutdown)
	MountDir = nullptr;
	Parent = nullptr;
}

bool FScanDir::IsValid() const
{
	return MountDir != nullptr;
}

FMountDir* FScanDir::GetMountDir() const
{
	return MountDir;
}

FStringView FScanDir::GetRelPath() const
{
	return RelPath;
}

void FScanDir::AppendLocalAbsPath(FStringBuilderBase& OutFullPath) const
{
	if (!MountDir)
	{
		return;
	}

	if (Parent)
	{
		Parent->AppendLocalAbsPath(OutFullPath);
		FPathViews::AppendPath(OutFullPath, RelPath);
	}
	else
	{
		// The root ScanDir should have an empty RelPath from the MountDir
		check(RelPath.IsEmpty());
		OutFullPath << MountDir->GetLocalAbsPath();
	}
}

FString FScanDir::GetLocalAbsPath() const
{
	TStringBuilder<128> Result;
	AppendLocalAbsPath(Result);
	return FString(Result);
}

void FScanDir::AppendMountRelPath(FStringBuilderBase& OutRelPath) const
{
	if (!MountDir)
	{
		return;
	}

	if (Parent)
	{
		Parent->AppendMountRelPath(OutRelPath);
		FPathViews::AppendPath(OutRelPath, RelPath);
	}
	else
	{
		// The root ScanDir should have an empty RelPath from the MountDir
		check(RelPath.IsEmpty());
	}
}

FString FScanDir::GetMountRelPath() const
{
	TStringBuilder<128> Result;
	AppendMountRelPath(Result);
	return FString(Result);
}

bool FScanDir::FInherited::IsMonitored() const
{
	return IsOnAllowList() && !IsOnDenyList();
}

bool FScanDir::FInherited::IsOnDenyList() const
{
	return bMatchesDenyList && !bIgnoreDenyList;
}

bool FScanDir::FInherited::IsOnAllowList() const
{
	return bIsOnAllowList;
}

bool FScanDir::FInherited::HasSetting() const
{
	return bIsOnAllowList || bMatchesDenyList || bIgnoreDenyList;
}

FScanDir::FInherited::FInherited(const FInherited& Parent, const FInherited& Child)
	: bIsOnAllowList(Parent.bIsOnAllowList || Child.bIsOnAllowList)
	, bMatchesDenyList(Parent.bMatchesDenyList || Child.bMatchesDenyList)
	, bIgnoreDenyList(Parent.bIgnoreDenyList || Child.bIgnoreDenyList)
{
}

void FScanDir::GetMonitorData(FStringView InRelPath, const FInherited& ParentData, FInherited& OutData) const
{
	if (!MountDir)
	{
		OutData = FInherited();
		return;
	}

	FInherited Accumulated(ParentData, DirectData);

	const FScanDir* SubDir = nullptr;
	FStringView FirstComponent;
	FStringView RemainingPath;
	if (!InRelPath.IsEmpty())
	{
		FPathViews::SplitFirstComponent(InRelPath, FirstComponent, RemainingPath);
		SubDir = FindSubDir(FirstComponent);
	}
	if (!SubDir)
	{
		OutData = Accumulated;
	}
	else
	{
		SubDir->GetMonitorData(RemainingPath, Accumulated, OutData);
	}
}

bool FScanDir::IsMonitored(const FInherited& ParentData) const
{
	if (!MountDir)
	{
		return false;
	}
	FInherited Accumulated(ParentData, DirectData);
	return Accumulated.IsMonitored();
}

bool FScanDir::ShouldScan(const FInherited& ParentData) const
{
	return !bHasScanned && IsMonitored(ParentData);
}

bool FScanDir::HasScanned() const
{
	return bHasScanned;
}

bool FScanDir::IsComplete() const
{
	return bIsComplete;
}

SIZE_T FScanDir::GetAllocatedSize() const
{
	SIZE_T Result = 0;
	Result += SubDirs.GetAllocatedSize();
	for (const TRefCountPtr<FScanDir>& Value : SubDirs)
	{
		Result += sizeof(*Value);
		Result += Value->GetAllocatedSize();
	}
	Result += AlreadyScannedFiles.GetAllocatedSize();
	for (const FString& Value : AlreadyScannedFiles)
	{
		Result += Value.GetAllocatedSize();
	}
	Result += RelPath.GetAllocatedSize();
	return Result;
}

FScanDir* FScanDir::GetControllingDir(FStringView InRelPath, bool bIsDirectory, const FInherited& ParentData, FInherited& OutData, FString& OutRelPath)
{
	// GetControllingDir can only be called on valid ScanDirs, which we rely on since we need to call FindOrAddSubDir which relies on that
	check(IsValid());

	FInherited Accumulated(ParentData, DirectData);
	if (InRelPath.IsEmpty())
	{
		if (!bIsDirectory)
		{
			UE_LOGF(LogAssetRegistry, Warning, "GetControllingDir called on %ls with !bIsDirectory, but we have it recorded as a directory. Returning null.", *GetLocalAbsPath());
			OutData = FInherited();
			OutRelPath.Reset();
			return nullptr;
		}
		else
		{
			OutData = Accumulated;
			OutRelPath = InRelPath;
			return this;
		}
	}

	FStringView FirstComponent;
	FStringView RemainingPath;
	FPathViews::SplitFirstComponent(InRelPath, FirstComponent, RemainingPath);
	if (RemainingPath.IsEmpty() && !bIsDirectory)
	{
		OutData = Accumulated;
		OutRelPath = InRelPath;
		return this;
	}
	else
	{
		FScanDir* SubDir = nullptr;
		if (ShouldScan(ParentData))
		{
			SubDir = &FindOrAddSubDir(FirstComponent);
		}
		else
		{
			SubDir = FindSubDir(FirstComponent);
			if (!SubDir)
			{
				OutData = Accumulated;
				OutRelPath = InRelPath;
				return this;
			}
		}
		return SubDir->GetControllingDir(RemainingPath, bIsDirectory, Accumulated, OutData, OutRelPath);
	}
}

void FScanDir::TrySetDirectoryProperties(FPathExistence& QueryPath, FStringView InRelPath, FInherited& ParentData,
	const FSetPathProperties& InProperties, FScanDirAndParentData& OutControllingDir,
	FStringView& OutControllingDirRelPath, bool& bInOutMadeChanges)
{
	// TrySetDirectoryProperties can only be called on valid ScanDirs, which we rely on because we call FindOrAddSubDir
	// which relies on that.
	check(IsValid()); 

	if (InRelPath.IsEmpty())
	{
		// The properties apply to this entire directory
		if (InProperties.IsOnAllowList.IsSet() && DirectData.bIsOnAllowList != *InProperties.IsOnAllowList)
		{
			bInOutMadeChanges = true;
			SetComplete(false);
			if (bScanInFlight)
			{
				bScanInFlightInvalidated = true;
			}
			DirectData.bIsOnAllowList = *InProperties.IsOnAllowList;

			if (DirectData.bIsOnAllowList)
			{
				// Since we are setting this directory to be monitored, we need to implement the guarantee that all Monitored flags of its children are set to false
				// We also need to SetComplete false on all directories in between this and a previously allow listed directory, since those non-allow listed parent directories
				// marked themselves complete once their allow listed children finished
				ForEachDescendent([](FScanDir& ScanDir)
					{
						ScanDir.DirectData.bIsOnAllowList = false;
						ScanDir.SetComplete(false);
					});
			}
			else
			{
				// Cancel any scans since they are no longer allow listed
				ForEachDescendent([](FScanDir& ScanDir)
					{
						if (ScanDir.bScanInFlight)
						{
							ScanDir.bScanInFlightInvalidated = true;
						}
					});
			}
		}
		if ((InProperties.MatchesDenyList.IsSet() && DirectData.bMatchesDenyList != *InProperties.MatchesDenyList) ||
			(InProperties.IgnoreDenyList.IsSet() && DirectData.bIgnoreDenyList != *InProperties.IgnoreDenyList))
		{
			bInOutMadeChanges = true;
			SetComplete(false);
			if (InProperties.MatchesDenyList.IsSet())
			{
				DirectData.bMatchesDenyList = *InProperties.MatchesDenyList;
			}
			if (InProperties.IgnoreDenyList.IsSet())
			{
				DirectData.bIgnoreDenyList = *InProperties.IgnoreDenyList;
			}
			bool bIgnoreDenyList = false;
			bool bMatchesDenyList = false;
			for (FScanDir* Current = this; Current; Current = Current->Parent)
			{
				bIgnoreDenyList = bIgnoreDenyList || Current->DirectData.bIgnoreDenyList;
				bMatchesDenyList = bMatchesDenyList || Current->DirectData.bMatchesDenyList;
			}
			bool bIsOnDenyList = bMatchesDenyList && !bIgnoreDenyList;

			// Mark all children as incomplete
			// Also cancel any scans since they are now potentially on the deny list
			if (bIsOnDenyList && bScanInFlight)
			{
				bScanInFlightInvalidated = true;
			}
			ForEachDescendent([&bIsOnDenyList](FScanDir& ScanDir)
				{
					if (bIsOnDenyList && ScanDir.bScanInFlight)
					{
						ScanDir.bScanInFlightInvalidated = true;
					}
					ScanDir.SetComplete(false);
				});
		}
		if (InProperties.HasScanned.IsSet())
		{
			bInOutMadeChanges = true;
			SetComplete(false);
			bool bNewValue = *InProperties.HasScanned;
			auto SetProperties = [bNewValue](FScanDir& ScanDir)
			{
				if (ScanDir.bScanInFlight)
				{
					ScanDir.bScanInFlightInvalidated = true;
				}
				ScanDir.bHasScanned = bNewValue;
				ScanDir.AlreadyScannedFiles.Reset();
			};
			SetProperties(*this);
			ForEachDescendent(SetProperties);
		}

		OutControllingDir.ScanDir = this;
		OutControllingDir.ParentData = ParentData;
		OutControllingDirRelPath = FStringView();
		return;
	}
	else
	{
		TOptional<FSetPathProperties> ModifiedProperties;
		const FSetPathProperties* Properties = &InProperties;
		if (Properties->IsOnAllowList.IsSet() && DirectData.bIsOnAllowList)
		{
			// If this directory is set to be monitored, all Monitored flags of its children are unused, are guaranteed set to false, and should not be changed
			ModifiedProperties = *Properties;
			ModifiedProperties->IsOnAllowList.Reset();
			Properties = &ModifiedProperties.GetValue();
		}

		FStringView FirstComponent;
		FStringView Remainder;
		FPathViews::SplitFirstComponent(InRelPath, FirstComponent, Remainder);

		FInherited ParentDataForSubDir(ParentData, DirectData);

		FScanDir* SubDir = nullptr;
		if (bHasScanned &&
			(!Properties->HasScanned.IsSet() || *Properties->HasScanned == true) &&
			(!Properties->IsOnAllowList.IsSet() || *Properties->IsOnAllowList == ParentDataForSubDir.bIsOnAllowList) &&
			(!Properties->IgnoreDenyList.IsSet() || *Properties->IgnoreDenyList == ParentDataForSubDir.bIgnoreDenyList) &&
			(!Properties->MatchesDenyList.IsSet() || *Properties->MatchesDenyList == ParentDataForSubDir.bMatchesDenyList)
			)
		{
			// If this parent directory has already been scanned and we are not changing any values on the target
			// path to a different value than the current directory, and the next child subdirectory is not recorded
			// on *this, then one of these is true and we can early exit and report *this as the ControllingDir.
			//     * The QueryPath is a file path instead of a directory and we don't need to take any action for it
			//     * The next child directory towards the QueryPath has already been completed and we do not need
			//       to set any properties on it.
			SubDir = FindSubDir(FirstComponent);
			if (!SubDir)
			{
				OutControllingDir.ScanDir = this;
				OutControllingDir.ParentData = ParentData;
				OutControllingDirRelPath = InRelPath;
				return;
			}
		}
		else
		{
			SubDir = FindSubDir(FirstComponent);
			if (!SubDir)
			{
				if (!QueryPath.HasExistenceData())
				{
					QueryPath.LoadExistenceData();
					// InRelPath might have been invalidated by the call to LoadExistenceData, and
					// it's capitalization might have changed anyway. Recreate InRelPath from the QueryPath's
					// relative path from this.
					FString ThisDirAbsPath = GetLocalAbsPath();
					// TryMakeChildPathRelativeTo should succeed because our caller promises that QueryPath's relative
					// path from this exists in the old InRelPath.
					ensure(FPathViews::TryMakeChildPathRelativeTo(QueryPath.GetLocalAbsPath(), ThisDirAbsPath, InRelPath));
					FPathViews::SplitFirstComponent(InRelPath, FirstComponent, Remainder);
				}

				// If the path does not exist on disk, then contractually we are not required to set any properties for it
				// and we can return this, the closest existing scandir to its path.
				// If the path is a file, then we need to create directories down to its parent directory, and set the
				// requested properties on its parent directory, and we exit out when we have found its parent directory
				// which we detect by SplitFirstComponent returns a single component with no remainder.
				if (QueryPath.GetType() == FPathExistence::EType::MissingParentDir
					|| (Remainder.IsEmpty() && QueryPath.GetType() != FPathExistence::EType::Directory))
				{
					OutControllingDir.ScanDir = this;
					OutControllingDir.ParentData = ParentData;
					OutControllingDirRelPath = InRelPath;
					return;
				}

				SubDir = &FindOrAddSubDir(FirstComponent);
				// If the current directory has already been scanned then the SubDir we just created must have been
				// previously discovered, or it was created on disk after the last time we scanned the current
				// directory. If it was created on disk, and it is not being force rescanned
				// (aka Properties->HasScanned=true) then we are allowed to ignore it. If it was previously discovered,
				// then we completed and deleted it: either it was not IsMonitored, or we scanned it. To avoid an
				// unnecessary rescan, we should therefore set bHasScanned=true if the current directory has
				// bHasScanned=true and the SubDir has IsMonitored=true. If force rescan is requested on the SubDir,
				// then we will set it back to bHasScanned=false in TrySetDirectoryProperties below.
				if (this->bHasScanned)
				{
					if (SubDir->IsMonitored(ParentDataForSubDir))
					{
						SubDir->bHasScanned = true;
					}
				}
				bInOutMadeChanges = true;
				SetComplete(false);
			}
		}

		SubDir->TrySetDirectoryProperties(QueryPath, Remainder, ParentDataForSubDir, *Properties,
			OutControllingDir, OutControllingDirRelPath, bInOutMadeChanges);
		if (OutControllingDir.ScanDir && !OutControllingDir.ScanDir->IsComplete())
		{
			bInOutMadeChanges = true;
			SetComplete(false);
		}
		return;
	}
}

void FScanDir::MarkFileAlreadyScanned(FStringView BaseName)
{
	if (bHasScanned)
	{
		return;
	}
	check(FPathViews::IsPathLeaf(BaseName));
	for (const FString& AlreadyScannedFile : AlreadyScannedFiles)
	{
		if (FStringView(AlreadyScannedFile).Equals(BaseName, ESearchCase::IgnoreCase))
		{
			return;
		}
	}
	AlreadyScannedFiles.Emplace(BaseName);
}

void FScanDir::SetScanResults(FStringView LocalAbsPath, const FInherited& ParentData, TArrayView<FDiscoveredPathData>& InOutSubDirs, TArrayView<FDiscoveredPathData>& InOutFiles)
{
	SetComplete(false);
	check(!bScanInFlightInvalidated);
	check(MountDir);

	if (!ensure(!bHasScanned))
	{
		return;
	}
	FInherited Accumulated(ParentData, DirectData);

	// Add SubDirectories in the tree for the directories found by the scan, and report the directories as discovered directory paths as well
	for (int32 Index = 0; Index < InOutSubDirs.Num(); )
	{
		FDiscoveredPathData& SubDirPath = InOutSubDirs[Index];
		FScanDir& SubScanDir = FindOrAddSubDir(SubDirPath.RelPath);
		bool bReportResult = SubScanDir.IsMonitored(Accumulated);
		if (!bReportResult)
		{
			Swap(SubDirPath, InOutSubDirs.Last());
			InOutSubDirs = InOutSubDirs.Slice(0, InOutSubDirs.Num() - 1);
		}
		else
		{
			++Index;
		}
	}

	// Add the files that were found in the scan, skipping any files that have already been scanned
	if (InOutFiles.Num())
	{
		auto IsAlreadyScanned = [this, &LocalAbsPath](const FDiscoveredPathData& InFile)
		{
			return Algo::AnyOf(AlreadyScannedFiles, [&InFile](const FString& AlreadyScannedFileRelPath) { return FPathViews::Equals(AlreadyScannedFileRelPath, InFile.RelPath); });
		};
		bool bScanAll = AlreadyScannedFiles.Num() == 0;
		for (int32 Index = 0; Index < InOutFiles.Num(); )
		{
			FDiscoveredPathData& InFile = InOutFiles[Index];
			if (!bScanAll && IsAlreadyScanned(InFile))
			{
				// Remove this file from InOutFiles
				Swap(InFile, InOutFiles.Last());
				InOutFiles = InOutFiles.Slice(0, InOutFiles.Num() - 1);
			}
			else
			{
				++Index;
			}
		}
	}
	AlreadyScannedFiles.Empty();

	MountDir->SetHasStartedScanning();
	bHasScanned = true;
}

void FScanDir::Update(TArray<FScanDirAndParentData>& OutScanRequests, const FScanDir::FInherited& ParentData)
{
	check(MountDir);
	if (bIsComplete)
	{
		return;
	}

	bool bScanThis = ShouldScan(ParentData);
	if (bScanThis)
	{
		OutScanRequests.Add(FScanDirAndParentData{ this, ParentData });
	}

	bool bAllSubDirsComplete = true;
	if (SubDirs.Num())
	{
		FInherited ParentDataForSubDirs(ParentData, DirectData);
		TArray<TRefCountPtr<FScanDir>> CopySubDirs = SubDirs;
		for (const TRefCountPtr<FScanDir>& SubDir : CopySubDirs)
		{
			if (SubDir->bIsComplete)
			{
				continue;
			}
			int32 PreviousCount = OutScanRequests.Num();
			SubDir->Update(OutScanRequests, ParentDataForSubDirs);
			bool bSubDirComplete = SubDir->IsComplete();
			check(OutScanRequests.Num() > PreviousCount || bSubDirComplete);
			bAllSubDirsComplete &= bSubDirComplete;
		}
	}

	if (bScanThis || !bAllSubDirsComplete)
	{
		return;
	}

	SetComplete(true);
	// After calling SetComplete, this may have been removed from tree and should no longer run calculations
}

FScanDir* FScanDir::GetFirstIncompleteScanDir()
{
	for (const TRefCountPtr<FScanDir>& SubDir : SubDirs)
	{
		FScanDir* Result = SubDir->GetFirstIncompleteScanDir();
		if (Result)
		{
			return Result;
		}
	}
	if (!bIsComplete)
	{
		return this;
	}
	return nullptr;
}

bool FScanDir::IsScanInFlight() const
{
	return bScanInFlight;
}

void FScanDir::SetScanInFlight(bool bInScanInFlight)
{
	bScanInFlight = bInScanInFlight;
}

bool FScanDir::IsScanInFlightInvalidated() const
{
	return bScanInFlightInvalidated;
}

void FScanDir::SetScanInFlightInvalidated(bool bInvalidated)
{
	bScanInFlightInvalidated = bInvalidated;
}

void FScanDir::MarkDirty(bool bMarkDescendents)
{
	if (bMarkDescendents)
	{
		ForEachDescendent([](FScanDir& Descendent) { Descendent.SetComplete(false); });
	}
	FScanDir* Current = this;
	while (Current)
	{
		Current->SetComplete(false);
		Current = Current->Parent;
	}
}

void FScanDir::Shrink()
{
	ForEachSubDir([](FScanDir& SubDir) {SubDir.Shrink(); });
	SubDirs.Shrink();
	AlreadyScannedFiles.Shrink();
}


void FScanDir::SetComplete(bool bInIsComplete)
{
	if (!MountDir || bIsComplete == bInIsComplete)
	{
		return;
	}

	bIsComplete = bInIsComplete;
	if (bIsComplete)
	{
		MountDir->GetDiscovery().NumDirectoriesToScan.Decrement();
		// Upon completion, subdirs that do not need to be maintained are deleted, which is done by removing them from the parent.
		// ScanDirs need to be maintained if they are the root, or have persistent settings, or have child ScanDirs that need to be maintained,
		// or the parent scan has not been done yet.
		if ((Parent != nullptr && Parent->bHasScanned) && !HasPersistentSettings() && SubDirs.IsEmpty())
		{
			Parent->RemoveSubDir(GetRelPath());
			// *this is Shutdown (e.g. Parent is now null) and it may also have been deallocated
			return;
		}
	}
	else
	{
		FAssetDataDiscovery& Discovery = MountDir->GetDiscovery();

		// SetComplete is called within the treelock but not the results lock.
		// For the two atomics bIsIdle and NumDirectoriesToScan, we have a contract that bIsIdle is never true whenever
		// NumDirectoriesToScan is non-zero; this is relied upon in GetAndTrimSearchResults. Therefore we need to
		// SetIsIdle(false) before incrementing NumDirectoriesToScan.
		Discovery.SetIsIdle(false);
		Discovery.NumDirectoriesToScan.Increment();
	}
}

bool FScanDir::HasPersistentSettings() const
{
	return DirectData.HasSetting();
}

FScanDir* FScanDir::FindSubDir(FStringView SubDirBaseName)
{
	int32 Index = FindLowerBoundSubDir(SubDirBaseName);
	if (Index == SubDirs.Num() || !FPathViews::Equals(SubDirs[Index]->GetRelPath(), SubDirBaseName))
	{
		return nullptr;
	}
	else
	{
		return SubDirs[Index].GetReference();
	}
}

const FScanDir* FScanDir::FindSubDir(FStringView SubDirBaseName) const
{
	return const_cast<FScanDir*>(this)->FindSubDir(SubDirBaseName);
}

FScanDir& FScanDir::FindOrAddSubDir(FStringView SubDirBaseName)
{
	// FindOrAddSubDir is only allowed to be called on valid FScanDirs, which we rely on since we need a non-null MountDir which valid ScanDirs have
	check(MountDir != nullptr);

	int32 Index = FindLowerBoundSubDir(SubDirBaseName);
	if (Index == SubDirs.Num() || !FPathViews::Equals(SubDirs[Index]->GetRelPath(), SubDirBaseName))
	{
		return *SubDirs.EmplaceAt_GetRef(Index, new FScanDir(*MountDir, this, SubDirBaseName));
	}
	else
	{
		return *SubDirs[Index];
	}
}

void FScanDir::RemoveSubDir(FStringView SubDirBaseName)
{
	int32 Index = FindLowerBoundSubDir(SubDirBaseName);
	if (Index < SubDirs.Num() && FPathViews::Equals(SubDirs[Index]->GetRelPath(), SubDirBaseName))
	{
		// Destruction contract for FScanDir requires that the parent calls Shutdown before dropping the reference
		SubDirs[Index]->Shutdown();
		SubDirs.RemoveAt(Index);
	}
}

int32 FScanDir::FindLowerBoundSubDir(FStringView SubDirBaseName)
{
	return Algo::LowerBound(SubDirs, SubDirBaseName,
		[](const TRefCountPtr<FScanDir>& SubDir, FStringView BaseName)
		{
			return FPathViews::Less(SubDir->GetRelPath(), BaseName);
		}
	);
}

template <typename CallbackType>
void FScanDir::ForEachSubDir(const CallbackType& Callback)
{
	for (TRefCountPtr<FScanDir>& Ptr : SubDirs)
	{
		Callback(*Ptr);
	}
}

/** Depth-first-search traversal of all descedent subdirs under this (not including this). Callback is called on parents before children. */
template <typename CallbackType>
void FScanDir::ForEachDescendent(const CallbackType& Callback)
{
	TArray<TPair<FScanDir*, int32>, TInlineAllocator<10>> Stack; // 10 chosen arbitrarily as a depth that is greater than most of our content root directory tree depths
	Stack.Add(TPair<FScanDir*, int32>(this, 0));
	while (Stack.Num())
	{
		TPair<FScanDir*, int32>& Top = Stack.Last();
		FScanDir* ParentOnStack = Top.Get<0>();
		int32& NextIndex = Top.Get<1>();
		if (NextIndex == ParentOnStack->SubDirs.Num())
		{
			Stack.SetNum(Stack.Num() - 1, EAllowShrinking::No);
			continue;
		}
		FScanDir* Child = ParentOnStack->SubDirs[NextIndex++];
		Callback(*Child);
		Stack.Add(TPair<FScanDir*, int32>(Child, 0));
	}
}

FMountDir::FMountDir(FAssetDataDiscovery& InDiscovery, FStringView InLocalAbsPath, FStringView InLongPackageName)
	: LocalAbsPath(InLocalAbsPath)
	, LongPackageName(InLongPackageName)
	, Discovery(InDiscovery)
{
	Root = new FScanDir(*this, nullptr, FStringView());
	UpdateDenyList();
}

FMountDir::~FMountDir()
{
	// ScanDir's destruction contract requires that the parent calls Shutdown on it before dropping the reference
	Root->Shutdown();
	Root.SafeRelease();
}

FStringView FMountDir::GetLocalAbsPath() const
{
	return LocalAbsPath;
}

FStringView FMountDir::GetLongPackageName() const
{
	return LongPackageName;
}

FAssetDataDiscovery& FMountDir::GetDiscovery() const
{
	return Discovery;
}

FScanDir* FMountDir::GetControllingDir(FStringView InLocalAbsPath, bool bIsDirectory, FScanDir::FInherited& OutData,
	FString& OutRelPath)
{
	FStringView RemainingPath;
	if (!FPathViews::TryMakeChildPathRelativeTo(InLocalAbsPath, GetLocalAbsPath(), RemainingPath))
	{
		return nullptr;
	}
	return Root->GetControllingDir(RemainingPath, bIsDirectory, FScanDir::FInherited() /* ParentData */,
		OutData, OutRelPath);
}

SIZE_T FMountDir::GetAllocatedSize() const
{
	SIZE_T Result = sizeof(*Root);
	Result += Root->GetAllocatedSize();
	Result += ChildMountPaths.GetAllocatedSize();
	for (const FString& Value : ChildMountPaths)
	{
		Result += Value.GetAllocatedSize();
	}
	Result += LongPackageName.GetAllocatedSize();
	Result += RelPathsDenyList.GetAllocatedSize();
	for (const FString& Value : RelPathsDenyList)
	{
		Result += Value.GetAllocatedSize();
	}
	return Result;
}

void FMountDir::Shrink()
{
	Root->Shrink();
	ChildMountPaths.Shrink();
	RelPathsDenyList.Shrink();
}

bool FMountDir::IsComplete() const
{
	return Root->IsComplete();
}

void FMountDir::GetMonitorData(FStringView InLocalAbsPath, FScanDir::FInherited& OutData) const
{
	FStringView QueryRelPath;
	if (!ensure(FPathViews::TryMakeChildPathRelativeTo(InLocalAbsPath, GetLocalAbsPath(), QueryRelPath)))
	{
		OutData = FScanDir::FInherited();
		return;
	}

	return Root->GetMonitorData(QueryRelPath, FScanDir::FInherited() /* ParentData */, OutData);
}

bool FMountDir::IsMonitored(FStringView InLocalAbsPath) const
{
	FScanDir::FInherited MonitorData;
	GetMonitorData(InLocalAbsPath, MonitorData);
	return MonitorData.IsMonitored();
}

void FMountDir::TrySetDirectoryProperties(FPathExistence& QueryPath, const FSetPathProperties& InProperties,
	FScanDirAndParentData* OutControllingDir, FStringView* OutControllingDirRelPath, bool* bOutMadeChanges)
{
	if (OutControllingDir)
	{
		OutControllingDir->ScanDir.SafeRelease();
		OutControllingDir->ParentData = FScanDir::FInherited();
	}
	if (OutControllingDirRelPath)
	{
		*OutControllingDirRelPath = FStringView();
	}
	if (bOutMadeChanges)
	{
		*bOutMadeChanges = false;
	}

	FStringView RelPath;
	if (!ensure(FPathViews::TryMakeChildPathRelativeTo(QueryPath.GetLocalAbsPath(), GetLocalAbsPath(), RelPath)))
	{
		return;
	}
	if (InProperties.IgnoreDenyList.IsSet())
	{
		if (!ensure(!IsChildMountPath(RelPath)))
		{
			// Setting IgnoreDenyList on a child path would break behavior because we use MatchesDenyList to indicate that 
			// the scandir is a child path, and setting it to IgnoreDenyLists will defeat that setting.
			// This should never be called, because setting IgnoreDenyList is only called external to FAssetDataDiscovery, and 
			// FAssetDataDiscovery would call it on the child mount dir instead of this parent mountdir
			FSetPathProperties NewProperties(InProperties);
			NewProperties.IgnoreDenyList.Reset();
			return TrySetDirectoryProperties(QueryPath, NewProperties, OutControllingDir,
				OutControllingDirRelPath, bOutMadeChanges);
		}
	}
	FScanDirAndParentData PlaceHolderOutControllingDir;
	FStringView PlaceholderOutControllingDirRelPath;
	bool bPlaceholderOutMadeChanges = false;
	if (!OutControllingDir)
	{
		OutControllingDir = &PlaceHolderOutControllingDir;
	}
	if (!OutControllingDirRelPath)
	{
		OutControllingDirRelPath = &PlaceholderOutControllingDirRelPath;
	}
	if (!bOutMadeChanges)
	{
		bOutMadeChanges = &bPlaceholderOutMadeChanges;
	}
	FScanDir::FInherited ParentData;
	Root->TrySetDirectoryProperties(QueryPath, RelPath, ParentData, InProperties, *OutControllingDir,
		*OutControllingDirRelPath, *bOutMadeChanges);
}

void FMountDir::UpdateDenyList()
{
	TSet<FString> RemovedDenyLists;
	for (const FString& Old : RelPathsDenyList)
	{
		RemovedDenyLists.Add(Old);
	}

	RelPathsDenyList.Empty(Discovery.MountRelativePathsDenyList.Num());
	for (const FString& DenyListEntry : Discovery.LongPackageNamesDenyList)
	{
		FStringView MountRelPath;
		if (FPathViews::TryMakeChildPathRelativeTo(DenyListEntry, LongPackageName, MountRelPath))
		{
			// Note that an empty RelPath means we deny the entire mountpoint
			RelPathsDenyList.Emplace(MountRelPath);
		}
	}
	for (const FString& MountRelPath : Discovery.MountRelativePathsDenyList)
	{
		RelPathsDenyList.Emplace(MountRelPath);
	}
	for (const FString& ChildPath : ChildMountPaths)
	{
		RelPathsDenyList.Emplace(ChildPath);
	}

	TSet<FString> AddedDenyListPaths;
	for (const FString& New : RelPathsDenyList)
	{
		if (!RemovedDenyLists.Remove(New))
		{
			AddedDenyListPaths.Add(New);
		}
	}

	TStringBuilder<256> AbsPathDenyList;
	IFileManager& FileManager = IFileManager::Get();
	FSetPathProperties ChangeDenyList;
	FScanDir::FInherited ParentData;
	ChangeDenyList.MatchesDenyList = true;
	for (const FString& RelPath : AddedDenyListPaths)
	{
		AbsPathDenyList.Reset();
		AbsPathDenyList << LocalAbsPath;
		FPathViews::AppendPath(AbsPathDenyList, RelPath);
		if (FileManager.DirectoryExists(AbsPathDenyList.ToString()))
		{
			FScanDirAndParentData UnusedControllingDir;
			FStringView UnusedControllingDirRelPath;
			bool bUnusedMadeChanges = false;

			FPathExistence QueryPath(AbsPathDenyList);
			QueryPath.SetConfirmedExists(true);

			Root->TrySetDirectoryProperties(QueryPath, RelPath, ParentData, ChangeDenyList,
				UnusedControllingDir, UnusedControllingDirRelPath, bUnusedMadeChanges);
		}
	}
	ChangeDenyList.MatchesDenyList = false;
	for (const FString& RelPath : RemovedDenyLists)
	{
		FScanDirAndParentData UnusedControllingDir;
		FStringView UnusedControllingDirRelPath;
		bool bUnusedMadeChanges = false;

		AbsPathDenyList.Reset();
		AbsPathDenyList << LocalAbsPath;
		FPathViews::AppendPath(AbsPathDenyList, RelPath);

		// We don't need to check for existence on QueryPath when setting the removal property, because the scandir
		// already exists
		FPathExistence QueryPath(AbsPathDenyList);

		Root->TrySetDirectoryProperties(QueryPath, RelPath, ParentData, ChangeDenyList,
			UnusedControllingDir, UnusedControllingDirRelPath, bUnusedMadeChanges);
	}
}

void FMountDir::Update(TArray<FScanDirAndParentData>& OutScanRequests)
{
	FScanDir::FInherited ParentData = FScanDir::FInherited();
	Root->Update(OutScanRequests, ParentData);
}

FScanDir* FMountDir::GetFirstIncompleteScanDir()
{
	return Root->GetFirstIncompleteScanDir();
}

void FMountDir::SetHasStartedScanning()
{
	bHasStartedScanning = true;
}

void FMountDir::AddChildMount(FMountDir* ChildMount)
{
	if (!ChildMount)
	{
		return;
	}
	FStringView RelPath;
	if (!FPathViews::TryMakeChildPathRelativeTo(ChildMount->GetLocalAbsPath(), LocalAbsPath, RelPath))
	{
		return;
	}
	AddChildMountPath(RelPath);
	if (bHasStartedScanning)
	{
		UE_LOGF(LogAssetRegistry, Warning,
			"AssetDataGatherer directory %.*ls has already started scanning when a new mountpoint was added under it at %.*ls. "
			"Assets in the new mount point may exist twice in the AssetRegistry under two different package names.",
			LocalAbsPath.Len(), *LocalAbsPath, ChildMount->LocalAbsPath.Len(), *ChildMount->LocalAbsPath);
	}
	UpdateDenyList();
	MarkDirty(RelPath);
}

void FMountDir::RemoveChildMount(FMountDir* ChildMount)
{
	if (!ChildMount)
	{
		return;
	}
	FStringView RelPath;
	if (!FPathViews::TryMakeChildPathRelativeTo(ChildMount->GetLocalAbsPath(), LocalAbsPath, RelPath))
	{
		return;
	}
	if (!RemoveChildMountPath(RelPath))
	{
		return;
	}
	if (ChildMount->bHasStartedScanning)
	{
		UE_LOGF(LogAssetRegistry, Warning,
			"AssetDataGatherer directory %.*ls has already started scanning when it was removed and merged into its parent mount at %.*ls. "
			"Assets in the new mount point may exist twice in the AssetRegistry under two different package names.",
			ChildMount->LocalAbsPath.Len(), *ChildMount->LocalAbsPath, LocalAbsPath.Len(), *LocalAbsPath);
	}
	UpdateDenyList();
	MarkDirty(RelPath);
}

void FMountDir::OnDestroyClearChildMounts()
{
	ChildMountPaths.Empty();
}

void FMountDir::SetParentMount(FMountDir* Parent)
{
	ParentMount = Parent;
}

FMountDir* FMountDir::GetParentMount() const
{
	return ParentMount;
}

TArray<FMountDir*> FMountDir::GetChildMounts() const
{
	// Called within Discovery's TreeLock
	TArray<FMountDir*> Result;
	for (const FString& ChildPath : ChildMountPaths)
	{
		TStringBuilder<256> ChildAbsPath;
		ChildAbsPath << LocalAbsPath;
		FPathViews::AppendPath(ChildAbsPath, ChildPath);
		FMountDir* ChildMount = Discovery.FindMountPoint(ChildAbsPath);
		if (ensure(ChildMount)) // This PathData information should have been removed with RemoveChildMount when the child MountDir was removed from the Discovery
		{
			Result.Add(ChildMount);
		}
	}
	return Result;
}

void FMountDir::MarkDirty(FStringView MountRelPath)
{
	FScanDir::FInherited UnusedMonitorData;
	FString ControlRelPath;
	FScanDir* ScanDir = Root->GetControllingDir(MountRelPath, true /* bIsDirectory */, FScanDir::FInherited() /* ParentData */,
		UnusedMonitorData, ControlRelPath);
	if (ScanDir)
	{
		// If a ScanDir exists for the directory that is being marked dirty, mark all of its descendants dirty as well.
		// If the control dir is a parent directory of the requested path, just mark it and its parents dirty
		// Mark all parents dirty in either case
		bool bDirtyAllDescendents = ControlRelPath.IsEmpty();
		ScanDir->MarkDirty(bDirtyAllDescendents);
	}
}

void FMountDir::AddChildMountPath(FStringView MountRelPath)
{
	FString* ExistingPath = ChildMountPaths.FindByPredicate([&MountRelPath](const FString& ChildPath) { return FPathViews::Equals(ChildPath, MountRelPath); });
	if (!ExistingPath)
	{
		ChildMountPaths.Emplace(MountRelPath);
	}
}

bool FMountDir::RemoveChildMountPath(FStringView MountRelPath)
{
	return ChildMountPaths.RemoveAllSwap([MountRelPath](const FString& ChildPath) { return FPathViews::Equals(ChildPath, MountRelPath);  }) != 0;
}

bool FMountDir::IsChildMountPath(FStringView MountRelPath) const
{
	for (const FString& ChildPath : ChildMountPaths)
	{
		if (FPathViews::IsParentPathOf(ChildPath, MountRelPath))
		{
			return true;
		}
	}
	return false;
}


FAssetDataDiscovery::FAssetDataDiscovery()
	: LongPackageNamesDenyList()
	, MountRelativePathsDenyList()
	, Thread(nullptr)
	, bIsIdle(false)
	, IsStopped(0)
	, IsPaused(0)
	, NumDirectoriesToScan(0)
{
	GGatherSettings.Initialize();
	bAsyncEnabled = GGatherSettings.IsAsyncEnabled();

	if (FConfigFile* EngineIni = GConfig->FindConfigFile(GEngineIni))
	{
		EngineIni->GetArray(TEXT("AssetRegistry"), TEXT("BlacklistPackagePathScanFilters"), LongPackageNamesDenyList);
		EngineIni->GetArray(TEXT("AssetRegistry"), TEXT("BlacklistContentSubPathScanFilters"), MountRelativePathsDenyList);
	}

	PriorityDataUpdated->Trigger();
}

FAssetDataDiscovery::~FAssetDataDiscovery()
{
	EnsureCompletion();
	// Remove pointers to other MountDirs before we delete any of them
	for (TUniquePtr<FMountDir>& MountDir : MountDirs)
	{
		MountDir->SetParentMount(nullptr);
		MountDir->OnDestroyClearChildMounts();
	}
	MountDirs.Empty();
}

void FAssetDataDiscovery::StartAsync()
{
	if (bAsyncEnabled && !Thread)
	{
		Thread = FRunnableThread::Create(this, TEXT("FAssetDataDiscovery"), 0, TPri_BelowNormal);
		checkf(Thread, TEXT("Failed to create asset data discovery thread"));
	}
}

bool FAssetDataDiscovery::IsSynchronous() const
{
	return Thread == nullptr;
}

bool FAssetDataDiscovery::Init()
{
	return true;
}

uint32 FAssetDataDiscovery::Run()
{
	LLM_SCOPE(ELLMTag::AssetRegistry);
	constexpr float IdleSleepTime = 0.1f;

	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(TreeLock);
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
	while (!IsStopped)
	{
		bool bTickOwner = false;
		while (!IsStopped && !bIsIdle && !IsPaused)
		{
			if (!bTickOwner)
			{
				if (TickOwner.TryTakeOwnership(TreeLock))
				{
					bTickOwner = true;
				}
			}
			if (bTickOwner)
			{
				TickInternal(true /* bTickAll */);
			}
			else
			{
				FPlatformProcess::Sleep(IdleSleepTime);
			}
		}
		if (bTickOwner)
		{
			TickOwner.ReleaseOwnershipChecked(TreeLock);
			bTickOwner = false;
		}

		while (!IsStopped && (IsPaused || bIsIdle))
		{
			// No work to do. Sleep for a little and try again later.
			// TODO: Need IsPaused to be a condition variable so we avoid sleeping while waiting for it and then taking a long time to wake after it is unset.
			FPlatformProcess::Sleep(IdleSleepTime);
		} 
	}
	return 0;
}

FAssetDataDiscovery::FScopedPause::FScopedPause(const FAssetDataDiscovery& InOwner)
	:Owner(InOwner)
{
	if (!Owner.IsSynchronous())
	{
		Owner.IsPaused++;
	}
	while (!Owner.TickOwner.TryTakeOwnership(Owner.TreeLock))
	{
		check(!Owner.TickOwner.IsOwnedByCurrentThread());
		constexpr float BlockingSleepTime = 0.001f;
		FPlatformProcess::Sleep(BlockingSleepTime);
	}
}

FAssetDataDiscovery::FScopedPause::~FScopedPause()
{
	Owner.TickOwner.ReleaseOwnershipChecked(Owner.TreeLock);
	if (!Owner.IsSynchronous())
	{
		check(Owner.IsPaused > 0);
		Owner.IsPaused--;
	}
}

void FAssetDataDiscovery::Stop()
{
	IsStopped++;
}

void FAssetDataDiscovery::Exit()
{
}

void FAssetDataDiscovery::EnsureCompletion()
{
	Stop();

	if (Thread != nullptr)
	{
		Thread->WaitForCompletion();
		delete Thread;
		Thread = nullptr;
	}
}

void FAssetDataDiscovery::OnInitialSearchCompleted()
{
	Cache.SaveCache();
	Cache.Shutdown();
}

void FAssetDataDiscovery::OnAdditionalMountSearchCompleted()
{
	// After the initial search completed, OnInitialSearchCompleted cleared 
	// out the data for the cache so we no longer have enough data to save it.
}

void FAssetDataDiscovery::TickInternal(bool bTickAll)
{
	LLM_SCOPE(ELLMTag::AssetRegistry);
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
	check(TickOwner.IsOwnedByCurrentThread());

	if (!Cache.IsInitialized())
	{
		Cache.LoadAndUpdateCache();
	}

	TArray<FScanDirAndParentData> ScanRequests;
	TStringBuilder<128> DirMountRelPath;

	int32 LocalNumCachedDirectories = 0;
	int32 DirToScanDatasNum = 0;
	bool bUpdatedPriorityData = false;
	double TickStartTime = FPlatformTime::Seconds();
	ON_SCOPE_EXIT
	{
		if (TickStartTime >= 0.)
		{
			CurrentDiscoveryTime += FPlatformTime::Seconds() - TickStartTime;
			TickStartTime = -1.;
		}
	};
	for (;;)
	{
		{
			FGathererScopeLock TreeScopeLock(&TreeLock);

			// Process scanned directories from the previous iteration of the for(;;) loop.
			int32 NumScanned = 0;
			if (DirToScanDatasNum > 0)
			{
				for (FDirToScanData& Data : TArrayView<FDirToScanData>(DirToScanDatas.GetData(), DirToScanDatasNum))
				{
					if (Data.bScanned)
					{
						++NumScanned;
						TArrayView<FDiscoveredPathData> LocalSubDirs(Data.IteratedSubDirs.GetData(), Data.NumIteratedDirs);
						TArrayView<FDiscoveredPathData> LocalDiscoveredFiles(Data.IteratedFiles.GetData(), Data.NumIteratedFiles);
						if (!Data.ScanDir->IsValid())
						{
							// The ScanDir has been shutdown, and it is only still allocated to prevent us from crashing. Drop our reference and allow it to delete.
						}
						else if (Data.ScanDir->IsScanInFlightInvalidated())
						{
							// Some setting has been applied to the ScanDir that requires a new scan
							// Consume the invalidated flag and ignore the results of our scan
							Data.ScanDir->SetScanInFlightInvalidated(false);
							UE_LOGF(LogAssetRegistry, Display, "ScanDir %ls has been invalidated (Current State: %ls). Will not scan.", *Data.ScanDir->GetLocalAbsPath(), (Data.ScanDir->IsValid() ? TEXT("valid") : TEXT("invalid")));
						}
						else
						{
							Data.ScanDir->SetScanResults(Data.DirLocalAbsPath, Data.ParentData, LocalSubDirs, LocalDiscoveredFiles);
							if (!LocalSubDirs.IsEmpty() || !LocalDiscoveredFiles.IsEmpty())
							{
								AddDiscovered(Data.DirLocalAbsPath, Data.DirLongPackageName, LocalSubDirs, LocalDiscoveredFiles);
							}
						}
						Data.ScanDir->SetScanInFlight(false);
						Data.ScanDir.SafeRelease();
					}
				}
				// Rather than collecting LocalNumUncachedDirectories from inside the parallel for that scans them,
				// just calculate it from the number of Data.bScanned and the number of LocalNumCachedDirectories
				int32 LocalNumUncachedDirectories = NumScanned - LocalNumCachedDirectories;
				DirToScanDatasNum = 0;
				{
					FGathererScopeLock ResultsScopeLock(&ResultsLock);
					NumCachedDirectories += LocalNumCachedDirectories;
					NumUncachedDirectories += LocalNumUncachedDirectories;
				}
				LocalNumCachedDirectories = 0;
				LocalNumUncachedDirectories = 0;
			}

			// Look for new dirs to scan, break out of the for (;;) loop if we don't find any
			auto AddScanRequest = [this, &DirToScanDatasNum, &DirMountRelPath](FScanDirAndParentData&& ScanRequest)
			{
				if (DirToScanDatas.Num() <= DirToScanDatasNum)
				{
					// We increment DirToScanDatasNum one at a time so DirToScanDatas.Num() should always be >= to it.
					check(DirToScanDatas.Num() == DirToScanDatasNum);
					DirToScanDatas.Emplace();
				}
				FScanDir* ScanDir = ScanRequest.ScanDir.GetReference();
				FDirToScanData& ScanData = DirToScanDatas[DirToScanDatasNum++];
				ScanData.Reset();
				DirMountRelPath.Reset();
				ScanDir->SetScanInFlight(true);
				FMountDir* MountDir = ScanDir->GetMountDir();
				check(MountDir);
				ScanDir->AppendMountRelPath(DirMountRelPath);
				ScanData.DirLocalAbsPath << MountDir->GetLocalAbsPath();
				FPathViews::AppendPath(ScanData.DirLocalAbsPath, DirMountRelPath);
				ScanData.DirLongPackageName << MountDir->GetLongPackageName();
				FPathViews::AppendPath(ScanData.DirLongPackageName, DirMountRelPath);
				// The DirLocalAbsPath and DirLongPackageName need to be normalized. They are already mostly
				// normalized, but might have a redundant terminating separator
				while (FPathViews::HasRedundantTerminatingSeparator(ScanData.DirLocalAbsPath))
				{
					ScanData.DirLocalAbsPath.RemoveSuffix(1);
				}
				while (FPathViews::HasRedundantTerminatingSeparator(ScanData.DirLongPackageName))
				{
					ScanData.DirLongPackageName.RemoveSuffix(1);
				}
				ScanData.ScanDir = MoveTemp(ScanRequest.ScanDir);
				ScanData.ParentData = MoveTemp(ScanRequest.ParentData);
			};

			bool bExitAfterPriorityUpdate = !bTickAll && bUpdatedPriorityData;
			if (!PriorityScanDirs.IsEmpty())
			{
				ScanRequests.Reset();
				for (int32 PriorityIndex = 0; PriorityIndex < PriorityScanDirs.Num(); ++PriorityIndex)
				{
					FPriorityScanDirData& PriorityData = PriorityScanDirs[PriorityIndex];
					int32 OriginalScanRequestsNum = ScanRequests.Num();
					if (PriorityData.ScanDir->IsValid() && !PriorityData.ScanDir->IsComplete())
					{
						PriorityData.ScanDir->Update(ScanRequests, PriorityData.ParentData);
					}
					if (PriorityData.ScanDir->IsComplete())
					{
						// Update should not add ScanRequests if it was already or transitioned to complete
						check(ScanRequests.Num() == OriginalScanRequestsNum); 
						if (PriorityData.bReleaseWhenComplete)
						{
							PriorityData.bReleaseWhenComplete = false;
							check(PriorityData.RequestCount > 0);
							PriorityData.RequestCount--;
							if (PriorityData.RequestCount == 0)
							{
								PriorityScanDirs.RemoveAtSwap(PriorityIndex);
								--PriorityIndex; // Counteract the ++PriorityIndex in the for loop iteration
							}
							continue;
						}
					}
				}

				if (!bExitAfterPriorityUpdate)
				{
					// A ScanDir and its parent can both be in PriorityScanDirs, and in that case we can get duplicates
					// in the list of ScanRequests. Ensure uniqueness now.
					Algo::Sort(ScanRequests,
						[](const FScanDirAndParentData & A, const FScanDirAndParentData& B)
						{
							return A.ScanDir.GetReference() < B.ScanDir.GetReference();
						});
					ScanRequests.SetNum(Algo::Unique(ScanRequests,
						[](const FScanDirAndParentData& A, const FScanDirAndParentData& B)
						{
							return A.ScanDir.GetReference() == B.ScanDir.GetReference();
						}));

					for (FScanDirAndParentData& ScanRequest : ScanRequests)
					{
						AddScanRequest(MoveTemp(ScanRequest));
					}
				}
				ScanRequests.Reset();
			}
			if (bUpdatedPriorityData)
			{
				PriorityDataUpdated->Trigger();
			}
			if (bExitAfterPriorityUpdate)
			{
				return; // exit to check the done condition
			}
			bPriorityDirty = false;
			bUpdatedPriorityData = DirToScanDatasNum > 0;

			if (bTickAll && DirToScanDatasNum == 0)
			{
				ScanRequests.Reset();
				UpdateAll(ScanRequests);
				for (FScanDirAndParentData& ScanRequest : ScanRequests)
				{
					AddScanRequest(MoveTemp(ScanRequest));
				}
				ScanRequests.Reset();
			}

			if (DirToScanDatasNum == 0)
			{
				if (!bTickAll)
				{
					return;
				}

				int32 LocalNumDirectoriesToScan = NumDirectoriesToScan.GetValue();
				if (LocalNumDirectoriesToScan != 0)
				{
					// We have some directories left to scan, but we were unable to find any of them. Print diagnostics.
					FScanDir* Incomplete = nullptr;
					for (TUniquePtr<FMountDir>& MountDir : MountDirs)
					{
						Incomplete = MountDir->GetFirstIncompleteScanDir();
						if (Incomplete)
						{
							break;
						}
					}
					UE_LOGF(LogAssetRegistry, Warning, "FAssetDataDiscovery::SetIsIdle(true) called when NumDirectoriesToScan == %d.\n" "First incomplete scandir: %ls", LocalNumDirectoriesToScan, Incomplete ? *Incomplete->GetLocalAbsPath() : TEXT("<NoneFound>"));
				}
				SetIsIdle(true, TickStartTime);
				return;
			}
			SetIsIdle(false);
		}

		// Outside of the TreeLock critical section, scan the directories

		// Process on a single thread any of the DirToScans that we find in the cache.
		for (int32 DirToScanDatasIndex = 0; DirToScanDatasIndex < DirToScanDatasNum; ++DirToScanDatasIndex)
		{
			FDirToScanData& Data = DirToScanDatas[DirToScanDatasIndex];
			FCachedDirScanDir* PathData = Cache.FindDir(Data.DirLocalAbsPath);
			if (!PathData || !PathData->bCacheValid)
			{
				continue;
			}
			Data.bScanned = true;
			++LocalNumCachedDirectories;

			for (const FString& RelPath : PathData->SubDirRelPaths)
			{
				// Don't enter directories that contain invalid packagepath characters (including '.';
				// extensions are not valid in content directories because '.' is not valid in a packagepath)
				if (!FPackageName::DoesPackageNameContainInvalidCharacters(RelPath))
				{
					int32 DirLongPackageRootNameLen = Data.DirLongPackageName.Len();
					int32 DirLocalAbsPathLen = Data.DirLocalAbsPath.Len();
					ON_SCOPE_EXIT
					{
						Data.DirLongPackageName.RemoveSuffix(Data.DirLongPackageName.Len() - DirLongPackageRootNameLen);
						Data.DirLocalAbsPath.RemoveSuffix(Data.DirLocalAbsPath.Len() - DirLocalAbsPathLen);
					};

					FPathViews::AppendPath(Data.DirLongPackageName, RelPath);
					FPathViews::AppendPath(Data.DirLocalAbsPath, RelPath);
					if (Data.IteratedSubDirs.Num() < Data.NumIteratedDirs + 1)
					{
						check(Data.IteratedSubDirs.Num() == Data.NumIteratedDirs);
						Data.IteratedSubDirs.Emplace();
					}
					Data.IteratedSubDirs[Data.NumIteratedDirs++].Assign(Data.DirLocalAbsPath, Data.DirLongPackageName, RelPath,
						EGatherableFileType::Directory);
				}
			}
			for (const FCachedDirScanFile& FileData : PathData->Files)
			{
				FStringView RelPath(FileData.RelPath);

				EGatherableFileType FileType = GetFileType(RelPath);
				// Don't record files that contain invalid packagepath characters (not counting their extension)
				// or that do not end with a recognized extension
				if (FileType != EGatherableFileType::Invalid)
				{
					FStringView BaseName = FPathViews::GetBaseFilename(RelPath);
					if (!DoesPathContainInvalidCharacters(FileType, BaseName))
					{
						int32 DirLongPackageRootNameLen = Data.DirLongPackageName.Len();
						int32 DirLocalAbsPathLen = Data.DirLocalAbsPath.Len();
						ON_SCOPE_EXIT
						{
							Data.DirLongPackageName.RemoveSuffix(Data.DirLongPackageName.Len() - DirLongPackageRootNameLen);
							Data.DirLocalAbsPath.RemoveSuffix(Data.DirLocalAbsPath.Len() - DirLocalAbsPathLen);
						};

						if (Data.IteratedFiles.Num() < Data.NumIteratedFiles + 1)
						{
							check(Data.IteratedFiles.Num() == Data.NumIteratedFiles);
							Data.IteratedFiles.Emplace();
						}
						FPathViews::AppendPath(Data.DirLongPackageName, BaseName);
						FPathViews::AppendPath(Data.DirLocalAbsPath, RelPath);
						Data.IteratedFiles[Data.NumIteratedFiles++].Assign(Data.DirLocalAbsPath, Data.DirLongPackageName, RelPath,
							FileData.ModificationTime, !!FileData.bIsReadOnly, FileType);
					}
				}
			}
		}

		// If we found any cached directories, keep looking in their children before we start querying
		// the disk for uncached
		if (LocalNumCachedDirectories > 0 && !bUpdatedPriorityData)
		{
			continue;
		}

		// Otherwise look on disk in parallel for all of the DirToScans
		int32 NumThreads = FMath::Max(FTaskGraphInterface::Get().GetNumWorkerThreads(), 1);
		if (FGatherSettings::GARDiscoverThreads > 0)
		{
			NumThreads = FMath::Min(NumThreads, FGatherSettings::GARDiscoverThreads);
		}
		int32 DirToScanBuffersNum = FMath::Min(NumThreads, DirToScanDatasNum);
		if (DirToScanBuffers.Num() < DirToScanBuffersNum)
		{
			DirToScanBuffers.SetNum(DirToScanBuffersNum);
		}
		TArrayView<FDirToScanBuffer> LocalScanBuffers(DirToScanBuffers.GetData(), DirToScanBuffersNum);
		for (FDirToScanBuffer& ScanBuffer : LocalScanBuffers)
		{
			ScanBuffer.Reset();
		}

		ParallelForWithExistingTaskContext(LocalScanBuffers, DirToScanDatasNum, FGatherSettings::GARDiscoverMinBatchSize,
		[this](FDirToScanBuffer& ScanBuffer, int32 DirToScanDatasIndex)
		{
			FDirToScanData& Data = DirToScanDatas[DirToScanDatasIndex];
			if (Data.bScanned)
			{
				return;
			}
			if (ScanBuffer.bAbort)
			{
				return;
			}
			if (bPriorityDirty.load(std::memory_order_relaxed))
			{
				ScanBuffer.bAbort = true;
				return;
			}

			FCachedDirScanDir CacheDataToAdd;
			auto ProcessIterData = [this, &Data, &CacheDataToAdd]
			(const TCHAR* IterFilename, bool bIsDirectory, const FDateTime& ModificationTime, bool bIsReadOnly, FFileJournalFileHandle JournalHandle, bool bIsReparsePoint)
			{
				FStringView LocalAbsPath(IterFilename);
				FStringView RelPath;
				FString Buffer;
				if (!FPathViews::TryMakeChildPathRelativeTo(IterFilename, Data.DirLocalAbsPath, RelPath))
				{
					// Try again with the path converted to the absolute path format that we passed in; some
					// IFileManagers can send relative paths to the visitor even though the search path is absolute
					Buffer = FPaths::ConvertRelativePathToFull(FString(IterFilename));
					LocalAbsPath = Buffer;
					if (!FPathViews::TryMakeChildPathRelativeTo(Buffer, Data.DirLocalAbsPath, RelPath))
					{
						UE_LOGF(LogAssetRegistry, Warning,
							"IterateDirectory returned unexpected result %ls which is not a child of the requested path %ls.",
							IterFilename, Data.DirLocalAbsPath.ToString());
						return true;
					}
				}
				if (FPathViews::GetPathLeaf(RelPath).Len() != RelPath.Len())
				{
					UE_LOGF(LogAssetRegistry, Warning,
						"IterateDirectory returned unexpected result %ls which is not a direct child of the requested path %ls.",
						IterFilename, Data.DirLocalAbsPath.ToString());
					return true;
				}
				int32 DirLongPackageRootNameLen = Data.DirLongPackageName.Len();
				ON_SCOPE_EXIT
				{
					Data.DirLongPackageName.RemoveSuffix(Data.DirLongPackageName.Len() - DirLongPackageRootNameLen);
				};

				if (bIsDirectory)
				{
					if (Cache.IsWriteEnabled() != EFeatureEnabled::Never)
					{
						CacheDataToAdd.SubDirRelPaths.Add(FString(RelPath));
						Cache.QueueAdd(FString(LocalAbsPath), JournalHandle, bIsReparsePoint);
					}

					FPathViews::AppendPath(Data.DirLongPackageName, RelPath);
					// Don't enter directories that contain invalid packagepath characters (including '.';
					// extensions are not valid in content directories because '.' is not valid in a packagepath)
					if (!FPackageName::DoesPackageNameContainInvalidCharacters(RelPath))
					{
						if (Data.IteratedSubDirs.Num() < Data.NumIteratedDirs + 1)
						{
							check(Data.IteratedSubDirs.Num() == Data.NumIteratedDirs);
							Data.IteratedSubDirs.Emplace();
						}
						Data.IteratedSubDirs[Data.NumIteratedDirs++].Assign(LocalAbsPath, Data.DirLongPackageName, RelPath,
							EGatherableFileType::Directory);
					}
				}
				else
				{
					if (Cache.IsWriteEnabled() != EFeatureEnabled::Never)
					{
						FCachedDirScanFile& FileData = CacheDataToAdd.Files.Emplace_GetRef();
						FileData.RelPath = FString(RelPath);
						FileData.ModificationTime = ModificationTime;
						FileData.bIsReadOnly = bIsReadOnly;
					}
					EGatherableFileType FileType = GetFileType(RelPath);
					// Don't record files that contain invalid packagepath characters (not counting their extension)
					// or that do not end with a recognized extension
					if (FileType != EGatherableFileType::Invalid)
					{
						FStringView BaseName = FPathViews::GetBaseFilename(RelPath);
						if (!DoesPathContainInvalidCharacters(FileType, BaseName))
						{
							if (Data.IteratedFiles.Num() < Data.NumIteratedFiles + 1)
							{
								check(Data.IteratedFiles.Num() == Data.NumIteratedFiles);
								Data.IteratedFiles.Emplace();
							}
							FPathViews::AppendPath(Data.DirLongPackageName, BaseName);
							Data.IteratedFiles[Data.NumIteratedFiles++].Assign(LocalAbsPath, Data.DirLongPackageName, RelPath,
								ModificationTime, bIsReadOnly, FileType);
						}
					}
				}
				return true;
			};

			bool bIteratedDirectory = false;
			if (Cache.IsWriteEnabled() != EFeatureEnabled::Never)
			{
				// If we fail to iterate this directory, fall back to the old way. Meaning we will fail to cache this but still pick up on the assets in the directories
				if (FPlatformFileManager::Get().GetPlatformFile().FileJournalIterateDirectory(Data.DirLocalAbsPath.ToString(),
					[&ProcessIterData]
					(const TCHAR* IterFilename, const FFileJournalData& IterData)
					{
						return ProcessIterData(IterFilename, IterData.bIsDirectory, IterData.ModificationTime, IterData.bIsReadOnly, IterData.JournalHandle, IterData.bIsReparsePoint);
					}))
				{
					bIteratedDirectory = true;
					Cache.QueueAdd(FString(Data.DirLocalAbsPath), MoveTemp(CacheDataToAdd));
				}
				else
				{
					// Only run this once to capture more information on why we fail sometimes here
					static std::atomic<bool> bRunOnce = false;
					if (bRunOnce)
					{
						// If we failed once, run once more time but grab an error so we can send some telemetry on the issue
						FString OutError;
						if (!FPlatformFileManager::Get().GetPlatformFile().FileJournalIterateDirectory(Data.DirLocalAbsPath.ToString(),
							[]
							(const TCHAR* IterFilename, const FFileJournalData& IterData)
							{
								return true; // if it doesnt return true it acts as a failure for this function
							}, &OutError))
						{
							bRunOnce = true;

							UE_LOGF(LogAssetRegistry, Warning, "Failed to FileJournalIterateDirectory, failing to cache this directory due to:\n  %ls", *OutError);

							UE::Telemetry::AssetRegistry::FFileJournalErrorTelemetry Telemetry;
							Telemetry.Directory = Data.DirLocalAbsPath.ToString();
							Telemetry.ErrorString = OutError;
							FTelemetryRouter::Get().ProvideTelemetry(Telemetry);
						}
					}
				}
			}

			if (!bIteratedDirectory)
			{
				IFileManager::Get().IterateDirectoryStat(Data.DirLocalAbsPath.ToString(),
					[&ProcessIterData]
					(const TCHAR* IterFilename, const FFileStatData& IterData)
					{
						return ProcessIterData(IterFilename, IterData.bIsDirectory, IterData.ModificationTime, IterData.bIsReadOnly, FFileJournalFileHandle(), false /* bIsReparsePoint */);
					});
			}

			Data.bScanned = true;
		}, EParallelForFlags::BackgroundPriority);

		if (Cache.IsWriteEnabled() != EFeatureEnabled::Never)
		{
			Cache.QueueConsume();
		}
	}
}

void FAssetDataDiscovery::UpdateAll(TArray<FScanDirAndParentData>& OutScanRequests)
{
	CHECK_IS_LOCKED_CURRENT_THREAD(TreeLock);
	for (TUniquePtr<FMountDir>& MountDirOwner : MountDirs)
	{
		FMountDir* MountDir = MountDirOwner.Get();
		if (MountDir->IsComplete())
		{
			continue;
		}

		MountDir->Update(OutScanRequests);
	}
}

void FAssetDataDiscovery::SetIsIdle(bool bInIsIdle)
{
	double TickStartTime = -1.;
	SetIsIdle(bInIsIdle, TickStartTime);
}

void FAssetDataDiscovery::SetIsIdle(bool bInIsIdle, double& TickStartTime)
{
	CHECK_IS_LOCKED_CURRENT_THREAD(TreeLock);

	// Caller is responsible for holding TreeLock around this function; writes of SetIsIdle are done inside the TreeLock
	// If bIsIdle is true, caller holds TickOwner and TreeLock
	if (bIsIdle == bInIsIdle)
	{
		return;
	}
	FGathererScopeLock ResultsScopeLock(&ResultsLock);
	bIsIdle = bInIsIdle;
	if (!IsSynchronous())
	{
		if (bIsIdle)
		{
			if (TickStartTime >= 0.)
			{
				CurrentDiscoveryTime += FPlatformTime::Seconds() - TickStartTime;
				TickStartTime = -1.;
			}

			CumulativeDiscoveryTime += static_cast<float>(CurrentDiscoveryTime);
			CumulativeDiscoveredFiles += NumDiscoveredFiles;
			UE_LOGF(LogAssetRegistry, Verbose,
				"Discovery took %0.4f seconds to add %d files, Cumulative=%0.4f seconds to add %d.",
				CurrentDiscoveryTime, NumDiscoveredFiles, CumulativeDiscoveryTime, CumulativeDiscoveredFiles);
			CurrentDiscoveryTime = 0.;
		}
		else
		{
			NumDiscoveredFiles = 0;
		}
	}

	if (bIsIdle)
	{
		check(TickOwner.IsOwnedByCurrentThread());
		Shrink();
	}
}

void FAssetDataDiscovery::GetAndTrimSearchResults(bool& bOutIsComplete, TArray<FString>& OutDiscoveredPaths, FFilesToSearch& OutFilesToSearch, int32& OutNumPathsToSearch)
{
	FGathererScopeLock ResultsScopeLock(&ResultsLock);

	OutDiscoveredPaths.Append(MoveTemp(DiscoveredDirectories));
	DiscoveredDirectories.Reset();

	for (FDirectoryResult& DirectoryResult : DiscoveredFiles)
	{
		OutFilesToSearch.AddFiles(MoveTemp(DirectoryResult.Files));
	}
	DiscoveredFiles.Reset();
	for (FGatheredPathData& FileResult : DiscoveredSingleFiles)
	{
		// Single files are currently only added from the blocking function FAssetDataDiscovery::SetPropertiesAndWait,
		// so we add them at blocking priority.
		OutFilesToSearch.AddPriorityFile(MoveTemp(FileResult));
	}
	DiscoveredSingleFiles.Reset();

	OutNumPathsToSearch = NumDirectoriesToScan.GetValue();
	bOutIsComplete = bIsIdle;
	if (bIsIdle && OutNumPathsToSearch != 0)
	{
		UE_LOGF(LogAssetRegistry, Warning, "FAssetDataDiscovery::GetAndTrimSearchResults is returning bIsIdle=true while OutNumPathsToSearch=%d.",
			OutNumPathsToSearch);
	}
}

void FAssetDataDiscovery::GetDiagnostics(float& OutCumulativeDiscoveryTime, int32& OutNumCachedDirectories,
	int32& OutNumUncachedDirectories)
{
	FGathererScopeLock ResultsScopeLock(&ResultsLock);
	OutCumulativeDiscoveryTime = CumulativeDiscoveryTime;
	OutNumCachedDirectories = NumCachedDirectories;
	OutNumUncachedDirectories = NumUncachedDirectories;
}


void FAssetDataDiscovery::WaitForIdle(double EndTimeSeconds)
{
	if (bIsIdle)
	{
		return;
	}
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(TreeLock);
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);

	constexpr float IdleSleepTime = 0.1f;
	bool bTickOwner = false;
	while (!bIsIdle)
	{
		if (!bTickOwner)
		{
			bTickOwner = TickOwner.TryTakeOwnership(TreeLock);
		}
		if (bTickOwner)
		{
			TickInternal(true /* bTickAll */);
		}
		else
		{
			float SleepTime = IdleSleepTime;
			if (EndTimeSeconds > 0)
			{
				SleepTime = FMath::Min(SleepTime, static_cast<float>(EndTimeSeconds - FPlatformTime::Seconds()));
			}
			if (SleepTime > 0.f)
			{
				FPlatformProcess::Sleep(SleepTime);
			}
		}
		if (EndTimeSeconds > 0 && FPlatformTime::Seconds() > EndTimeSeconds)
		{
			break;
		}
	}
	if (bTickOwner)
	{
		TickOwner.ReleaseOwnershipChecked(TreeLock);
	}
}

bool FAssetDataDiscovery::IsIdle() const
{
	return bIsIdle;
}

FPathExistence::FPathExistence(FStringView InLocalAbsPath)
	:LocalAbsPath(InLocalAbsPath)
{
}

const FString& FPathExistence::GetLocalAbsPath() const
{
	return LocalAbsPath;
}

FStringView FPathExistence::GetLowestExistingPath()
{
	LoadExistenceData();
	switch (PathType)
	{
	case EType::MissingButDirExists:
		return FPathViews::GetPath(LocalAbsPath);
	case EType::MissingParentDir:
		return FStringView();
	default:
		return LocalAbsPath;
	}
}

FPathExistence::EType FPathExistence::GetType()
{
	LoadExistenceData();
	return PathType;
}

FDateTime FPathExistence::GetModificationTime()
{
	LoadExistenceData();
	return ModificationTime;
}

bool FPathExistence::IsReadOnly()
{
	LoadExistenceData();
	return bIsReadOnly;
}

bool TryFixSymlinkCapitalization(FString& LocalAbsPath, FString& CorrectedCapitalization)
{
	// try to find the last junction / symlink 
	FString Path = FString(FPathViews::GetPath(LocalAbsPath));
	while (!Path.IsEmpty() && !IFileManager::Get().IsSymlink(*Path))
	{
		Path = FString(FPathViews::GetPath(Path));
	}

	// We couldn't find a symlink on the path, we don't know why the capitalization is wrong.
	if (Path.IsEmpty())
	{
		return false;
	}

	FStringView LocalAbsPathAfterJunction(LocalAbsPath);
	LocalAbsPathAfterJunction = LocalAbsPathAfterJunction.RightChop(Path.Len());

	FStringView CorrectedCapitalizationAfterJunction(CorrectedCapitalization);
	CorrectedCapitalizationAfterJunction = CorrectedCapitalizationAfterJunction.RightChop(CorrectedCapitalizationAfterJunction.Len() - LocalAbsPathAfterJunction.Len());

	// we expect something like: [junction]/foo/bar.uasset <-> [fullpath]/Foo/BAR.uasset
	if (!LocalAbsPathAfterJunction.Equals(CorrectedCapitalizationAfterJunction, ESearchCase::IgnoreCase))
	{
		//unexpected but no need for a separate error
		return false;
	}

	int LocalAbsPathIt = LocalAbsPath.Len() - 1;
	int CorrectedCapitalizationIt = CorrectedCapitalization.Len() - 1;
	for (int i = 0; i < LocalAbsPathAfterJunction.Len(); ++i)
	{
		LocalAbsPath[LocalAbsPathIt--] = CorrectedCapitalization[CorrectedCapitalizationIt--];
	}

	return true;
}

void FPathExistence::LoadExistenceData()
{
	if (bHasExistenceData)
	{
		return;
	}
	FFileStatData StatData = IFileManager::Get().GetStatData(*LocalAbsPath);
	if (StatData.bIsValid)
	{
		FString CorrectedCapitalization = IFileManager::Get().GetFilenameOnDisk(*LocalAbsPath);
		if (LocalAbsPath == CorrectedCapitalization)
		{
			LocalAbsPath = MoveTemp(CorrectedCapitalization);
		}
		else // if the case insensitive comparison above failed, there might be junction / symlink in the path
		{
			if (!TryFixSymlinkCapitalization(LocalAbsPath, CorrectedCapitalization))
			{
				UE_LOGF(LogAssetRegistry, Warning,
					"FPathExistence failed to gather correct capitalization from disk for %ls, because GetFilenameOnDisk returned non-matching filename '%ls'.",
					*LocalAbsPath, *CorrectedCapitalization);
			}
		}

		ModificationTime = StatData.ModificationTime;
		bIsReadOnly = StatData.bIsReadOnly;
		PathType = StatData.bIsDirectory ? EType::Directory : EType::File;
	}
	else
	{
		FString ParentPath, BaseName, Extension;
		FPaths::Split(LocalAbsPath, ParentPath, BaseName, Extension);
		StatData = IFileManager::Get().GetStatData(*ParentPath);
		if (StatData.bIsValid && StatData.bIsDirectory)
		{
			FString CorrectedCapitalization = IFileManager::Get().GetFilenameOnDisk(*ParentPath);
			CorrectedCapitalization = FPaths::Combine(CorrectedCapitalization, BaseName) +
				(!Extension.IsEmpty() ? TEXT(".") : TEXT("")) + Extension;
			if (LocalAbsPath == CorrectedCapitalization)
			{
				LocalAbsPath = MoveTemp(CorrectedCapitalization);
			}
			else // if the case insensitive comparison above failed, there might be junction / symlink in the path
			{
				if (!TryFixSymlinkCapitalization(LocalAbsPath, CorrectedCapitalization))
				{
					UE_LOGF(LogAssetRegistry, Warning,
						"FPathExistence failed to gather correct capitalization from disk for %ls, because GetFilenameOnDisk returned non-matching filename '%ls'.",
						*LocalAbsPath, *CorrectedCapitalization);
				}
			}
			PathType = EType::MissingButDirExists;
		}
		else
		{
			PathType = EType::MissingParentDir;
		}
		bIsReadOnly = false;
	}

	bHasExistenceData = true;
}

bool FPathExistence::HasExistenceData() const
{
	return bHasExistenceData;
}

void FPathExistence::SetConfirmedExists(bool bValue)
{
	bHasExistenceData = bValue;
}

void FAssetDataDiscovery::SetPropertiesAndWait(TArrayView<FPathExistence> QueryPaths, bool bAddToAllowList,
	bool bForceRescan, bool bIgnoreDenyListScanFilters)
{
	struct FScanDirAndQueryPath
	{
		TRefCountPtr<FScanDir> ScanDir;
		bool bScanEntireTree = false;
	};
	TArray<FScanDirAndQueryPath> DirsToScan;
	bool bTickOwner = false;
	{
		CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
		FGathererScopeLock TreeScopeLock(&TreeLock);
		for (FPathExistence& QueryPath : QueryPaths)
		{
			// Performance note: It is important not to call any functions on QueryPath that access its existence data
			// until after the early exit for if (ScanDir->IsComplete() && !bForceRescan). Avoiding them lets us skip
			// the IO cost of reading existence data in the case where we have been asked to add a directory to the
			// AssetRegistry's list of scanned directories, but it already exists.

			FMountDir* MountDir = FindContainingMountPoint(QueryPath.GetLocalAbsPath());
			if (!MountDir)
			{
				UE_LOGF(LogAssetRegistry, Log, "SetPropertiesAndWait called on %ls which is not in a mounted directory. Call will be ignored.",
					*QueryPath.GetLocalAbsPath());
				continue;
			}

			FSetPathProperties Properties;
			if (bAddToAllowList)
			{
				Properties.IsOnAllowList = bAddToAllowList;
			}
			if (bForceRescan)
			{
				Properties.HasScanned = false;
			}
			if (bIgnoreDenyListScanFilters)
			{
				Properties.IgnoreDenyList = true;
			}
			FScanDirAndParentData ControlDirData;
			FStringView RelPathFromControlDirView;
			bool bMadeChanges;
			MountDir->TrySetDirectoryProperties(QueryPath, Properties, &ControlDirData, &RelPathFromControlDirView, &bMadeChanges);
			if (bMadeChanges)
			{
				SetIsIdle(false);
			}

			TRefCountPtr<FScanDir>& ScanDir = ControlDirData.ScanDir;
			FScanDir::FInherited MonitorData;
			bool bIsAllowedInThisCall = false;
			bool bIsDeniedInThisCall = false;
			bool bIsMonitoredInThisCall = false;
			if (ScanDir)
			{
				ScanDir->GetMonitorData(FStringView(), ControlDirData.ParentData, MonitorData);
				bIsAllowedInThisCall = MonitorData.IsOnAllowList() || bAddToAllowList;
				bIsDeniedInThisCall = MonitorData.IsOnDenyList() && !bIgnoreDenyListScanFilters;
				bIsMonitoredInThisCall = bIsAllowedInThisCall && !bIsDeniedInThisCall;
			}
			if (!ScanDir || !bIsMonitoredInThisCall)
			{
				UE_LOGF(LogAssetRegistry, Verbose, "SetPropertiesAndWait called on %ls which is not monitored. Call will be ignored.",
					*QueryPath.GetLocalAbsPath());
				continue;
			}

			if (!bForceRescan && ScanDir->IsComplete() && !QueryPath.HasExistenceData())
			{
				// For good performance, we need to avoid fetching existencedata for directories and files that have
				// already been scanned. Therefore we need to carefully use the information we have to early exit when
				// it is provable that the scan will produce no new data.
				// 
				// If we were asked to forcerescan, then we can't prove there is no new data, so this early exit is
				// only possible in the !bForceRescan case.
				// 
				// Otherwise, if the TrySetDirectoryProperties reports that we've already evaluated monitorability for
				// the requested settings, and scanned if we needed to because it is monitored, and that scan has
				// completed, then there is provably no new information that will come from the scan and we can exit.
				// 
				// TrySetDirectoryProperties will report that state by keeping the previously-set value of
				// IsComplete=true if it found there were no directory settings that needed to change. But it can also
				// find there are no directory settings that need to change if it reaches the parent directory of a file,
				// notices it does not have an entry for the subdirectory named for the file, and then checks whether
				// the requested path does not exist or is a file, and finds that it is. It will early exit with no
				// changes to the directory's completion status in that case, but we can't early exit here because we
				// still need to scan that file. But in that case, it will necessarily have fetched existencedata on
				// the QueryPath, so we can require that to not have been set before we allow the early exit from here.
				// And in that case there will be no further effort to use the existence data so we don't have a
				// performance problem.
				// 
				// So early exit (after check for bForceRescan) if and only if IsComplete && !HasExistenceData.
				continue;
			}

			// Save a copy of RelPathFromControlDirView; it points into QueryPath.AbsolutePath which might change
			FString RelPathFromControlDir(RelPathFromControlDirView);

			// After this point we are no longer in the common case of having no work to do, so now we can pay the IO
			// cost of loading the querypath's existence data, if we haven't already encountered that not-in-common-case
			// and fetched the data inside of TrySetDirectoryProperties.

			// We might have been asked to wait on a filename missing the extension, in which case QueryPath.GetType() == MissingButDirExists
			// We need to handle Directory, File, and MissingButDirExists in unique ways.
			FPathExistence::EType PathType = QueryPath.GetType();
			if (PathType == FPathExistence::EType::MissingParentDir)
			{
				// SetPropertiesAndWait is called for every ScanPathsSynchronous, and this is the first spot that checks for existence. 
				// Some systems call ScanPathsSynchronous speculatively to scan whatever is present, so this log is verbose-only.
				UE_LOGF(LogAssetRegistry, Verbose, "SetPropertiesAndWait called on non-existent path %ls. Call will be ignored.",
					*QueryPath.GetLocalAbsPath());
				continue;
			}

			bool bSearchPathIsDirectory = PathType == FPathExistence::EType::Directory || PathType == FPathExistence::EType::MissingButDirExists;
			if (bSearchPathIsDirectory)
			{
				if (ScanDir->IsComplete())
				{
					// The requested path (if a directory) or its closest parent directory already had the settings we
					// were asked to apply, and it is already complete, and (applicable if the requested path is a
					// file) the file did not exist so we can ignore the request to rescan the file. Therefore there
					// is no new scanned data to gather. This is similar to the early exit made above, but we also
					// early exit now even if bForceRescan is true.
					continue;
				}

				// If Relpath from the controlling dir to the requested dir (or the parent of the requested file if
				// the file is missing) is not empty then we have found a parent directory rather than the requested
				// directory.
				// This can only occur for a monitored directory when the requested directory is already complete and
				// we do not need to wait on it.
				if ((PathType == FPathExistence::EType::Directory && !RelPathFromControlDir.IsEmpty())
					|| (PathType == FPathExistence::EType::MissingButDirExists 
						&& UE::String::FindFirstOfAnyChar(RelPathFromControlDir, { '/', '\\' }) != INDEX_NONE))
				{
					continue;
				}

				FScanDirAndQueryPath& ScanQuery = DirsToScan.Emplace_GetRef();
				ScanQuery.ScanDir = ScanDir;
				ScanQuery.bScanEntireTree = PathType == FPathExistence::EType::Directory;
				FPriorityScanDirData* PriorityData = PriorityScanDirs.FindByPredicate(
					[&ScanDir](const FPriorityScanDirData& ScanDirData) { return ScanDirData.ScanDir == ScanDir; });
				if (!PriorityData)
				{
					PriorityData = &PriorityScanDirs.Add_GetRef(FPriorityScanDirData{ ScanDir });
				}
				++PriorityData->RequestCount;
				PriorityData->ParentData = ControlDirData.ParentData;
			}
			else
			{
				check(PathType == FPathExistence::EType::File);
				bool bAlreadyScanned = ScanDir->HasScanned() && MonitorData.IsMonitored();
				if (!bAlreadyScanned || bForceRescan)
				{
					FStringView RelPathFromParentDir = FPathViews::GetCleanFilename(RelPathFromControlDir);
					EGatherableFileType FileType = GetFileType(RelPathFromParentDir);
					if (FileType != EGatherableFileType::Invalid)
					{
						FStringView FileRelPathNoExt = FPathViews::GetBaseFilenameWithPath(RelPathFromControlDir);
						if (!DoesPathContainInvalidCharacters(FileType, FileRelPathNoExt))
						{
							TStringBuilder<256> LongPackageName;
							LongPackageName << MountDir->GetLongPackageName();
							FPathViews::AppendPath(LongPackageName, ScanDir->GetMountRelPath());
							FPathViews::AppendPath(LongPackageName, FileRelPathNoExt);
							AddDiscoveredFile(FDiscoveredPathData(QueryPath.GetLocalAbsPath(), LongPackageName,
								RelPathFromParentDir, QueryPath.GetModificationTime(), QueryPath.IsReadOnly(), FileType));
							if (FPathViews::IsPathLeaf(RelPathFromControlDir) && !ScanDir->HasScanned())
							{
								SetIsIdle(false);
								ScanDir->MarkFileAlreadyScanned(RelPathFromControlDir);
							}
						}
					}
				}
			}
		}

		if (!DirsToScan.IsEmpty())
		{
			bPriorityDirty = true;
			PriorityDataUpdated->Reset();
			bTickOwner = TickOwner.TryTakeOwnership(TreeScopeLock);
		}
	}

	while (!DirsToScan.IsEmpty())
	{
		if (bTickOwner)
		{
			TickInternal(false /* bTickAll */);
		}
		else
		{
			constexpr uint32 WaitTimeMilliseconds = 100;
			PriorityDataUpdated->Wait(WaitTimeMilliseconds);
		}

		{
			FGathererScopeLock LoopTreeScopeLock(&TreeLock);
			for (int32 Index = 0; Index < DirsToScan.Num(); ++Index)
			{
				FScanDirAndQueryPath& ScanQuery = DirsToScan[Index];
				FScanDir* ScanDir = ScanQuery.ScanDir.GetReference();
				auto RemoveCurrent = [&DirsToScan, &Index, ScanDir, this]()
				{
					DirsToScan.RemoveAtSwap(Index);
					int32 PriorityDataIndex = PriorityScanDirs.IndexOfByPredicate(
						[ScanDir](const FPriorityScanDirData& ScanDirData) { return ScanDirData.ScanDir.GetReference() == ScanDir; });

					check(PriorityDataIndex != INDEX_NONE); // Nothing should be able to remove it until we remove our RequestCount
					FPriorityScanDirData& PriorityData = PriorityScanDirs[PriorityDataIndex];
					check(PriorityData.RequestCount > 0);
					--PriorityData.RequestCount;
					if (PriorityData.RequestCount == 0)
					{
						PriorityScanDirs.RemoveAtSwap(PriorityDataIndex);
					}
				};

				if (!ScanDir->IsValid())
				{
					RemoveCurrent();
					continue;
				}
				if (ScanDir->IsComplete() || (!ScanQuery.bScanEntireTree && ScanDir->HasScanned()))
				{
					RemoveCurrent();
					continue;
				}
				else if (!ensureMsgf(!bIsIdle, TEXT("It should not be possible for the Discovery to go idle while there is an incomplete ScanDir.")))
				{
					RemoveCurrent();
					continue;
				}
			}

			if (DirsToScan.IsEmpty())
			{
				if (bTickOwner)
				{
					TickOwner.ReleaseOwnershipChecked(LoopTreeScopeLock);
					bTickOwner = false;
				}
			}
			else
			{
				PriorityDataUpdated->Reset();
				if (!bTickOwner)
				{
					bTickOwner = TickOwner.TryTakeOwnership(LoopTreeScopeLock);
				}
			}
		}
	}
}

void FAssetDataDiscovery::PrioritizeSearchPath(const FString& LocalAbsPath, EPriority Priority)
{
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
	FGathererScopeLock TreeScopeLock(&TreeLock);
	SetIsIdle(false);
	FMountDir* MountDir = FindContainingMountPoint(LocalAbsPath);
	if (!MountDir)
	{
		UE_LOGF(LogAssetRegistry, Warning, "FAssetDataGatherer::PrioritizeSearchPath called on unmounted path %.*ls. Call will be ignored.",
			LocalAbsPath.Len(), *LocalAbsPath);
		return;
	}

	FSetPathProperties EmptyProperties;
	FScanDirAndParentData ScanDirAndParent;
	FPathExistence QueryPath(LocalAbsPath);
	MountDir->TrySetDirectoryProperties(QueryPath, EmptyProperties, &ScanDirAndParent);
	FScanDir* ScanDir = ScanDirAndParent.ScanDir.GetReference();
	if (ScanDir && ScanDir->IsValid() && !ScanDir->IsComplete())
	{
		FPriorityScanDirData* PriorityData = PriorityScanDirs.FindByPredicate(
			[&ScanDir](const FPriorityScanDirData& ScanDirData) { return ScanDirData.ScanDir.GetReference() == ScanDir; });
		if (!PriorityData)
		{
			PriorityData = &PriorityScanDirs.Add_GetRef(FPriorityScanDirData{ ScanDir });
		}
		if (!PriorityData->bReleaseWhenComplete)
		{
			PriorityData->bReleaseWhenComplete = true;
			++PriorityData->RequestCount;
		}
		PriorityData->ParentData = ScanDirAndParent.ParentData;
	}
}

void FAssetDataDiscovery::TrySetDirectoryProperties(const FString& LocalAbsPath,
	const FSetPathProperties& InProperties, bool bConfirmedExists)
{
	if (!InProperties.IsSet())
	{
		return;
	}
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
	FPathExistence QueryPath(LocalAbsPath);
	QueryPath.SetConfirmedExists(bConfirmedExists);
	FGathererScopeLock TreeScopeLock(&TreeLock);
	TrySetDirectoryPropertiesInternal(QueryPath, InProperties);
}

void FAssetDataDiscovery::TrySetDirectoryPropertiesInternal(FPathExistence& QueryPath,
	const FSetPathProperties& InProperties)
{
	CHECK_IS_LOCKED_CURRENT_THREAD(TreeLock);
	FMountDir* MountDir = FindContainingMountPoint(QueryPath.GetLocalAbsPath());
	if (!MountDir)
	{
		UE_LOGF(LogAssetRegistry, Warning,
			"FAssetDataGatherer::SetDirectoryProperties called on unmounted path %ls. Call will be ignored.",
			*QueryPath.GetLocalAbsPath());
		return;
	}

	bool bMadeChanges;
	MountDir->TrySetDirectoryProperties(QueryPath, InProperties, nullptr /* OutControllingDir */,
		nullptr /* OutControllingDirRelPath */, &bMadeChanges);
	if (bMadeChanges)
	{
		SetIsIdle(false);
	}
}

bool FAssetDataDiscovery::IsOnAllowList(FStringView LocalAbsPath) const
{
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
	FGathererScopeLock TreeScopeLock(&TreeLock);
	const FMountDir* MountDir = FindContainingMountPoint(LocalAbsPath);
	if (!MountDir)
	{
		return false;
	}
	FScanDir::FInherited MonitorData;
	MountDir->GetMonitorData(LocalAbsPath, MonitorData);
	return MonitorData.IsOnAllowList();
}

bool FAssetDataDiscovery::IsOnDenyList(FStringView LocalAbsPath) const
{
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
	FGathererScopeLock TreeScopeLock(&TreeLock);
	const FMountDir* MountDir = FindContainingMountPoint(LocalAbsPath);
	if (!MountDir)
	{
		return false;
	}
	FScanDir::FInherited MonitorData;
	MountDir->GetMonitorData(LocalAbsPath, MonitorData);
	return MonitorData.IsOnDenyList();
}

bool FAssetDataDiscovery::IsMonitored(FStringView LocalAbsPath) const
{
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
	FGathererScopeLock TreeScopeLock(&TreeLock);
	const FMountDir* MountDir = FindContainingMountPoint(LocalAbsPath);
	return MountDir && MountDir->IsMonitored(LocalAbsPath);
}

bool FAssetDataDiscovery::IsCacheWriteEnabled() const
{
	// Upon initialization, WriteEnabled will be forced to Never if not supported by the platform
	return Cache.IsInitialized() && Cache.IsWriteEnabled() != EFeatureEnabled::Never;
}

template <typename ContainerType>
SIZE_T GetArrayRecursiveAllocatedSize(const ContainerType& Container)
{
	SIZE_T Result = Container.GetAllocatedSize();
	for (const auto& Value : Container)
	{
		Result += Value.GetAllocatedSize();
	}
	return Result;
};

SIZE_T FAssetDataDiscovery::GetAllocatedSize() const
{
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(TreeLock);
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
	check(!TickOwner.IsOwnedByCurrentThread());
	FScopedPause ScopedPause(*this);
	FGathererScopeLock TreeScopeLock(&TreeLock);
	FGathererScopeLock ResultsScopeLock(&ResultsLock);

	SIZE_T Result = 0;
	Result += GetArrayRecursiveAllocatedSize(LongPackageNamesDenyList);
	Result += GetArrayRecursiveAllocatedSize(MountRelativePathsDenyList);
	if (Thread)
	{
		// TODO: Thread->GetAllocatedSize()
		Result += sizeof(*Thread);
	}

	Result += GetArrayRecursiveAllocatedSize(DiscoveredDirectories);
	Result += GetArrayRecursiveAllocatedSize(DiscoveredFiles);
	Result += GetArrayRecursiveAllocatedSize(DiscoveredSingleFiles);

	Result += MountDirs.GetAllocatedSize();
	for (const TUniquePtr<FMountDir>& Value : MountDirs)
	{
		Result += sizeof(*Value);
		Result += Value->GetAllocatedSize();
	}
	Result += GetArrayRecursiveAllocatedSize(DirToScanDatas);
	Result += DirToScanBuffers.GetAllocatedSize();
	return Result;
}

void FAssetDataDiscovery::FDirToScanData::Reset()
{
	DirLocalAbsPath.Reset();
	DirLongPackageName.Reset();
	NumIteratedDirs = 0;
	NumIteratedFiles = 0;
	bScanned = false;
}

SIZE_T FAssetDataDiscovery::FDirToScanData::GetAllocatedSize() const
{
	SIZE_T Result = 0;
	Result += DirLocalAbsPath.GetAllocatedSize();
	Result += DirLongPackageName.GetAllocatedSize();
	Result += GetArrayRecursiveAllocatedSize(IteratedSubDirs);
	Result += GetArrayRecursiveAllocatedSize(IteratedFiles);
	return Result;
}

void FAssetDataDiscovery::FDirToScanBuffer::Reset()
{
	bAbort = false;
}

SIZE_T FAssetDataDiscovery::FDirectoryResult::GetAllocatedSize() const
{
	return DirAbsPath.GetAllocatedSize() + Files.GetAllocatedSize();
}

void FAssetDataDiscovery::Shrink()
{
	check(TickOwner.IsOwnedByCurrentThread());
	CHECK_IS_LOCKED_CURRENT_THREAD(TreeLock);
	CHECK_IS_LOCKED_CURRENT_THREAD(ResultsLock);
	DiscoveredDirectories.Shrink();
	DiscoveredFiles.Shrink();
	DiscoveredSingleFiles.Shrink();
	MountDirs.Shrink();
	for (TUniquePtr<FMountDir>& MountDir : MountDirs)
	{
		MountDir->Shrink();
	}
	DirToScanDatas.Empty();
	DirToScanBuffers.Empty();
}

void FAssetDataDiscovery::AddMountPoint(const FString& LocalAbsPath, FStringView LongPackageName, bool& bOutAlreadyExisted)
{
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
	FGathererScopeLock TreeScopeLock(&TreeLock);
	AddMountPointInternal(LocalAbsPath, LongPackageName, bOutAlreadyExisted);
	if (!bOutAlreadyExisted)
	{
		SetIsIdle(false);
	}
}

void FAssetDataDiscovery::AddMountPointInternal(const FString& LocalAbsPath, FStringView LongPackageName, bool& bOutAlreadyExisted)
{
	CHECK_IS_LOCKED_CURRENT_THREAD(TreeLock);

	bOutAlreadyExisted = false;
	if (FMountDir* ExistingMount = FindMountPoint(LocalAbsPath))
	{
		bOutAlreadyExisted = true;
		return;
	}

	TArray<FMountDir*> ChildMounts;
	FMountDir* ParentMount = nullptr;
	for (TUniquePtr<FMountDir>& ExistingMount : MountDirs)
	{
		if (FPathViews::IsParentPathOf(ExistingMount->GetLocalAbsPath(), LocalAbsPath))
		{
			// Overwrite any earlier ParentMount; later mounts are more direct parents than earlier mounts
			ParentMount = ExistingMount.Get();
		}
		else if (FPathViews::IsParentPathOf(LocalAbsPath, ExistingMount->GetLocalAbsPath()))
		{
			// A mount under the new directory might be a grandchild mount.
			// Don't add it as a child mount unless there is no other mount in between the new mount and the mount.
			FMountDir* ExistingParentMount = ExistingMount->GetParentMount();
			if (!ExistingParentMount || ExistingParentMount == ParentMount)
			{
				ChildMounts.Add(ExistingMount.Get());
			}
		}
	}
	SetIsIdle(false);

	FMountDir& Mount = FindOrAddMountPoint(LocalAbsPath, LongPackageName);
	if (ParentMount)
	{
		FStringView RelPath;
		verify(FPathViews::TryMakeChildPathRelativeTo(LocalAbsPath, ParentMount->GetLocalAbsPath(), RelPath));
		ParentMount->AddChildMount(&Mount);
		Mount.SetParentMount(ParentMount);
		for (FMountDir* ChildMount : ChildMounts)
		{
			ParentMount->RemoveChildMount(ChildMount);
		}
	}
	for (FMountDir* ChildMount : ChildMounts)
	{
		Mount.AddChildMount(ChildMount);
		ChildMount->SetParentMount(&Mount);
	}
}

void FAssetDataDiscovery::RemoveMountPoint(const FString& LocalAbsPath)
{
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
	FGathererScopeLock TreeScopeLock(&TreeLock);
	SetIsIdle(false);
	RemoveMountPointInternal(LocalAbsPath);
}

void FAssetDataDiscovery::RemoveMountPointInternal(const FString& LocalAbsPath)
{
	CHECK_IS_LOCKED_CURRENT_THREAD(TreeLock);
	int32 ExistingIndex = FindLowerBoundMountPoint(LocalAbsPath);
	if (ExistingIndex == MountDirs.Num() || !FPathViews::Equals(MountDirs[ExistingIndex]->GetLocalAbsPath(), LocalAbsPath))
	{
		return;
	}
	TUniquePtr<FMountDir> Mount = MoveTemp(MountDirs[ExistingIndex]);
	MountDirs.RemoveAt(ExistingIndex);
	FMountDir* ParentMount = Mount->GetParentMount();

	if (ParentMount)
	{
		for (FMountDir* ChildMount : Mount->GetChildMounts())
		{
			ParentMount->AddChildMount(ChildMount);
			ChildMount->SetParentMount(ParentMount);
		}
		ParentMount->RemoveChildMount(Mount.Get());
	}
	else
	{
		for (FMountDir* ChildMount : Mount->GetChildMounts())
		{
			ChildMount->SetParentMount(nullptr);
		}
	}
}

void FAssetDataDiscovery::OnDirectoryCreated(FStringView LocalAbsPath)
{
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
	FGathererScopeLock TreeScopeLock(&TreeLock);
	FMountDir* MountDir = FindContainingMountPoint(LocalAbsPath);
	if (!MountDir || !MountDir->IsMonitored(LocalAbsPath))
	{
		return;
	}

	FStringView MountRelPath;
	verify(FPathViews::TryMakeChildPathRelativeTo(LocalAbsPath, MountDir->GetLocalAbsPath(), MountRelPath));
	TStringBuilder<128> LongPackageName;
	LongPackageName << MountDir->GetLongPackageName();
	FPathViews::AppendPath(LongPackageName, MountRelPath);
	if (FPackageName::DoesPackageNameContainInvalidCharacters(LongPackageName))
	{
		return;
	}

	FDiscoveredPathData DirData;
	DirData.LocalAbsPath = LocalAbsPath;
	DirData.LongPackageName = LongPackageName;
	DirData.RelPath = FPathViews::GetCleanFilename(MountRelPath);

	// Note that we AddDiscovered but do not scan the directory
	// Any files and paths under it will be added by their own event from the directory watcher, so a scan is unnecessary.
	// The directory may also be scanned in the future because a parent directory is still yet pending to scan,
	// we do not try to prevent that wasteful rescan because this is a rare event and it does not cause a behavior problem
	SetIsIdle(false);
	AddDiscovered(DirData.LocalAbsPath, DirData.LongPackageName, TConstArrayView<FDiscoveredPathData>(&DirData, 1), TConstArrayView<FDiscoveredPathData>());
}

void FAssetDataDiscovery::OnFilesCreated(TConstArrayView<FString> LocalAbsPaths)
{
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
	FGathererScopeLock TreeScopeLock(&TreeLock);
	SetIsIdle(false);
	for (const FString& LocalAbsPath : LocalAbsPaths)
	{
		OnFileCreated(LocalAbsPath);
	}
}

void FAssetDataDiscovery::OnFileCreated(const FString& LocalAbsPath)
{
	CHECK_IS_LOCKED_CURRENT_THREAD(TreeLock);
	// Detect whether the file should be scanned and if so pass it through to the gatherer
	FMountDir* MountDir = FindContainingMountPoint(LocalAbsPath);
	if (!MountDir)
	{
		// The content root of the file is not registered; ignore it
		return;
	}
	FFileStatData StatData = IFileManager::Get().GetStatData(*LocalAbsPath);
	if (!StatData.bIsValid || StatData.bIsDirectory)
	{
		// The caller has erroneously told us a file exists that doesn't exist (perhaps due to create/delete hysteresis); ignore it
		return;
	}

	FString FileRelPath;
	FScanDir::FInherited MonitorData;
	FScanDir* ScanDir = MountDir->GetControllingDir(LocalAbsPath, false /* bIsDirectory */, MonitorData, FileRelPath);
	if (!ScanDir || !MonitorData.IsMonitored())
	{
		// The new file is in an unmonitored directory; ignore it
		return;
	}

	FStringView RelPathFromParentDir = FPathViews::GetCleanFilename(FileRelPath);
	EGatherableFileType FileType = GetFileType(RelPathFromParentDir);
	if (FileType != EGatherableFileType::Invalid)
	{
		FStringView FileRelPathNoExt = FPathViews::GetBaseFilenameWithPath(FileRelPath);
		if (!DoesPathContainInvalidCharacters(FileType, FileRelPathNoExt))
		{
			TStringBuilder<256> LongPackageName;
			LongPackageName << MountDir->GetLongPackageName();
			FPathViews::AppendPath(LongPackageName, ScanDir->GetMountRelPath());
			FPathViews::AppendPath(LongPackageName, FileRelPathNoExt);
			AddDiscoveredFile(FDiscoveredPathData(LocalAbsPath, LongPackageName, RelPathFromParentDir, StatData.ModificationTime, StatData.bIsReadOnly, FileType));
			if (FPathViews::IsPathLeaf(FileRelPath))
			{
				ScanDir->MarkFileAlreadyScanned(FileRelPath);
			}
		}
	}
}

FMountDir* FAssetDataDiscovery::FindContainingMountPoint(FStringView LocalAbsPath)
{
	CHECK_IS_LOCKED_CURRENT_THREAD(TreeLock);
	int32 Index = FindLowerBoundMountPoint(LocalAbsPath);
	// The LowerBound is >= LocalAbsPath, so it is a parentpath of LocalAbsPath only if it is equal to LocalAbsPath
	if (Index < MountDirs.Num() && FPathViews::Equals(MountDirs[Index]->GetLocalAbsPath(), LocalAbsPath))
	{
		return MountDirs[Index].Get();
	}

	// The last element before the lower bound is either (1) an unrelated path and LocalAbsPath does not have a parent
	// (2) a parent path of LocalAbsPath, (3) A sibling path that is a child of an earlier path that is a parent path of LocalAbsPath
	// (4) An unrelated path that is a child of an earlier path, but none of its parents are a parent path of LocalAbsPath
	// Distinguishing between cases (3) and (4) doesn't have a fast algorithm based on sorted paths alone, but we have recorded the parent
	// so we can figure it out that way
	if (Index > 0)
	{
		FMountDir* Previous = MountDirs[Index - 1].Get();
		while (Previous)
		{
			if (FPathViews::IsParentPathOf(Previous->GetLocalAbsPath(), LocalAbsPath))
			{
				return Previous;
			}
			Previous = Previous->GetParentMount();
		}
	}
	return nullptr;
}

const FMountDir* FAssetDataDiscovery::FindContainingMountPoint(FStringView LocalAbsPath) const
{
	CHECK_IS_LOCKED_CURRENT_THREAD(TreeLock);
	return const_cast<FAssetDataDiscovery*>(this)->FindContainingMountPoint(LocalAbsPath);
}

FMountDir* FAssetDataDiscovery::FindMountPoint(FStringView LocalAbsPath)
{
	CHECK_IS_LOCKED_CURRENT_THREAD(TreeLock);
	int32 Index = FindLowerBoundMountPoint(LocalAbsPath);
	if (Index != MountDirs.Num() && FPathViews::Equals(MountDirs[Index]->GetLocalAbsPath(), LocalAbsPath))
	{
		return MountDirs[Index].Get();
	}
	return nullptr;
}

FMountDir& FAssetDataDiscovery::FindOrAddMountPoint(FStringView LocalAbsPath, FStringView LongPackageName)
{
	CHECK_IS_LOCKED_CURRENT_THREAD(TreeLock);
	int32 Index = FindLowerBoundMountPoint(LocalAbsPath);
	if (Index != MountDirs.Num() && FPathViews::Equals(MountDirs[Index]->GetLocalAbsPath(), LocalAbsPath))
	{
		// Already exists
		return *MountDirs[Index];
	}
	return *MountDirs.EmplaceAt_GetRef(Index, new FMountDir(*this, LocalAbsPath, LongPackageName));
}

int32 FAssetDataDiscovery::FindLowerBoundMountPoint(FStringView LocalAbsPath) const
{
	CHECK_IS_LOCKED_CURRENT_THREAD(TreeLock);
	return Algo::LowerBound(MountDirs, LocalAbsPath, [](const TUniquePtr<FMountDir>& MountDir, FStringView LocalAbsPath)
		{
			return FPathViews::Less(MountDir->GetLocalAbsPath(), LocalAbsPath);
		}
	);
}

void FAssetDataDiscovery::AddDiscovered(FStringView DirAbsPath, FStringView DirPackagePath, TConstArrayView<FDiscoveredPathData> SubDirs, TConstArrayView<FDiscoveredPathData> Files)
{
	// This function is inside the critical section so we have moved filtering results outside of it
	// Caller is responsible for filtering SubDirs and Files by ShouldScan and packagename validity
	FGathererScopeLock ResultsScopeLock(&ResultsLock);

	if (UE::AssetDataGather::Private::bIgnoreEmptyDirectories)
	{
		// Only register this directory, this will get called for anything that has files or subdirectories
		DiscoveredDirectories.Add(FString(DirPackagePath));
	}
	else 
	{
		// Register all of the subdirectories even if they are empty
		for (const FDiscoveredPathData& SubDir : SubDirs)
		{
			DiscoveredDirectories.Add(FString(SubDir.LongPackageName));
		}
	}

	if (Files.Num())
	{
		DiscoveredFiles.Emplace(DirAbsPath, Files);
		NumDiscoveredFiles += Files.Num();
	}
}

void FAssetDataDiscovery::AddDiscoveredFile(FDiscoveredPathData&& File)
{
	FGathererScopeLock ResultsScopeLock(&ResultsLock);
	DiscoveredSingleFiles.Emplace(MoveTemp(File));
	NumDiscoveredFiles++;
}

EGatherableFileType FAssetDataDiscovery::GetFileType(FStringView FilePath)
{
	if (FPackageName::IsPackageFilename(FilePath))
	{
		return EGatherableFileType::PackageFile;
	}
	else if (FilePath.EndsWith(TEXT(".verse")))
	{
		return EGatherableFileType::VerseFile;
	}
	else if (FilePath.EndsWith(TEXT(".vmodule")))
	{
		return EGatherableFileType::VerseModule;
	}
	else
	{
		return EGatherableFileType::Invalid;
	}
}

FAssetDataDiscovery::FDirectoryResult::FDirectoryResult(FStringView InDirAbsPath, TConstArrayView<FDiscoveredPathData> InFiles)
	: DirAbsPath(InDirAbsPath)
{
	Files.Reserve(InFiles.Num());
	for (const FDiscoveredPathData& DiscoveredFile : InFiles)
	{
		Files.Emplace(DiscoveredFile); // Emplace passes the FDiscoveredPathData to the FGatheredPathData explicit constructor for it
	}

}

// this namespace contains deprecated code
namespace AssetRegistryBackwardCompat
{

// Reads an FMemoryView once
class FMemoryViewReader
{
public:
	FMemoryViewReader() = default;
	FMemoryViewReader(FMemoryView Data) : Remaining(Data), TotalSize(Data.GetSize()) {}

	uint64 GetRemainingSize() const { return Remaining.GetSize(); }
	uint64 GetTotalSize() const { return TotalSize; }
	uint64 Tell() const { return TotalSize - Remaining.GetSize(); }

	FMemoryView Load(uint64 Size)
	{
		check(Size <= Remaining.GetSize());
		FMemoryView Out(Remaining.GetData(), Size);
		Remaining += Size;
		return Out;
	}

	void Load(FMutableMemoryView Out)
	{
		FMemoryView In = Load(Out.GetSize());
		if (In.GetSize())
		{
			FMemory::Memcpy(Out.GetData(), In.GetData(), In.GetSize());
		}
	}

	template<typename T>
	T Load()
	{
		static_assert(std::is_integral_v<T>, "Only integer loading supported");
		static_assert(PLATFORM_LITTLE_ENDIAN, "Byte-swapping not implemented");
		return FPlatformMemory::ReadUnaligned<T>(Load(sizeof(T)).GetData());
	}

	template<typename T>
	TOptional<T> TryLoad()
	{
		return sizeof(T) <= Remaining.GetSize() ? Load<T>() : TOptional<T>();
	}

private:
	FMemoryView Remaining;
	uint64 TotalSize = 0;
};


// Enables both versioning and distinguishing out-of-sync reads from data corruption
static constexpr uint32 BlockMagic = 0xb1a3;

struct FBlockHeader
{
	uint32 Magic = 0;
	uint32 Size = 0;
	uint64 Checksum = 0;
};

TOptional<FBlockHeader> LoadBlockHeader(FMemoryView Data)
{
	check(Data.GetSize() == sizeof(FBlockHeader));

	FMemoryViewReader Reader(Data);
	FBlockHeader Header;
	Header.Magic = Reader.Load<uint32>();
	Header.Size = Reader.Load<uint32>();
	Header.Checksum = Reader.Load<uint64>();

	if (Header.Magic != BlockMagic)
	{
		UE_LOGF(LogAssetRegistry, Warning, "Wrong block magic (0x%x)", Header.Magic);
		return {};
	}

	return Header;
}

static uint64 CalculateBlockChecksum(FMemoryView Data)
{
	return INTEL_ORDER64(FXxHash64::HashBuffer(Data).Hash);
}

class FChecksumArchiveBase : public FArchiveProxy
{
	static constexpr uint32 SaveBlockSize = 4 << 20;

	struct FBlock
	{
		uint8* Begin = nullptr;
		uint8* Cursor = nullptr;
		uint8* End = nullptr;

		explicit FBlock(uint32 Size)				{ Reset(Size); }
		~FBlock()									{ FMemory::Free(Begin); }
		uint64 GetCapacity() const					{ return End - Begin; }
		uint64 GetUsedSize() const					{ return Cursor - Begin; }
		uint64 GetRemainingSize() const				{ return End - Cursor; }
		FMutableMemoryView GetUsed() const			{ return {Begin, GetUsedSize()}; }
		FMutableMemoryView GetRemaining() const		{ return {Cursor, GetRemainingSize()}; };

		void Reset(uint32 Size)
		{
			if (GetCapacity() < Size)
			{
				FMemory::Free(Begin);
				Begin = (uint8*)FMemory::Malloc(Size);
			}
			
			// All blocks have the same size except the last one, which may be smaller.
			// It doesn't matter that we lose some capacity when loading the last block.
			End = Begin + Size;
			Cursor = Begin;
		}

		void Write(FMemoryView In)
		{
			check(GetRemainingSize() >= In.GetSize());
			if (In.GetSize())
			{
				FMemory::Memcpy(Cursor, In.GetData(), In.GetSize());
			}
			Cursor += In.GetSize();	
		}

		void Read(FMutableMemoryView Out)
		{
			check(GetRemainingSize() >= Out.GetSize());
			if (Out.GetSize())
			{
				FMemory::Memcpy(Out.GetData(), Cursor, Out.GetSize());
			}
			Cursor += Out.GetSize();	
		}
	};

	FBlock Block;

	void SaveBlock()
	{
		FBlockHeader Header;
		Header.Magic = BlockMagic;
		Header.Size = IntCastChecked<uint32>(Block.GetUsedSize());
		Header.Checksum = CalculateBlockChecksum(Block.GetUsed());
		InnerArchive << Header.Magic << Header.Size << Header.Checksum;

		InnerArchive.Serialize(Block.Begin, Header.Size);

		Block.Cursor = Block.Begin;
	}

	bool LoadBlock()
	{
		check(Block.GetRemainingSize() == 0);

		uint8 HeaderData[sizeof(FBlockHeader)];
		InnerArchive.Serialize(HeaderData, sizeof(HeaderData));
		if (InnerArchive.IsError())
		{
			UE_LOGF(LogAssetRegistry, Warning, "Couldn't read block header");
			return false;
		}

		if (TOptional<FBlockHeader> Header = LoadBlockHeader(MakeMemoryView(HeaderData)))
		{
			Block.Reset(Header->Size);

			InnerArchive.Serialize(Block.Begin, Header->Size);

			if (InnerArchive.IsError())
			{
				UE_LOGF(LogAssetRegistry, Warning, "Couldn't read block data");
				return false;
			}
			else if (CalculateBlockChecksum(Block.GetRemaining()) != Header->Checksum)
			{
				UE_LOGF(LogAssetRegistry, Warning, "Wrong block checksum");
				return false;
			}

			return true;
		}

		return false;
	}

public:
	explicit FChecksumArchiveBase(FArchive& Inner)
	: FArchiveProxy(Inner)
	, Block(IsLoading() ? 0 : SaveBlockSize)
	{}

protected:
	~FChecksumArchiveBase()
	{
		if (!IsLoading() && Block.GetUsedSize() > 0)
		{
			SaveBlock();
		}
	}
	const FBlock& GetCurrentBlock() const
	{
		return Block;
	}

	void Save(FMemoryView Data)
	{
		for (uint64 Size = Block.GetRemainingSize(); Size < Data.GetSize(); Size = Block.GetRemainingSize())
		{
			Block.Write(Data.Left(Size));
			Data += Size;
			SaveBlock();
		}

		Block.Write(Data);
	}

	void Load(FMutableMemoryView Data)
	{
		if (IsError())
		{
			return;
		}

		for (uint64 Size = Block.GetRemainingSize(); Size < Data.GetSize(); Size = Block.GetRemainingSize())
		{
			Block.Read(Data.Left(Size));
			Data += Size;

			if (!LoadBlock())
			{
				UE_LOGF(LogAssetRegistry, Warning, "Integrity check failed, '%ls' cache will be discarded", *InnerArchive.GetArchiveName());
				SetError();
				return;
			}
		}

		Block.Read(Data);
	}

public:
	// Use FArchive implementations that map back to Serialize() instead of FArchiveProxy overloads
	// that forward to the inner archive and bypass integrity checking
	virtual FArchive& operator<<(FText& Value) override					{ return FArchive::operator<<(Value); }
	virtual void SerializeBits(void* Bits, int64 LengthBits) override	{ FArchive::SerializeBits(Bits, LengthBits); }
	virtual void SerializeInt(uint32& Value, uint32 Max) override		{ FArchive::SerializeInt(Value, Max); }
	virtual void SerializeIntPacked(uint32& Value) override				{ FArchive::SerializeIntPacked(Value); }

private:
	// Wrapping an inner FArchiveUObject is not supported. The inner archive should be low-level
	// and an outer archive should have intercepted these calls.
	virtual FArchive& operator<<(FName& Value) override					{ unimplemented(); return *this; }
	virtual FArchive& operator<<(UObject*& Value) override				{ unimplemented(); return *this; }
	virtual FArchive& operator<<(FObjectPtr& Value) override			{ unimplemented(); return *this; }
	virtual FArchive& operator<<(FLazyObjectPtr& Value) override		{ unimplemented(); return *this; }
	virtual FArchive& operator<<(FSoftObjectPath& Value) override		{ unimplemented(); return *this; }
	virtual FArchive& operator<<(FSoftObjectPtr& Value) override		{ unimplemented(); return *this; }
	virtual FArchive& operator<<(FWeakObjectPtr& Value) override		{ unimplemented(); return *this; }
	virtual FArchive& operator<<(FField*& Value) override				{ unimplemented(); return *this; }

	virtual void Seek(int64 InPos) override								{ unimplemented(); }
};

class FChecksumArchiveReader : public FChecksumArchiveBase
{
public:
	using FChecksumArchiveBase::FChecksumArchiveBase;
	virtual void Serialize(void* V, int64 Len) override { Load(FMutableMemoryView(V, Len)); }
	virtual int64 Tell() override { return InnerArchive.Tell() - GetCurrentBlock().GetRemainingSize(); }
};

// Memory-mapped equivalent of FChecksumArchiveReader
class FChecksumViewReader : public FArchive
{
public:
	explicit FChecksumViewReader(FMemoryViewReader&& Reader, FStringView InFileName)
	: RemainingBlocks(MoveTemp(Reader))
	, FileName(InFileName)
	{
		SetIsLoading(true);
	}

private:
	FMemoryViewReader RemainingBlocks;
	FMemoryViewReader CurrentBlock;
	FString FileName;

	virtual void Seek(int64) override { unimplemented(); }
	virtual int64 Tell() override { return RemainingBlocks.Tell() - CurrentBlock.GetRemainingSize(); }
	virtual int64 TotalSize() override { return RemainingBlocks.GetTotalSize(); }

	virtual void Serialize(void* V, int64 Len) override
	{
		FMutableMemoryView Out(V, Len);

		while (CurrentBlock.GetRemainingSize() < Out.GetSize())
		{
			if (IsError())
			{
				return;
			}

			FMutableMemoryView OutSlice(Out.GetData(), CurrentBlock.GetRemainingSize());
			Out += OutSlice.GetSize();
			CurrentBlock.Load(OutSlice);
			check(CurrentBlock.GetRemainingSize() == 0);

			TOptional<FMemoryView> NextBlock = LoadNextBlock(RemainingBlocks);
			if (!NextBlock)
			{
				UE_LOGF(LogAssetRegistry, Warning, "Integrity check failed, '%ls' cache will be discarded", *FileName);
				SetError();
				return;
			}
			CurrentBlock = *NextBlock;
		}

		CurrentBlock.Load(Out);
	}

	FORCENOINLINE static TOptional<FMemoryView> LoadNextBlock(FMemoryViewReader& In)
	{
		if (In.GetRemainingSize() < sizeof(FBlockHeader))
		{
			UE_LOGF(LogAssetRegistry, Warning, "Couldn't read block header");
			return {};
		}

		if (TOptional<FBlockHeader> Header = LoadBlockHeader(In.Load(sizeof(FBlockHeader))))
		{
			if (Header->Size > In.GetRemainingSize())
			{
				UE_LOGF(LogAssetRegistry, Warning, "Incomplete block");
				return {};
			}

			FMemoryView Block = In.Load(Header->Size);
			if (CalculateBlockChecksum(Block) != Header->Checksum)
			{
				UE_LOGF(LogAssetRegistry, Warning, "Wrong block checksum");
				return {};
			}

			return Block;
		}

		return {};
	}
};

}

#if UE_EDITOR
/** A class to preload the AssetData cache and start the AssetDataDiscovery thread (which does its own cache loading)
 * used by the FAssetDataGatherer. Preloading allows us to start very early in editor startup, so that we 
 * have time to finish the cache loads before the engine starts making package load requests that need to 
 * use the discovery and gatherer data.
 *
 * In UE_EDITOR, we know the values we need to decide whether we can preload early enough that it is useful to preload
 * In other configurations we do not know those parameters for sure until EDelayedRegisterRunPhase::ShaderTypesReady,
 * which occurs around the same time as UAssetRegistryImpl is created, so it is not useful to preload.
 */
class FPreloader : public FDelayedAutoRegisterHelper
{
public:
	FPreloader()
		// The callback needs to occur after GIsEditor, ProjectIntermediateDir, IsRunningCommandlet, and
		// IsRunningCookCommandlet have been set
		:FDelayedAutoRegisterHelper(EDelayedRegisterRunPhase::IniSystemReady, [this]() { DelayedInitialize(); }) //-V1099
	{
	}

	~FPreloader()
	{
		// This destructor is called during c++ global shutdown after main, and it is not valid to call TFuture::Wait
		// after Engine shutdown. TaskGraph has already shutdown at this point, so we do not need to worry about the 
		// Async thread still running and accessing *this.
		// PreloadReady.Wait();
	}

	TArray<FCachePayload> Consume()
	{
		check(bInitialized);
		if (bConsumedGatherCache)
		{
			return {};
		}
		bConsumedGatherCache = true;
		PreloadReady.Wait();
		PreloadReady.Reset();
		return MoveTemp(GatherCachePayloads);
	}

	TUniquePtr<FAssetDataDiscovery> ConsumeDiscovery()
	{
		check(bInitialized);
		if (bConsumedDiscoveryCache)
		{
			return nullptr;
		}
		bConsumedDiscoveryCache = true;
		return MoveTemp(Discovery);
	}

private:

	void DelayedInitialize()
	{
		GGatherSettings.Initialize();

		bCacheWriteEnabled = GGatherSettings.IsGatherCacheWriteEnabled();

		if (GGatherSettings.IsPreloadDiscoveryCache())
		{
			Discovery = MakeUnique<FAssetDataDiscovery>();
			Discovery->StartAsync();
		}
		if (GGatherSettings.IsPreloadGatherCache())
		{
			TArray<FString> CachePaths = GGatherSettings.FindShardedCacheFiles();
			if (CachePaths.IsEmpty())
			{
				CachePaths = GGatherSettings.FindLegacyShardedCacheFiles();
				if (CachePaths.Num())
				{
					UE_LOGF(LogAssetRegistry, Display, "Preload: found %d legacy cache files for migration", CachePaths.Num());
				}
			}
			if (CachePaths.Num())
			{
				PreloadReady = Async(EAsyncExecution::TaskGraph, [this, CachePaths=MoveTemp(CachePaths)]() { LoadAsync(CachePaths); });
			}
		}
		bInitialized = true;
	}

	void LoadAsync(const TArray<FString>& Paths)
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		CleanupOrphanedCacheFiles(GGatherSettings.GetCacheBaseFilename(), ECacheCleanupContext::PreLoad);
		GatherCachePayloads = LoadCacheFiles(Paths, bCacheWriteEnabled);
	}

	TFuture<void> PreloadReady;
	TArray<FCachePayload> GatherCachePayloads;
	TUniquePtr<UE::AssetDataGather::Private::FAssetDataDiscovery> Discovery;
	bool bInitialized = false;
	bool bConsumedGatherCache = false;
	bool bConsumedDiscoveryCache = false;
	bool bCacheWriteEnabled = false;
};
FPreloader GPreloader;
#endif

} // namespace UE::AssetDataGather::Private

static inline int32 GetNumberOfReadTaskThreadsToAllow()
{
	using namespace UE::AssetDataGather::Private;
	GGatherSettings.Initialize();

	// Cap how wide we allow read tasks to go. We don't want too much parallelism 
	// since ultimately there is stress on the OS for fetching things like File 
	// Control Blocks on Windows that can't be done in parallel. Experimentally, 
	// half the available physical cores has performed well.
	const int32 MaxReadThreads = FPlatformMisc::NumberOfCores();
	const int32 DefaultNumReadThreads = MaxReadThreads / 2;
	const int32 NumReadThreads = FGatherSettings::GARGatherThreads <= 0 ? DefaultNumReadThreads : FGatherSettings::GARGatherThreads;
	const int32 MinReadThreads = 1; // The TaskConcurrencyLimiter doesn't allow 0 worker threads
	return FMath::Min(MaxReadThreads, FMath::Max(MinReadThreads, NumReadThreads));
}

FAssetDataGatherer::FAssetDataGatherer(UAssetRegistryImpl& InRegistryImpl)
	: AssetRegistry(InRegistryImpl)
	, Thread(nullptr)
	, bAsyncEnabled(false)
	, GatherStartTime(FDateTime::Now())
	, IsStopped(0)
	, IsGatheringPaused(0)
	, IsProcessingPaused(0)
	, DelayedSaveAsyncCacheStartTime(0.0)
	, CurrentSearchTime(0.)
	, LastCacheWriteTime(0.0)
	, bHasLoadedCache(false)
	, bDiscoveryIsComplete(false)
	, bIsComplete(false)
	, bIsIdle(false)
	, bFirstTickAfterIdle(true)
	, bFinishedInitialDiscovery(false)
	, bIsInitialSearchCompleted(false)
	, bIsAdditionalMountSearchInProgress(false)
	, bGatherOnGameThreadOnly(false)
	, LastCacheSaveNumUncachedAssetFiles(0)
	, bIsSavingAsyncCache(false)
	, bFlushedRetryFiles(false)
{
	using namespace UE::AssetDataGather::Private;

	GGatherSettings.Initialize();

	constexpr bool bEditorExecutable = WITH_EDITOR;
	bGatherAssetPackageData = bEditorExecutable || GGatherSettings.IsForceDependsGathering();
	bGatherDependsData = GGatherSettings.IsGatherDependsData();
	bCacheReadEnabled = GGatherSettings.IsGatherCacheReadEnabled();
	bCacheWriteEnabled = GGatherSettings.IsGatherCacheWriteEnabled();
	bAsyncEnabled = GGatherSettings.IsAsyncEnabled();
	LastCacheWriteTime = FPlatformTime::Seconds();

	// Tick is synchronous until StartAsync is called, even if bAsyncEnabled
	bSynchronousTick = true; 

#if UE_EDITOR
	// If the preloader has already created the FAssetDataDiscovery, take ownership of it now
	if (TUniquePtr<FAssetDataDiscovery> PreloadedDiscovery = GPreloader.ConsumeDiscovery())
	{
		Discovery = MoveTemp(PreloadedDiscovery);
	}
	else
#endif
	{
		Discovery = MakeUnique<UE::AssetDataGather::Private::FAssetDataDiscovery>();
	}

	FilesToSearch = MakeUnique<UE::AssetDataGather::Private::FFilesToSearch>();
	FileReadScheduler = MakeUnique<FFileReadScheduler>(GetNumberOfReadTaskThreadsToAllow(), bGatherAssetPackageData, bGatherDependsData);
}

FAssetDataGatherer::~FAssetDataGatherer()
{
	EnsureCompletion();
	NewCachedAssetDataMap.Empty();
	DiskCachedAssetDataMap.Empty();

	for (FDiskCachedAssetData* AssetData : NewCachedAssetData)
	{
		delete AssetData;
	}
	NewCachedAssetData.Empty();
	for (TPair<int32, FDiskCachedAssetData*>& BlockData : DiskCachedAssetBlocks)
	{
		delete[] BlockData.Get<1>();
	}
	DiskCachedAssetBlocks.Empty();

	// Best-effort cleanup of orphaned binaries. Files still memory-mapped by this
	// or another process will fail to delete and be cleaned up on next startup.
	UE::AssetDataGather::Private::CleanupOrphanedCacheFiles(
		UE::AssetDataGather::Private::GGatherSettings.GetCacheBaseFilename(),
		UE::AssetDataGather::Private::ECacheCleanupContext::PostWrite);
}

void FAssetDataGatherer::UpdateCacheForSaving()
{
	// Note, this function must be called while holding the UAssetRegistry.InterfaceLock!
	TRACE_CPUPROFILER_EVENT_SCOPE(UpdateCacheForSaving)

	// Collect any DependsNode reservation information, if we have gathered any so far, to 
	// record how many nodes to create when loading the cache next time.
	{
		const FAssetRegistryState& AssetRegistryState = AssetRegistry.GuardedData.State;

		// When accessing the cache maps, we must ensure the tick lock is held
		FGathererScopeLock RunScopeLock(&TickLock);

		TArray<FName> AssetNames; 
		NewCachedAssetDataMap.GenerateKeyArray(AssetNames);
		ParallelFor(AssetNames.Num(), [&AssetRegistryState , &AssetNames, this](int Index)
			{
				FName AssetName = AssetNames[Index];
				FDiskCachedAssetData* CachedData = NewCachedAssetDataMap[AssetName];
				if (FDependsNode* Node = AssetRegistryState.FindDependsNode(AssetName))
				{
					CachedData->DependencyData.DependsNodeReservations = FDependsNodeReservations(*Node);
				}
			});

		DiskCachedAssetDataMap.GenerateKeyArray(AssetNames);
		ParallelFor(AssetNames.Num(), [&AssetRegistryState, &AssetNames, this](int Index)
			{
				FName AssetName = AssetNames[Index];
				FDiskCachedAssetData* CachedData = DiskCachedAssetDataMap[AssetName];
				if (FDependsNode* Node = AssetRegistryState.FindDependsNode(AssetName))
				{
					CachedData->DependencyData.DependsNodeReservations = FDependsNodeReservations(*Node);
				}
			});
	}
}

void FAssetDataGatherer::OnInitialSearchCompleted()
{
	if (Discovery)
	{
		Discovery->OnInitialSearchCompleted();
	}

	RequestAsyncCacheSave();
	bIsInitialSearchCompleted.store(true, std::memory_order_relaxed);
	FileReadScheduler->FreeReadTaskLoaders();
}

void FAssetDataGatherer::OnAdditionalMountSearchCompleted()
{
	if (Discovery)
	{
		Discovery->OnAdditionalMountSearchCompleted();
	}
	// If we haven't done so already, request a delayed save of the cache.
	// Since additional mounts may be added in quick succession we want to save the cache but only
	// after a brief period of time (i.e. FGatherSettings::MinSecondsToElapseBeforeCacheWrite) 
	// to avoid saving the entire cache multiples times in a row.
	RequestDelayedAsyncCacheSave();
	bIsAdditionalMountSearchInProgress.store(false, std::memory_order_relaxed);
	FileReadScheduler->FreeReadTaskLoaders();
}

void FAssetDataGatherer::StartAsync()
{
	if (bAsyncEnabled && !Thread)
	{
		bSynchronousTick = false;
		Thread = FRunnableThread::Create(this, TEXT("FAssetDataGatherer"), 0, TPri_BelowNormal);
		checkf(Thread, TEXT("Failed to create asset data gatherer thread"));
		Discovery->StartAsync();
	}
}

bool FAssetDataGatherer::Init()
{
	return true;
}

uint32 FAssetDataGatherer::Run()
{
	constexpr float IdleSleepTime = 0.1f;
	constexpr float PausedSleepTime = 0.005f;
	LLM_SCOPE(ELLMTag::AssetRegistry);

	while (!IsStopped || IsCacheSaveReady())
	{
		bool bShouldRequestAssetRegistryTick = false;
		ETickResult TickResult = InnerTickLoop(false /* bInSynchronousTick */, true /* bContributeToCacheSave */,
			-1. /* EndTimeSeconds */, &bShouldRequestAssetRegistryTick);

		if (bShouldRequestAssetRegistryTick)
		{
			TryTickOnBackgroundThread();
		}

		bool bLocalIdle = false;
		while (ShouldBackgroundGatherThreadPauseGathering(bLocalIdle))
		{
			UE::AssetRegistry::Impl::EGatherStatus Status = UE::AssetRegistry::Impl::EGatherStatus::Complete;
			if (bLocalIdle)
			{
				Status = TryTickOnBackgroundThread();
			}

			// TODO: Need IsGatheringPaused to be a condition variable so we avoid sleeping while waiting for it and then taking a long time to wake after it is unset.
			if ((Status != UE::AssetRegistry::Impl::EGatherStatus::TickActiveGatherActive)
				&& (Status != UE::AssetRegistry::Impl::EGatherStatus::TickActiveGatherIdle))
			{
				const bool bInitialSearchCompleted = bIsInitialSearchCompleted.load(std::memory_order_relaxed);
				const bool bAdditionalMountSearchInProgress = bIsAdditionalMountSearchInProgress.load(std::memory_order_relaxed);
				const bool bShouldLog = !bInitialSearchCompleted || bAdditionalMountSearchInProgress;

				TRACE_CPUPROFILER_EVENT_SCOPE_STR_CONDITIONAL("FAssetDataGatherer Sleep", bShouldLog);
				FPlatformProcess::Sleep(bLocalIdle ? IdleSleepTime : PausedSleepTime);
			}

			if (TickResult == ETickResult::PollDiscovery)
			{
				using namespace UE::AssetDataGather::Private;
				// The gatherer thread is waiting on results from the discovery thread, and we should sleep rather
				// than busy wait on it, to reduce contention. TODO: Change this to an event triggered by discovery
				// results, rather than sleeping for a fixed time interval.
				FPlatformProcess::Sleep(FGatherSettings::PollDiscoveryPeriodSeconds);
			}
		}
	}
	return 0;
}

FAssetDataGatherer::ETickResult FAssetDataGatherer::InnerTickLoop(bool bInSynchronousTick,
	bool bContributeToCacheSave, double EndTimeSeconds, bool* bShouldRequestAssetRegistryTick)
{
	using namespace UE::AssetDataGather::Private;
	ETickResult Result = ETickResult::KeepTicking;

	// Synchronous ticks during Wait contribute to saving of the async cache only if there is no dedicated async thread to do it
	// The dedicated async thread always contributes
	bContributeToCacheSave = !bInSynchronousTick || (IsSynchronous() && bContributeToCacheSave);

	bool bShouldSaveCache = false;
	TArray<TPair<FName, FDiskCachedAssetData*>> AssetsToSave;
	{
		CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
		FGathererScopeLock RunScopeLock(&TickLock);
		TGuardValue<bool> ScopeSynchronousTick(bSynchronousTick, bInSynchronousTick);
		TRACE_CPUPROFILER_EVENT_SCOPE(FAssetDataGatherer::Tick);
		double TickStartTime = FPlatformTime::Seconds();
		bool bPollDiscovery = true;
		double LastPollTimeSeconds = 0;

		for (;;)
		{
			ETickResult TickResult = TickInternal(TickStartTime, bPollDiscovery, bShouldRequestAssetRegistryTick);
			if (EndTimeSeconds > 0. && FPlatformTime::Seconds() > EndTimeSeconds)
			{
				Result = ETickResult::Interrupt;
				break;
			}
			if (IsStopped || (!bInSynchronousTick && IsGatheringPaused))
			{
				Result = ETickResult::Idle;
				break;
			}
			if (TickResult != ETickResult::KeepTicking && TickResult != ETickResult::PollDiscovery)
			{
				Result = TickResult; // Interrupt or Idle
				break;
			}
			double CurrentTimeSeconds = FPlatformTime::Seconds();
			if (bPollDiscovery)
			{
				LastPollTimeSeconds = CurrentTimeSeconds;
			}
			if (TickResult == ETickResult::KeepTicking)
			{
				// Poll discovery every so often to reduce super-linear costs
				float TimeSinceLastPoll = static_cast<float>(CurrentTimeSeconds - LastPollTimeSeconds);
				bPollDiscovery = TimeSinceLastPoll > FGatherSettings::PollDiscoveryPeriodSeconds;
			}
			else
			{
				// ETickResult::PollDiscovery
				if (!bPollDiscovery)
				{
					bPollDiscovery = true;
				}
				else
				{
					// We just polled discovery, don't poll it again because we want to avoid busy-spinning and
					// causing contention on the discovery critical section.
					// Report back to the caller that they should wait for the discovery thread before they tick again.
					Result = ETickResult::PollDiscovery;
					break;
				}
			}
		}
		if (TickStartTime >= 0.) // TickInternal might have updated CurrentSearchTime and cleared TickStartTime
		{
			CurrentSearchTime += FPlatformTime::Seconds() - TickStartTime;
			TickStartTime = 0.;
		}

		if (bContributeToCacheSave)
		{
			TryReserveSaveCache(bShouldSaveCache, AssetsToSave);
		}
	}
	if (bShouldSaveCache)
	{
		TrySaveCacheFile(AssetsToSave, true /*bRequestSaveOnFailure*/);
	}
	return Result;
}

bool FAssetDataGatherer::ShouldBackgroundGatherThreadPauseGathering(bool& bOutGathererIsIdle)
{
	FGathererScopeLock ResultsScopeLock(&ResultsLock); // bIsIdle requires the lock
	bOutGathererIsIdle = bIsIdle;
	return !IsStopped && !IsCacheSaveReady() && (IsGatheringPaused || bIsIdle);
}

UE::AssetRegistry::Impl::EGatherStatus FAssetDataGatherer::TryTickOnBackgroundThread()
{
	// TickOnBackgroundThread calls this->GetAndTrimSearchResults, which enters ResultsLock and might enter TickLock,
	// so assert that we are not already holding them.
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(TickLock);
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);

	UE::AssetRegistry::Impl::EGatherStatus Status = UE::AssetRegistry::Impl::EGatherStatus::Complete;
	if (!IsGatherOnGameThreadOnly() && (IsProcessingPaused.load(std::memory_order_relaxed) == 0))
	{
		IAssetRegistry& Registry = IAssetRegistry::GetChecked();
		Status = Cast<UAssetRegistryImpl>(&Registry)->TickOnBackgroundThread();
	}

	return Status;
}

bool FAssetDataGatherer::TrySaveCacheFile(const TArray<TPair<FName,FDiskCachedAssetData*>>& AssetsToSave, bool bRequestSaveOnFailure)
{
	using namespace UE::AssetDataGather::Private;
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("Save Cache")
	
	const int32 MaxShards = FPlatformMisc::NumberOfCores();
	constexpr int32 CacheShardAssetCount = FGatherSettings::CacheShardAssetCount;
	const int32 CacheShards = FMath::RoundUpToPowerOfTwo(FMath::Min((AssetsToSave.Num() + CacheShardAssetCount-1) / CacheShardAssetCount, MaxShards));
	const int32 ShardMask = CacheShards - 1;
	std::atomic<int64> TotalCacheSize{0};
	bool bSuccess = true;
	FString CorruptTagText;
	bool bCorruptTagDetected = false;

	TArray<TArray<TPair<FName, FDiskCachedAssetData*>>> DataPerShard;
	{
		// Hash package names by string rather than by name id so that cache shard is maintained across runs in case of interruption
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("Build Shards")
		DataPerShard.AddDefaulted(CacheShards);
		for (TArray<TPair<FName, FDiskCachedAssetData*>>& ShardData : DataPerShard)
		{
			ShardData.Reserve(AssetsToSave.Num() / CacheShards);	
		}
		TStringBuilder<256> Buffer;
		for (const TPair<FName, FDiskCachedAssetData*>& Entry : AssetsToSave)
		{
			Buffer.Reset();
			Buffer << Entry.Key;
			// Force to lower case to maintain ordering across runs where fname casing changes because of load order
			// Hash doesn't need to be persistent across utf-8 transition so no need to re-encode strings
			for (int32 i = 0; i < Buffer.Len(); ++i)
			{
				Buffer.GetData()[i] = FChar::ToLower(Buffer.GetData()[i]);
			}
			uint64 Hash = CityHash64((const char*)Buffer.GetData(), Buffer.Len() * sizeof(TCHAR));
			int32 Index = (Hash & ShardMask);
			DataPerShard[Index].Add(Entry);
			// UE-372209: Work-around a not-yet-diagnosed AssetRegistry corruption bug that is allowing empty strings
			// to be present in some AssetRegistry tag values. Delete the old cache and skip saving the cache if that
			// has occurred; we don't know the extent of the corruption.
			for (const FAssetData& AssetData : Entry.Value->AssetDataList)
			{
				if (bCorruptTagDetected)
				{
					break;
				}

				for (const TPair<FName, FAssetTagValueRef>& TagAndValue : AssetData.TagsAndValues)
				{
					if (TagAndValue.Value.GetStorageString().IsEmpty())
					{
						bCorruptTagDetected = true;
						CorruptTagText = FString::Printf(
							TEXT("AssetRegistry corruption detected: the TagValue on an AssetData is empty string, which is invalid. ")
							TEXT("The existing cache will be deleted and the new cache save will be skipped.")
							TEXT("\n\tAsset: '%s'")
							TEXT("\n\tKey: '%s', Value: '%s', IsSet: %s, IsFixed: %s"),
							*AssetData.ToSoftObjectPath().ToString(), *TagAndValue.Key.ToString(), *TagAndValue.Value.GetStorageString(),
							*LexToString(TagAndValue.Value.IsSet()), *LexToString(TagAndValue.Value.IsFixed()));
						break;
					}
				}
			}
		}
	}

	// If we recently saved or loaded the file then pause for 0.5 seconds before trying to save on
	// top of it, to avoid failure to be able to delete the file we just saved/loaded.
	double LocalLastCacheWriteTime;
	{
		FGathererScopeLock ResultsScopeLock(&ResultsLock);
		LocalLastCacheWriteTime = LastCacheWriteTime;
	}
	double CurrentTime = FPlatformTime::Seconds();
	constexpr float WaitTimeBeforeReopen = 0.5f;
	if (CurrentTime < LocalLastCacheWriteTime + WaitTimeBeforeReopen)
	{
		FPlatformProcess::Sleep(static_cast<float>(LocalLastCacheWriteTime + WaitTimeBeforeReopen - CurrentTime));
	}

#if CACHEFILE_SYSTEMWIDE_WRITELOCK
	// Attempt to get exclusive write access to the cache files. If we cannot, skip writing
	// and try again next tick. Failing to get the lock here means another process 
	// is currently writing so we avoid the possibility of never writing cache files 
	UE::FPlatformSystemWideMutex CacheFilesLock = TryLockSystemWideCacheFilesWrites();
	if (!CacheFilesLock.IsValid())
	{
		bSuccess = false;
		if (bRequestSaveOnFailure && !IsStopped.load(std::memory_order_relaxed))
		{
			RequestAsyncCacheSave();
		}
		// Intentionally fall-through to perform necessary cleanup
	}
	else
#endif
	{
		if (ensureMsgf(!bCorruptTagDetected, TEXT("%s"), *CorruptTagText))
		{
			IFileManager::Get().MakeDirectory(*GGatherSettings.GetCacheDirectory(), true);

			// Write each shard to a new uniquely-named binary file, then update its .ref.
			ParallelFor(CacheShards, [this, &TotalCacheSize, &DataPerShard](int32 Shard)
			{
				TArray<TPair<FName, FDiskCachedAssetData*>>& ShardData = DataPerShard[Shard];
				FString BinaryFilename = MakeShardBinaryFilename(GGatherSettings.GetCacheBaseFilename(), Shard);
				int64 CacheSize = SaveCacheFileInternal(BinaryFilename, ShardData);
				if (CacheSize > 0)
				{
					FString RefFilename = GetShardRefFilename(GGatherSettings.GetCacheBaseFilename(), Shard);
					if (WriteCacheRefFile(RefFilename, FPaths::GetCleanFilename(BinaryFilename)))
					{
						TotalCacheSize += CacheSize;
					}
					else
					{
						// .ref update failed; delete the orphaned binary so it doesn't accumulate.
						// The previous .ref still points to a valid (older) binary for this shard.
						IFileManager::Get().Delete(*BinaryFilename, false, false, true);
					}
				}
			}, EParallelForFlags::BackgroundPriority);

			UE_LOGF(LogAssetRegistry, Display, "Asset registry cache written as %.1f MiB to %ls_*.ref",
				static_cast<float>(TotalCacheSize) / 1024.f / 1024.f, *GGatherSettings.GetCacheBaseFilename());

			// Delete any excess .ref files if the number of shards was reduced
			{
				const FString CacheDirectory = FPaths::GetPath(GGatherSettings.GetCacheBaseFilename());
				TArray<FString> ExistingRefFiles;
				IFileManager::Get().FindFiles(ExistingRefFiles, *(GGatherSettings.GetCacheBaseFilename() + TEXT("_*.ref")), true, false);
				for (const FString& RefFilename : ExistingRefFiles)
				{
					FStringView BaseName = FPathViews::GetBaseFilename(FStringView(RefFilename));
					if (int32 Index; BaseName.FindLastChar('_', Index))
					{
						int32 ShardNum = 0;
						LexFromString(ShardNum, BaseName.Mid(Index + 1));
						if (ShardNum >= CacheShards)
						{
							IFileManager::Get().Delete(*(CacheDirectory / RefFilename), false, false, true);
						}
					}
				}
				// Old-style _N.bin files (including excess shards) are cleaned up by
				// CleanupOrphanedCacheFiles(PostWrite) which runs after the write lock is released.
			}
		}
		else
		{
			// Corruption detected, delete the cache files
			UE::AssetDataGather::Private::DeleteCacheFilesWithinLock();
			// Report this as a "successful" write, so that we stop attempting to write the cache files.
			bSuccess = true;
		}

		{
			FGathererScopeLock ResultsScopeLock(&ResultsLock);
			LastCacheWriteTime = FPlatformTime::Seconds();
		}
	}

	// Best-effort cleanup of orphaned binary files outside the write lock.
	CleanupOrphanedCacheFiles(GGatherSettings.GetCacheBaseFilename(), ECacheCleanupContext::PostWrite);

	FScopedGatheringPause ScopedPause(*this);
	FGathererScopeLock TickScopeLock(&TickLock);
	check(bIsSavingAsyncCache);
	bIsSavingAsyncCache = false;

	return bSuccess;
}

FAssetDataGatherer::FScopedGatheringPause::FScopedGatheringPause(const FAssetDataGatherer& InOwner)
	:Owner(InOwner)
{
	if (!Owner.IsSynchronous())
	{
		Owner.IsGatheringPaused++;
	}
}

FAssetDataGatherer::FScopedGatheringPause::~FScopedGatheringPause()
{
	if (!Owner.IsSynchronous())
	{
		check(Owner.IsGatheringPaused > 0)
		Owner.IsGatheringPaused--;
	}
}

void FAssetDataGatherer::Stop()
{
	Discovery->Stop();
	IsStopped++;
}

void FAssetDataGatherer::Exit()
{
}

bool FAssetDataGatherer::IsAsyncEnabled() const
{
	return bAsyncEnabled;
}

bool FAssetDataGatherer::IsSynchronous() const
{
	return Thread == nullptr;
}

void FAssetDataGatherer::EnsureCompletion()
{
	Discovery->EnsureCompletion();

	Stop();

	if (Thread != nullptr)
	{
		Thread->WaitForCompletion();
		delete Thread;
		Thread = nullptr;
	}
}

FAssetDataGatherer::ETickResult FAssetDataGatherer::TickInternal(double& TickStartTime, bool bPollDiscovery, bool* bShouldRequestAssetRegistryTick)
{
	using namespace UE::AssetDataGather::Private;
	using namespace UE::Tasks;

	typedef TInlineAllocator<FGatherSettings::ExpectedMaxBatchSize> FBatchInlineAllocator;

	CHECK_IS_LOCKED_CURRENT_THREAD(TickLock);
	TArray<FGatheredPathData, FBatchInlineAllocator> LocalFilesToSearch;
	TArray<TUniquePtr<FAssetData>, FBatchInlineAllocator> LocalAssetResults;
	TArray<TUniquePtr<FAssetData>, FBatchInlineAllocator> LocalAssetResultsForGameThread;
	TArray<FPackageDependencyData, FBatchInlineAllocator> LocalDependencyResults;
	TArray<FPackageDependencyData, FBatchInlineAllocator> LocalDependencyResultsForGameThread;
	TArray<FString, FBatchInlineAllocator> LocalCookedPackageNamesWithoutAssetDataResults;
	TArray<FName, FBatchInlineAllocator> LocalVerseResults;
	TArray<FString, FBatchInlineAllocator> LocalBlockedResults;
	TArray<FString> LocalPriorityDirectories;
	FWaitBatchDirectorySet LocalWaitBatchDirectories;
	bool bLoadCache = false;
	bool bLocalIsCacheWriteEnabled = false;
	double LocalLastCacheWriteTime = 0.0;
	ETickResult TickResult = ETickResult::KeepTicking;
	if (bShouldRequestAssetRegistryTick)
	{
		*bShouldRequestAssetRegistryTick = false;
	}

	{
		FGathererScopeLock ResultsScopeLock(&ResultsLock);

		if (bFirstTickAfterIdle)
		{
			bFirstTickAfterIdle = false;
			LastCacheWriteTime = FPlatformTime::Seconds();
		}

		if (bPollDiscovery)
		{
			IngestDiscoveryResults();
		}
		if (IsEngineStartupModuleLoadingComplete() && !bFlushedRetryFiles)
		{
			bFlushedRetryFiles = true;
			FilesToSearch->RetryLaterRetryFiles();
		}

		if (FilesToSearch->GetNumAvailable() == 0 && NumUncachedAssetFilesOutstanding == 0)
		{
			if (WaitBatchDirectories.IsSet())
			{
				WaitBatchDirectories.Reset(); // WaitBatchDirectories was set after we've already gathered all results, mark it completed
				TickResult = ETickResult::Interrupt;
			}

			if (bDiscoveryIsComplete)
			{
				if (TickResult == ETickResult::KeepTicking)
				{
					TickResult = ETickResult::Idle;
				}
				const bool bWasInitialDiscoveryFinished = bFinishedInitialDiscovery;
				SetIsIdle(true, TickStartTime);
			}
			else if (TickResult == ETickResult::KeepTicking)
			{
				TickResult = ETickResult::PollDiscovery;
			}

			return TickResult;
		}

		if (WaitBatchDirectories.IsSet())
		{
			if (WaitBatchDirectories->IsEmpty())
			{
				// We've finished executing the caller's requested batchcount and we have fulfilled our check-for-idle
				// contract above (and found we are not yet idle), so exit now without doing any further work.
				WaitBatchDirectories.Reset();
				return ETickResult::Interrupt;
			}

			// Otherwise we still have some work to do for the caller's requested batchcount, so do work up to
			// that batchcount.
			LocalWaitBatchDirectories = *WaitBatchDirectories;
		}
		DependencyResults.Reserve(FilesToSearch->GetNumAvailable() + DependencyResults.Num());
		FilesToSearch->PopFront(LocalFilesToSearch, FilesToSearch->GetNumAvailable());
		FilesToSearch->PopPriorityDirectories(LocalPriorityDirectories);

		if (bCacheReadEnabled && !bHasLoadedCache)
		{
			bLoadCache = true;
		}
		LocalLastCacheWriteTime = LastCacheWriteTime;
	}
	bLocalIsCacheWriteEnabled = bCacheWriteEnabled;

	// Load the async cache if not yet loaded
	if (bLoadCache)
	{
		double CacheLoadStartTime = FPlatformTime::Seconds();
		TArray<FCachePayload> Payloads;
		bool bGatherCacheWriteEnabled = false;
#if UE_EDITOR
		bGatherCacheWriteEnabled = GGatherSettings.IsGatherCacheWriteEnabled();
		if (GGatherSettings.IsPreloadGatherCache())
		{
			Payloads = GPreloader.Consume();
		}
		else
#endif
		{
			CleanupOrphanedCacheFiles(GGatherSettings.GetCacheBaseFilename(), ECacheCleanupContext::PreLoad);
			TArray<FString> CachePaths = GGatherSettings.FindShardedCacheFiles();
			if (CachePaths.IsEmpty())
			{
				CachePaths = GGatherSettings.FindLegacyShardedCacheFiles();
				if (CachePaths.Num())
				{
					UE_LOGF(LogAssetRegistry, Display, "Found %d legacy cache files for migration from %ls",
						CachePaths.Num(), *GGatherSettings.GetLegacyCacheBaseFilename());
				}
			}
			Payloads = LoadCacheFiles(CachePaths, bGatherCacheWriteEnabled);
		}
		ConsumeCacheFiles(MoveTemp(Payloads));
		UE_LOGF(LogAssetRegistry, Display, "AssetDataGatherer spent %.3fs loading caches from %ls.",
			FPlatformTime::Seconds() - CacheLoadStartTime, *GGatherSettings.GetCacheDirectory());

		FGathererScopeLock ResultsScopeLock(&ResultsLock);
		bHasLoadedCache = true;

		// After we load the cache, restart the write timer for it. We don't need to save to if we just
		// finished loading it (which we do before gathering anything) and we want to avoid failure to save due to
		// writing a file that we just closed a readhandle for.
		LastCacheWriteTime = FPlatformTime::Seconds();
		LocalLastCacheWriteTime = LastCacheWriteTime;
	}

	int32 NewCachedAssetFiles = 0;
	int32 NewUncachedAssetFiles = 0;
	int32 NewOutstandingReads = 0;
	int32 NumReadContextsToProcess = 0;

	// Try to read each file in the batch out of the cache, and accumulate a list for more expensive reading of all of the files that are not in the cache 
	for (FGatheredPathData& AssetFileData : LocalFilesToSearch)
	{
		// If this a Verse-related file, just directly add its file name to the Verse results
		if (UE::AssetDataGather::Private::IsVerseFile(AssetFileData.Type))
		{
			// Store Verse results in a hybrid format using the LongPackageName but keeping the extension
			LocalVerseResults.Emplace(WriteToString<256>(AssetFileData.LongPackageName, FPathViews::GetExtension(AssetFileData.LocalAbsPath, true)));
			continue;
		}

		if (AssetFileData.Type != EGatherableFileType::PackageFile)
		{
			ensureMsgf(false, TEXT("Encountered unrecognized gathered asset %s!"), *AssetFileData.LongPackageName);
			continue;
		}

		const FName PackageName = *AssetFileData.LongPackageName;

		FDiskCachedAssetData** DiskCachedAssetDataPtr = DiskCachedAssetDataMap.Find(PackageName);
		FDiskCachedAssetData* DiskCachedAssetData = DiskCachedAssetDataPtr ? *DiskCachedAssetDataPtr : nullptr;
		if (DiskCachedAssetData)
		{
			// Check whether we need to invalidate the cached data
			const FDateTime& CachedTimestamp = DiskCachedAssetData->ModificationTime;
			if (AssetFileData.PackageTimestamp != CachedTimestamp)
			{
				DiskCachedAssetData = nullptr;
			}
			else if ((!DiskCachedAssetData->DependencyData.PackageName.IsEqual(PackageName, ENameCase::CaseSensitive) && DiskCachedAssetData->DependencyData.PackageName != NAME_None) ||
				DiskCachedAssetData->Extension != FName(FPathViews::GetExtension(AssetFileData.LocalAbsPath)))
			{
				UE_LOGF(LogAssetRegistry, Display, "Cached dependency data for package '%ls' is invalid. Discarding cached data.", *PackageName.ToString());
				DiskCachedAssetData = nullptr;
			}
		}

		// Check for whether the cache data has the information we need to know to report whether the package is blocked
		if (bBlockPackagesWithMarkOfTheWeb && DiskCachedAssetData
			&& DiskCachedAssetData->HasMarkOfTheWeb == UE::AssetRegistry::Private::EOptionalBool::Unset)
		{
			DiskCachedAssetData = nullptr;
		}

		if (DiskCachedAssetData)
		{
			// If this is a blocked package, just directly add its file path to the Blocked results
			if (bBlockPackagesWithMarkOfTheWeb
				&& DiskCachedAssetData->HasMarkOfTheWeb == UE::AssetRegistry::Private::EOptionalBool::True)
			{
				// To avoid falsely blocked packages that have had the mark removed, always recalculate HasMarkOfTheWeb
				// before allowing it to block the package
				DiskCachedAssetData->HasMarkOfTheWeb = BoolToOptionalBool(
					IPlatformFile::GetPlatformPhysical().HasMarkOfTheWeb(AssetFileData.LocalAbsPath));
				if (DiskCachedAssetData->HasMarkOfTheWeb == UE::AssetRegistry::Private::EOptionalBool::True)
				{
					LocalBlockedResults.Add(AssetFileData.LocalAbsPath);
				}
				continue;
			}

			// Add the valid cached data to our results, and to the map of data we keep to write out the new version of the cache file
			++NewCachedAssetFiles;

			// Set the transient flags based on whether our current cache has dependency data.
			// Note that in editor, bGatherAssetPackageData is always true, no way to turn it off,
			// and in game it is always equal to bGatherDependsData, so it can share the cache with dependency data.
			DiskCachedAssetData->DependencyData.bHasPackageData = bGatherAssetPackageData;
			DiskCachedAssetData->DependencyData.bHasDependencyData = bGatherDependsData;

			bool MustBeHandledByGameThread = false;
			// 
			// In the future, we may need to process certain assets on the game thread because, e.g.,
			// we may need to support PostLoadAssetRegistryTags running on the game thread. 
			// The infrastructure is provided here to handle that case. In order to do so,
			// implement ClassRequiresGameThreadProcessing in UAssetRegistryImpl and call it as shown below
			// The rest of the functions in the asset registry respect the separation of data into general and 
			// ForGameThread containers. In particular, these are consumed in FAssetRegistryImpl::TickGatherer.
			// 
			//for (const FAssetData& AssetData : DiskCachedAssetData->AssetDataList)
			//{
			//	if (AssetRegistry.ClassRequiresGameThreadProcessing(AssetData.GetClass()))
			//	{
			//		MustBeHandledByGameThread = true;
			//		break;
			//	}
			//}

			TArray<TUniquePtr<FAssetData>, FBatchInlineAllocator>& TargetAssetResults = MustBeHandledByGameThread ? LocalAssetResultsForGameThread : LocalAssetResults;
			TArray<FPackageDependencyData, FBatchInlineAllocator>& TargetDependencyResults = MustBeHandledByGameThread ? LocalDependencyResultsForGameThread : LocalDependencyResults;

			TargetAssetResults.Reserve(TargetAssetResults.Num() + DiskCachedAssetData->AssetDataList.Num());
			for (const FAssetData& AssetData : DiskCachedAssetData->AssetDataList)
			{
				TargetAssetResults.Add(MakeUnique<FAssetData>(AssetData));
			}
			FPackageDependencyData& DependencyData = TargetDependencyResults.Add_GetRef(DiskCachedAssetData->DependencyData);
			DependencyData.PackageData.SetIsReadOnly(AssetFileData.bIsReadOnly);

			AddToCache(PackageName, DiskCachedAssetData);
		}
		else
		{
			// Not found in cache (or stale) - schedule to be read from disk
			FileReadScheduler->AddFile(PackageName, MoveTemp(AssetFileData));
			NewOutstandingReads++;
		}
	}

	// Wait on any files being explicitly waited on before scheduling other reads.
	// Cache if we are in a wait batch now since WaitOnDirectoriesAndUpdateWaitList will update
	// our wait list, and we don't want to perform a gather, even if we waited on nothing since 
	// the game thread is waiting for the tick to return.
	bool bInWaitBatch = !LocalWaitBatchDirectories.IsEmpty();
	FileReadScheduler->WaitOnDirectoriesAndUpdateWaitList(LocalWaitBatchDirectories);

	// Call Schedule to push any recently added uncached files to the read queue, and also give it a list of any new directory priorities we have
	FileReadScheduler->Schedule(LocalPriorityDirectories);

	// Collect any completed reads for processing on this thread
	TArray<TUniquePtr<FReadContext>> CompletedReadContexts;
	FileReadScheduler->CollectCompletedReads(CompletedReadContexts);

	// If we have finished any async reads outside of a wait batch, signal that we want to 
	// kick the gatherer to help avoid gather work from running on the game thread
	if (bShouldRequestAssetRegistryTick)
	{
		*bShouldRequestAssetRegistryTick = !bInWaitBatch && !CompletedReadContexts.IsEmpty();
	}

	// Accumulate the results
	for (TUniquePtr<FReadContext>& ReadContext : CompletedReadContexts)
	{
		if (ReadContext->bCanceled)
		{
			ReadContext->DirectoryReadTaskData->bHasCanceledRead = true;
		}
		else if (ReadContext->bResult)
		{
			// Do not add the results from a cooked package into the map of data we keep to write out the new version of the cache file
			bool bCachePackage = bLocalIsCacheWriteEnabled
				&& ReadContext->CookedPackageNamesWithoutAssetData.Num() == 0
				&& ensure(ReadContext->AssetFileData.Type == EGatherableFileType::PackageFile);
			if (bCachePackage)
			{
				for (const FAssetData* AssetData : ReadContext->AssetDataFromFile)
				{
					if (!!(AssetData->PackageFlags & PKG_FilterEditorOnly))
					{
						bCachePackage = false;
						break;
					}
				}
			}

			// Add the results from non-cooked packages into the map of data we keep to write out the new version of the cache file 
			if (bCachePackage)
			{
				// Update the cache
				FDiskCachedAssetData* NewData = new FDiskCachedAssetData(ReadContext->AssetFileData.PackageTimestamp, 
					GatherStartTime, ReadContext->Extension);
				NewData->AssetDataList.Reserve(ReadContext->AssetDataFromFile.Num());
				for (const FAssetData* BackgroundAssetData : ReadContext->AssetDataFromFile)
				{
					NewData->AssetDataList.Add(*BackgroundAssetData);
				}

				NewData->DependencyData = ReadContext->DependencyData;
				NewData->HasMarkOfTheWeb = ReadContext->HasMarkOfTheWeb;

				NewCachedAssetData.Add(NewData);
				AddToCache(ReadContext->PackageName, NewData);
			}

			bool bPackageBlocked = bBlockPackagesWithMarkOfTheWeb
				&& ReadContext->HasMarkOfTheWeb == UE::AssetRegistry::Private::EOptionalBool::True;
			if (bPackageBlocked)
			{
				// If this is a blocked package, add its file path to the Blocked results, and not the published result.
				// It still will have been cached above, so we don't have to read its data next time.
				LocalBlockedResults.Add(ReadContext->AssetFileData.LocalAbsPath);
			}
			else
			{
				++NewUncachedAssetFiles;
				// Add the results from a cooked package into our results on cooked package
				LocalCookedPackageNamesWithoutAssetDataResults.Append(MoveTemp(ReadContext->CookedPackageNamesWithoutAssetData));

				bool MustBeHandledByGameThread = false;
				// 
				// In the future, we may need to process certain assets on the game thread because, e.g.,
				// we may need to support PostLoadAssetRegistryTags running on the game thread. 
				// The infrastructure is provided here to handle that case. In order to do so,
				// implement ClassRequiresGameThreadProcessing in UAssetRegistryImpl and call it as shown below
				// The rest of the functions in the asset registry respect the separation of data into general and 
				// ForGameThread containers. In particular, these are consumed in FAssetRegistryImpl::TickGatherer.
				// 
				//for (const FAssetData* BackgroundAssetData : ReadContext.AssetDataFromFile)
				//{
				//	if (AssetRegistry.ClassRequiresGameThreadProcessing(BackgroundAssetData->GetClass()))
				//	{
				//		MustBeHandledByGameThread = true;
				//		break;
				//	}
				//}

				TArray<TUniquePtr<FAssetData>, FBatchInlineAllocator>& TargetAssetResults =
					MustBeHandledByGameThread ? LocalAssetResultsForGameThread : LocalAssetResults;
				TArray<FPackageDependencyData, FBatchInlineAllocator>& TargetDependencyResults =
					MustBeHandledByGameThread ? LocalDependencyResultsForGameThread : LocalDependencyResults;

				// Add the results from the package into our output results
				TargetAssetResults.Append(MoveTemp(ReadContext->AssetDataFromFile));
				FPackageDependencyData& DependencyData = TargetDependencyResults.Add_GetRef(MoveTemp(ReadContext->DependencyData));
				DependencyData.PackageData.SetIsReadOnly(ReadContext->AssetFileData.bIsReadOnly);
			}
		}
	}

	{
		FGathererScopeLock ResultsScopeLock(&ResultsLock);

		// Submit the results into the thread-shared lists
		AssetResults.Append(MoveTemp(LocalAssetResults));
		AssetResultsForGameThread.Append(MoveTemp(LocalAssetResultsForGameThread));
		DependencyResults.Append(MoveTemp(LocalDependencyResults));
		DependencyResultsForGameThread.Append(MoveTemp(LocalDependencyResultsForGameThread));
		CookedPackageNamesWithoutAssetDataResults.Append(MoveTemp(LocalCookedPackageNamesWithoutAssetDataResults));
		VerseResults.Append(MoveTemp(LocalVerseResults));
		BlockedResults.Append(MoveTemp(LocalBlockedResults));

		NumUncachedAssetFiles += NewUncachedAssetFiles;
		NumCachedAssetFiles += NewCachedAssetFiles;
		NumUncachedAssetFilesOutstanding += NewOutstandingReads - CompletedReadContexts.Num();

		// Pop all ready contexts so they can be destroyed and we can 
		// cleanup our DirectoryReadTaskData as necessary
		while (!CompletedReadContexts.IsEmpty())
		{
			TUniquePtr<FReadContext> ReadContext = CompletedReadContexts.Pop(EAllowShrinking::No);

			// This is the last read for the directory so we can now remove it from the Scheduler
			if (ReadContext->DirectoryReadTaskData->OutstandingReads == 1)
			{
				FString Directory = FPaths::GetPath(ReadContext->AssetFileData.LocalAbsPath);
				check(!Directory.IsEmpty());

				// Only remove directories from the wait batch when we know they were completed fully (i.e. 
				// didn't contain a canceled read that will need the directory to be rescheduled)
				if (bInWaitBatch && !ReadContext->DirectoryReadTaskData->bHasCanceledRead)
				{
					LocalWaitBatchDirectories.Remove(Directory);
				}
				FileReadScheduler->RemoveScheduledDirectory(MoveTemp(Directory));
			}

			if (ReadContext->bCanceled)
			{
				FilesToSearch->AddFileAgainAfterTimeout(MoveTemp(ReadContext->AssetFileData));
			}
			else if(ReadContext->bCanAttemptAssetRetry)
			{
				// If the read temporarily failed, return it to the worklist, pushed to the end
				FilesToSearch->AddFileForLaterRetry(MoveTemp(ReadContext->AssetFileData));
			}
		}

		if (bInWaitBatch)
		{
			// Update the wait batch, as we will have added and/or removed directories
			// based on possibly child directories outstanding and read completions
			WaitBatchDirectories = MoveTemp(LocalWaitBatchDirectories);
		}

		const int32 NumAssetsReadSinceLastCacheWrite = NumUncachedAssetFiles - LastCacheSaveNumUncachedAssetFiles;
		if (bCacheWriteEnabled && !bIsSavingAsyncCache 
			&& FPlatformTime::Seconds() - LocalLastCacheWriteTime >= FGatherSettings::MinSecondsToElapseBeforeCacheWrite
			&& NumAssetsReadSinceLastCacheWrite >= FGatherSettings::MinAssetReadsBeforeCacheWrite)
		{
			RequestAsyncCacheSave();
			TickResult = ETickResult::Interrupt;
		}
	}
	return TickResult;
}

void FAssetDataGatherer::IngestDiscoveryResults()
{
	CHECK_IS_LOCKED_CURRENT_THREAD(TickLock);
	CHECK_IS_LOCKED_CURRENT_THREAD(ResultsLock);
	Discovery->GetAndTrimSearchResults(bDiscoveryIsComplete, DiscoveredPaths, *FilesToSearch, NumPathsToSearchAtLastSyncPoint);
}

bool FAssetDataGatherer::ReadAssetFile(const FString& AssetLongPackageName, const FString& AssetFilename, TArray<FAssetData*>& AssetDataList, FPackageDependencyData& DependencyData, TArray<FString>& CookedPackageNamesWithoutAssetData, bool& OutCanRetry) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAssetDataGatherer::ReadAssetFile);
	OutCanRetry = false;
	AssetDataList.Reset();

	FPackageReader PackageReader;

	FPackageReader::EOpenPackageResult OpenPackageResult;
	if (!PackageReader.OpenPackageFile(AssetLongPackageName, AssetFilename, &OpenPackageResult))
	{
		// If we're missing a custom version, we might be able to load this package later once the module containing that version is loaded...
		//   -	Attempting a retry is only useful when engine startup module is not yet complete and therefore more plugins are expected.
		const bool bAllowRetry = !IsEngineStartupModuleLoadingComplete();
		if (OpenPackageResult == FPackageReader::EOpenPackageResult::CustomVersionMissing)
		{
			OutCanRetry = bAllowRetry;
			if (!bAllowRetry)
			{
				UE_LOGF(LogAssetRegistry, Display, "Package %ls uses an unknown custom version and cannot be loaded for the AssetRegistry", *AssetFilename);
			}
		}
		else
		{
			OutCanRetry = false;
		}
		return false;
	}
	else
	{
		return ReadAssetFile(PackageReader, AssetDataList, DependencyData, CookedPackageNamesWithoutAssetData,
			(bGatherAssetPackageData ? FPackageReader::EReadOptions::PackageData : FPackageReader::EReadOptions::None) |
			(bGatherDependsData ? FPackageReader::EReadOptions::Dependencies : FPackageReader::EReadOptions::None));
	}
}

bool FAssetDataGatherer::ReadAssetFile(FPackageReader& PackageReader, TArray<FAssetData*>& AssetDataList,
	FPackageDependencyData& DependencyData, TArray<FString>& CookedPackageNamesWithoutAssetData, FPackageReader::EReadOptions Options)
{
	bool bOutIsCookedWithoutAssetData;
	if (!PackageReader.ReadAssetRegistryData(AssetDataList, bOutIsCookedWithoutAssetData))
	{
		return false;
	}
	if (bOutIsCookedWithoutAssetData)
	{
		CookedPackageNamesWithoutAssetData.Add(PackageReader.GetLongPackageName());
	}

	if (!PackageReader.ReadDependencyData(DependencyData, Options))
	{
		return false;
	}

	if (PackageReader.UEVer() >= VER_UE4_ASSETREGISTRY_DEPENDENCYFLAGS && PackageReader.UEVer() < VER_UE4_CORRECT_LICENSEE_FLAG)
	{
		if (EnumHasAnyFlags(Options, FPackageReader::EReadOptions::Dependencies))
		{
			// In version VER_UE4_ASSETREGISTRY_DEPENDENCYFLAGS, UObjectRedirectors were incorrectly saved as having
			// editor-only imports, since UObjectRedirector is an editor-only class. But UObjectRedirectors are
			// followed during cooking and so their imports should be considered used-in-game. SavePackage was fixed
			// to save them as in-game imports by adding HasNonEditorOnlyReferences; the next version bump after that
			// fix was VER_UE4_CORRECT_LICENSEE_FLAG. Mark all dependencies in the affected version as used in game
			// if the package has a UObjectRedirector object.
			FTopLevelAssetPath RedirectorClassPathName = UE::AssetRegistry::GetClassPathObjectRedirector();
			if (Algo::AnyOf(AssetDataList, [RedirectorClassPathName](FAssetData* AssetData) { return AssetData->AssetClassPath == RedirectorClassPathName; }))
			{
				for (FPackageDependencyData::FPackageDependency& Dependency : DependencyData.PackageDependencies)
				{
					Dependency.Property |= UE::AssetRegistry::EDependencyProperty::Game;
				}
			}
		}
	}

	return true;
}

void FAssetDataGatherer::AddToCache(FName PackageName, FDiskCachedAssetData* DiskCachedAssetData)
{
	CHECK_IS_LOCKED_CURRENT_THREAD(TickLock);
	FDiskCachedAssetData*& ValueInMap = NewCachedAssetDataMap.FindOrAdd(PackageName, DiskCachedAssetData);
	if (ValueInMap != DiskCachedAssetData)
	{
		// An updated DiskCachedAssetData for the same package; replace the existing DiskCachedAssetData with the new one.
		// Note that memory management of the DiskCachedAssetData is handled in a separate structure; we do not need to delete the old value here.
		if (DiskCachedAssetData->Extension != ValueInMap->Extension)
		{
			// Two files with the same package name but different extensions, e.g. basename.umap and basename.uasset
			// This is invalid - some systems in the engine (Cooker's FPackageNameCache) assume that package : filename is 1 : 1 - so issue a warning
			// Because it is invalid, we don't fully support it here (our map is keyed only by packagename), and will remove from cache all but the last filename we find with the same packagename
			// TODO: Turn this into a warning once all sample projects have fixed it
			UE_LOGF(LogAssetRegistry, Display, "Multiple files exist with the same package name %ls but different extensions (%ls and %ls). " "This is invalid and will cause errors; merge or rename or delete one of the files.",
				*PackageName.ToString(), *ValueInMap->Extension.ToString(), *DiskCachedAssetData->Extension.ToString());
		}
		ValueInMap = DiskCachedAssetData;
	}
}

void FAssetDataGatherer::GetAndTrimSearchResults(UE::AssetDataGather::FResults& InOutResults,
	UE::AssetDataGather::FResultContext& OutContext)
{
	auto MoveAppendRangeToRingBuffer = [](auto& InOutRingBuffer, auto& InArray)
	{
		InOutRingBuffer.MoveAppendRange(InArray.GetData(), InArray.Num());
		InArray.Reset();
	};

	// GetPackageResults takes its own lock.
	GetPackageResults(InOutResults);

	bool bNeedsShrink = false;
	{
		FGathererScopeLock ResultsScopeLock(&ResultsLock);

		MoveAppendRangeToRingBuffer(InOutResults.Paths, DiscoveredPaths);


		MoveAppendRangeToRingBuffer(InOutResults.CookedPackageNamesWithoutAssetData, CookedPackageNamesWithoutAssetDataResults);
		MoveAppendRangeToRingBuffer(InOutResults.VerseFiles, VerseResults);

		InOutResults.BlockedFiles.Append(MoveTemp(BlockedResults));
		BlockedResults.Reset();

		OutContext.SearchTimes.Append(MoveTemp(SearchTimes));
		SearchTimes.Reset();

		OutContext.NumFilesToSearch = FilesToSearch->Num() + NumUncachedAssetFilesOutstanding;
		OutContext.NumPathsToSearch = NumPathsToSearchAtLastSyncPoint;
		OutContext.bIsDiscoveringFiles = !bDiscoveryIsComplete;

		// Idle means no more work OR we are blocked on external events, but complete means no more work period.
		bool bLocalIsComplete = bIsIdle && FilesToSearch->Num() == 0;
		if (bLocalIsComplete && !bIsComplete)
		{
			bIsComplete = true;
			bNeedsShrink = true;
		}
		OutContext.bIsSearching = !bLocalIsComplete;
		OutContext.bAbleToProgress = !bIsIdle;
	}

	if (bNeedsShrink)
	{
		CHECK_IS_NOT_LOCKED_CURRENT_THREAD(TickLock);
		CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
		FScopedGatheringPause ScopedPause(*this);
		FGathererScopeLock RunScopeLock(&TickLock);
		FGathererScopeLock ResultsScopeLock(&ResultsLock);
		Shrink();
	}

	if (GatheredResultsEvent.IsBound())
	{
		GatheredResultsEvent.Broadcast(InOutResults);
	}
}

FAssetGatherDiagnostics FAssetDataGatherer::GetDiagnostics()
{
	FAssetGatherDiagnostics Diag;
	Discovery->GetDiagnostics(Diag.DiscoveryTimeSeconds, Diag.NumCachedDirectories, Diag.NumUncachedDirectories);
	FGathererScopeLock ResultsScopeLock(&ResultsLock);
	Diag.GatherTimeSeconds = CumulativeGatherTime;
	Diag.NumCachedAssetFiles = NumCachedAssetFiles;
	Diag.NumUncachedAssetFiles = NumUncachedAssetFiles;
	Diag.WallTimeSeconds = static_cast<float>((FDateTime::Now() - GatherStartTime).GetTotalSeconds());
	return Diag;
}

void FAssetDataGatherer::GetPackageResults(UE::AssetDataGather::FResults& InOutResults)
{
	FGathererScopeLock ResultsScopeLock(&ResultsLock);

	for (TUniquePtr<FAssetData>& AssetData : AssetResults)
	{
		InOutResults.Assets.Add(AssetData->PackageName, MoveTemp(AssetData));
	}
	AssetResults.Reset();
	for (TUniquePtr<FAssetData>& AssetData : AssetResultsForGameThread)
	{
		InOutResults.AssetsForGameThread.Add(AssetData->PackageName, MoveTemp(AssetData));
	}
	AssetResultsForGameThread.Reset();
	for (FPackageDependencyData& DependencyData : DependencyResults)
	{
		FName PackageName = DependencyData.PackageName;
		InOutResults.Dependencies.Add(PackageName, MoveTemp(DependencyData));
	}
	DependencyResults.Reset();
	for (FPackageDependencyData& DependencyData : DependencyResultsForGameThread)
	{
		FName PackageName = DependencyData.PackageName;
		InOutResults.DependenciesForGameThread.Add(PackageName, MoveTemp(DependencyData));
	}
	DependencyResultsForGameThread.Reset();
}

void FAssetDataGatherer::WaitOnPath(FStringView InPath)
{
	{
		FGathererScopeLock ResultsScopeLock(&ResultsLock);
		if (bIsIdle)
		{
			return;
		}
	}
	FString LocalAbsPath = NormalizeLocalPath(InPath);
	UE::AssetDataGather::Private::FPathExistence QueryPath(LocalAbsPath);
	Discovery->SetPropertiesAndWait(TArrayView<UE::AssetDataGather::Private::FPathExistence>(&QueryPath, 1),
		false /* bAddToAllowList */, false /* bForceRescan */, false /* bIgnoreDenyListScanFilters */);
	WaitOnPathsInternal(TArrayView<UE::AssetDataGather::Private::FPathExistence>(&QueryPath, 1));
}

void FAssetDataGatherer::ClearCache()
{
	bool bOtherThreadIsSavingAsyncCache = false;
	bool bWasCacheEnabled = false;
	bool bShouldSaveCache = false;
	TArray<TPair<FName, FDiskCachedAssetData*>> AssetsToSave;
	{
		FGathererScopeLock TickScopeLock(&TickLock);
		// Must be done before we disable writing
		TryReserveSaveCache(bShouldSaveCache, AssetsToSave);

		bWasCacheEnabled = bCacheWriteEnabled || bCacheReadEnabled;
		bCacheWriteEnabled = false;
		bCacheReadEnabled = false;
		bOtherThreadIsSavingAsyncCache = bIsSavingAsyncCache && !bShouldSaveCache;
	}

	if (!bWasCacheEnabled)
	{
		return;
	}

	// Wait for any cache saves to complete because saves read the cache data we are about to delete.
	// Saves are executed outside of the lock, but they indicate they are in progress by setting bIsSavingAsyncCache=true.
	// bIsSavingAsyncCache cannot be set to true after bCacheWriteEnabled=false which we set above, so starvation should not be possible.
	while (bOtherThreadIsSavingAsyncCache)
	{
		constexpr float WaitForSaveCompleteTime = .001f;
		FPlatformProcess::Sleep(WaitForSaveCompleteTime);
		FGathererScopeLock TickScopeLock(&TickLock);
		bOtherThreadIsSavingAsyncCache = bIsSavingAsyncCache;
	}

	if (bShouldSaveCache)
	{
		// If this save fails, we don't re-attempt it and accept we couldn't write out the latest state
		TrySaveCacheFile(AssetsToSave);
	}

	{
		FGathererScopeLock TickScopeLock(&TickLock);
		// bIsSavingAsyncCache should now be false. We finished and set it to false if we were asked to save the cache,
		// and no other thread should be able to set it to true because we set bCacheWriteEnabled=false in the same critical
		// section that we used to test for bShouldSaveCache.,
		check(!bIsSavingAsyncCache);
		NewCachedAssetDataMap.Empty();
		DiskCachedAssetDataMap.Empty();

		for (FDiskCachedAssetData* AssetData : NewCachedAssetData)
		{
			delete AssetData;
		}
		NewCachedAssetData.Empty();
		for (TPair<int32, FDiskCachedAssetData*>& BlockData : DiskCachedAssetBlocks)
		{
			delete[] BlockData.Get<1>();
		}
		DiskCachedAssetBlocks.Empty();
	}

	// Best-effort cleanup of orphaned binaries. Files still memory-mapped by this
	// or another process will fail to delete and be cleaned up on next startup.
	UE::AssetDataGather::Private::CleanupOrphanedCacheFiles(
		UE::AssetDataGather::Private::GGatherSettings.GetCacheBaseFilename(),
		UE::AssetDataGather::Private::ECacheCleanupContext::PostWrite);
}

void FAssetDataGatherer::ScanPathsSynchronous(const TArray<FString>& InLocalPaths, bool bForceRescan,
	bool bIgnoreDenyListScanFilters)
{
	TArray<UE::AssetDataGather::Private::FPathExistence> QueryPaths;
	QueryPaths.Reserve(InLocalPaths.Num());
	for (const FString& LocalPath : InLocalPaths)
	{
		QueryPaths.Add(UE::AssetDataGather::Private::FPathExistence(NormalizeLocalPath(LocalPath)));
	}

	Discovery->SetPropertiesAndWait(QueryPaths, true /* bAddToAllowList */, bForceRescan, bIgnoreDenyListScanFilters);

	{
		FGathererScopeLock ResultsScopeLock(&ResultsLock);
		SetIsIdle(false);
	}

	WaitOnPathsInternal(QueryPaths);
}

void FAssetDataGatherer::WaitOnPathsInternal(TArrayView<UE::AssetDataGather::Private::FPathExistence> QueryPaths)
{
	LLM_SCOPE(ELLMTag::AssetRegistry);

	// Request a halt to the async tick
	FScopedGatheringPause ScopedPause(*this);
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
	{
		FGathererScopeLock TickScopeLock(&TickLock);

		// Read all results from Discovery into our worklist and then sort our worklist
		{
			FGathererScopeLock ResultsScopeLock(&ResultsLock);
			IngestDiscoveryResults();

			// Set WaitBatchCount to valid (set) but possibly to empty. If it is empty we still want to call TickInternal
			// to SetIsIdle(false) if necessary.
			if (!WaitBatchDirectories.IsSet())
			{
				WaitBatchDirectories.Emplace();
			}

			for (UE::AssetDataGather::Private::FPathExistence& PathExistence : QueryPaths)
			{
				using namespace UE::AssetDataGather::Private;
				FString WaitDir = PathExistence.GetLocalAbsPath();
				const bool bIsDirectory = PathExistence.GetType() == UE::AssetDataGather::Private::FPathExistence::Directory;
				if (!bIsDirectory)
				{
					FStringView DirectoryPath = FPathViews::GetPath(WaitDir);
					WaitDir.LeftInline(DirectoryPath.Len());
				}

				FWaitBatchDirectory* WaitBatchDirectory = WaitBatchDirectories->Find(WaitDir);
				if (WaitBatchDirectory)
				{
					// Ensure the directory is to be scanned recursively
					WaitBatchDirectory->bIsRecursive = bIsDirectory;
				}
				else
				{
					WaitBatchDirectories->Add(FWaitBatchDirectory(MoveTemp(WaitDir), bIsDirectory /*bIsRecursive*/));
				}
			}
		}
	}

	// Tick until NumDiscoveredPaths have been read
	TRACE_CPUPROFILER_EVENT_SCOPE(FAssetDataGatherer::Tick);
	for (;;)
	{
		InnerTickLoop(true /* bInSynchronousTick */, true /* bContributeToCacheSave */, -1. /* EndTimeSeconds */);
		FGathererScopeLock ResultsScopeLock(&ResultsLock); // WaitBatchCount requires the lock
		if (!WaitBatchDirectories.IsSet())
		{
			break;
		}
	}
}

void FAssetDataGatherer::WaitForIdle(float TimeoutSeconds)
{
	{
		FGathererScopeLock ResultsScopeLock(&ResultsLock);
		if (bIsIdle)
		{
			return;
		}
	}
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAssetDataGatherer::WaitForIdle");
	LLM_SCOPE(ELLMTag::AssetRegistry);

	double EndTimeSeconds = -1.;
	if (TimeoutSeconds >= 0.0f)
	{
		EndTimeSeconds = FPlatformTime::Seconds() + TimeoutSeconds;
	}
	if (Discovery->IsSynchronous())
	{
		Discovery->WaitForIdle(EndTimeSeconds);
		if (EndTimeSeconds > 0. && FPlatformTime::Seconds() > EndTimeSeconds)
		{
			return;
		}
	}

	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);

	// Request a halt to the async tick
	FScopedGatheringPause ScopedPause(*this);
	// Tick until idle
	for (;;)
	{
		InnerTickLoop(true /* bInSynchronousTick */, IsSynchronous() /* bContributeToCacheSave */, EndTimeSeconds);
		if (EndTimeSeconds > 0 && FPlatformTime::Seconds() > EndTimeSeconds)
		{
			break;
		}
		FGathererScopeLock ResultsScopeLock(&ResultsLock); // bIsIdle requires the lock
		if (bIsIdle)
		{
			// We need to break out of WaitForIdle whenever it requires main thread action to proceed,
			// so we check bIsIdle rather than whether we're complete
			break;
		}
	}
}

bool FAssetDataGatherer::IsComplete() const
{
	FGathererScopeLock ResultsScopeLock(&ResultsLock);
	return bIsComplete;
}

bool FAssetDataGatherer::HasSerializedDiscoveryCache() const
{
	using namespace UE::AssetDataGather::Private;
	FGathererScopeLock ResultsScopeLock(&ResultsLock);
	return bDiscoveryIsComplete && Discovery->IsCacheWriteEnabled();
}

void FAssetDataGatherer::SetInitialPluginsLoaded()
{
	FGathererScopeLock ResultsScopeLock(&ResultsLock);
	SetIsIdle(false);
}

bool FAssetDataGatherer::IsGatheringDependencies() const
{
	return bGatherDependsData;
}

bool FAssetDataGatherer::IsCacheReadEnabled() const
{
	return bCacheReadEnabled;
}

bool FAssetDataGatherer::IsCacheWriteEnabled() const
{
	return bCacheWriteEnabled;
}

void FAssetDataGatherer::ConsumeCacheFiles(TArray<UE::AssetDataGather::Private::FCachePayload> Payloads)
{
	CHECK_IS_LOCKED_CURRENT_THREAD(TickLock);
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
	
	int32 Count = 0;
	for (UE::AssetDataGather::Private::FCachePayload& Payload : Payloads)
	{
		Count += Payload.NumAssets;
	}

	if (Count == 0)
	{
		return;
	}

	DiskCachedAssetDataMap.Reserve(DiskCachedAssetDataMap.Num() + Count);
	for (UE::AssetDataGather::Private::FCachePayload& Payload : Payloads)
	{
		for (int32 AssetIndex = 0; AssetIndex < Payload.NumAssets; ++AssetIndex)
		{
			DiskCachedAssetDataMap.Add(*(Payload.PackageNames.Get() + AssetIndex),
				(Payload.AssetDatas.Get() + AssetIndex)); // -C6385
		}
		DiskCachedAssetBlocks.Emplace(Payload.NumAssets, Payload.AssetDatas.Release());
		Payload.Reset();
	}

	FGathererScopeLock ResultsScopeLock(&ResultsLock);
	DependencyResults.Reserve(DiskCachedAssetDataMap.Num());
	AssetResults.Reserve(DiskCachedAssetDataMap.Num());
}

bool FAssetDataGatherer::IsCacheSaveReady() const
{
	using namespace UE::AssetDataGather::Private;

	double SaveAsyncCacheStartTime = 0.0;
	if (bIsSavingAsyncCache || !IsCacheSaveRequested(&SaveAsyncCacheStartTime))
	{
		return false;
	}

	if (!IsStopped.load(std::memory_order_relaxed) && (FPlatformTime::Seconds() - SaveAsyncCacheStartTime) < FGatherSettings::MinSecondsToElapseBeforeCacheWrite)
	{
		// A delayed save request was made but we haven't waited long enough to start the save yet so we aren't ready.
		// However if we are stopped, ignore the timer and handle the save request immediately
		return false;
	}
	return true;
}

void FAssetDataGatherer::TryReserveSaveCache(bool& bOutShouldSave, TArray<TPair<FName,FDiskCachedAssetData*>>& AssetsToSave)
{
	bOutShouldSave = false;
	if (!IsCacheSaveReady())
	{
		return;
	}
	CHECK_IS_LOCKED_CURRENT_THREAD(TickLock);
	int32 LocalNumUncachedAssetFiles;
	{
		FGathererScopeLock ResultsScopeLock(&ResultsLock);
		bOutShouldSave = bCacheWriteEnabled;
		LocalNumUncachedAssetFiles = NumUncachedAssetFiles;
	}
	if (bOutShouldSave)
	{
		GetCacheAssetsToSave(AssetsToSave);
		bIsSavingAsyncCache = true;
		LastCacheSaveNumUncachedAssetFiles = LocalNumUncachedAssetFiles;
	}

	ClearRequestAsyncCacheSave();
}

void FAssetDataGatherer::GetCacheAssetsToSave(TArray<TPair<FName,FDiskCachedAssetData*>>& OutAssetsToSave)
{
	CHECK_IS_LOCKED_CURRENT_THREAD(TickLock);

	OutAssetsToSave.Reset();
	OutAssetsToSave.Reserve(FMath::Max(NewCachedAssetDataMap.Num(), DiskCachedAssetDataMap.Num()));
	for (const TPair<FName, FDiskCachedAssetData*>& Pair : NewCachedAssetDataMap)
	{
		OutAssetsToSave.Add(Pair);
	}

	for (const TPair<FName, FDiskCachedAssetData*>& Pair : DiskCachedAssetDataMap)
	{
		using namespace UE::AssetDataGather::Private;
		if (NewCachedAssetDataMap.Contains(Pair.Key))
		{
			continue; // Data was replaced when populating NewCachedAssetDataMap
		}	
		FTimespan Age = GatherStartTime - Pair.Value->LastGatheredTime;
		// Conservatively persist cached data until final save to avoid pruning the cache too much if discovery is interrupted 
		// when revisiting a workspace after some delay 
		if (!bFinishedInitialDiscovery || Age < FGatherSettings::CachePruneAge)
		{
			OutAssetsToSave.Add(Pair);
		}
	}
}

UE_TRACE_EVENT_BEGIN(Cpu, SaveCacheFile, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Path)
	UE_TRACE_EVENT_FIELD(int32, Count)
UE_TRACE_EVENT_END()

int64 FAssetDataGatherer::SaveCacheFileInternal(const FString& CacheFilename, const TArray<TPair<FName,FDiskCachedAssetData*>>& AssetsToSave)
{
	if (CacheFilename.IsEmpty())
	{
		return 0;
	}
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(TickLock);
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
#if CPUPROFILERTRACE_ENABLED
	UE_TRACE_LOG_SCOPED_T(Cpu, SaveCacheFile, CpuChannel)
		<< SaveCacheFile.Path(*CacheFilename)
		<< SaveCacheFile.Count(AssetsToSave.Num());
#endif // CPUPROFILERTRACE_ENABLED
	
	// Save to a temp file first, then move to the destination to avoid corruption
	FString CacheFilenameStr(CacheFilename);
	FString TempFilename = CacheFilenameStr + TEXT(".tmp");
	TUniquePtr<FArchive> FileAr(IFileManager::Get().CreateFileWriter(*TempFilename, 0));
	if (FileAr)
	{
		{
			UE::THashingArchiveProxy<FBlake3> FileArHash(*FileAr);

			using namespace UE::AssetDataGather::Private;
			uint64 CurrentVersion = FGatherSettings::CurrentVersion;
			FileArHash << CurrentVersion;

	#if ALLOW_NAME_BATCH_SAVING
			{
				// We might be able to reduce load time by using AssetRegistry::SerializationOptions
				// to save certain common tags as FName.
				FAssetRegistryWriter Ar(FAssetRegistryWriterOptions(), FileArHash);
				UE::AssetDataGather::Private::SerializeCacheSave(Ar, AssetsToSave);
			}
	#else		
			static_assert(false, "Cannot save asset registry cache in this configuration");
	#endif
			// write the integrity checksum
			FBlake3Hash Hash = FileArHash.GetHash();
			FileAr->Serialize(&Hash, sizeof(Hash));
		}

		int64 CacheSize = FileAr->TotalSize();
		// Close file handle before moving temp file to target 
		FileAr.Reset();
		IFileManager::Get().Move(*CacheFilenameStr, *TempFilename);
		return CacheSize;
	}
	else
	{
		UE_LOGF(LogAssetRegistry, Error, "Failed to open file for write %ls", *TempFilename);
		return 0;
	}
}

namespace UE::AssetDataGather::Private
{

void SerializeCacheSave(FAssetRegistryWriter& Ar, const TArray<TPair<FName,FDiskCachedAssetData*>>& AssetsToSave)
{
#if ALLOW_NAME_BATCH_SAVING
	double SerializeStartTime = FPlatformTime::Seconds();

	// serialize number of objects
	int32 LocalNumAssets = AssetsToSave.Num();
	Ar << LocalNumAssets;

	for (const TPair<FName, FDiskCachedAssetData*>& Pair : AssetsToSave)
	{
		FName AssetName = Pair.Key;
		Ar << AssetName;

		FDiskCachedAssetData* CachedData = Pair.Value;
		CachedData->SerializeForCache(Ar);
	}

	UE_LOGF(LogAssetRegistry, Verbose, "Asset data gatherer serialized in %0.6f seconds", FPlatformTime::Seconds() - SerializeStartTime);
#endif
}

FCachePayload SerializeCacheLoad(FAssetRegistryReader& Ar)
{
	double SerializeStartTime = FPlatformTime::Seconds();
	ON_SCOPE_EXIT
	{
		UE_LOGF(LogAssetRegistry, Verbose, "Asset data gatherer serialized in %0.6f seconds", FPlatformTime::Seconds() - SerializeStartTime);
	};

	// serialize number of objects
	int32 LocalNumAssets = 0;
	Ar << LocalNumAssets;

	const int32 MinAssetEntrySize = sizeof(int32);
	const int64 MaxPossibleNumAssets = (Ar.TotalSize() - Ar.Tell()) / MinAssetEntrySize;
	if (Ar.IsError() || LocalNumAssets < 0 || MaxPossibleNumAssets < LocalNumAssets)
	{
		Ar.SetError();
		return FCachePayload();
	}

	if (LocalNumAssets == 0)
	{
		FCachePayload Payload = FCachePayload();
		Payload.bSucceeded = true;
		return Payload;
	}

	FSoftObjectPathSerializationScope SerializationScope(NAME_None, NAME_None, ESoftObjectPathCollectType::NonPackage, ESoftObjectPathSerializeType::AlwaysSerialize);

	// allocate one single block for all asset data structs (to reduce tens of thousands of heap allocations)
	TUniquePtr<FName[]> PackageNameBlock(new FName[LocalNumAssets]);
	TUniquePtr<FDiskCachedAssetData[]> AssetDataBlock(new FDiskCachedAssetData[LocalNumAssets]);
	for (int32 AssetIndex = 0; AssetIndex < LocalNumAssets; ++AssetIndex)
	{
		// Load the name first to add the entry to the tmap below
		// Visual Studio Static Analyzer issues C6385 if we call Ar << PackageNameBlock[AssetIndex] or AssetDataBlock[AssetIndex].SerializeForCache
		Ar << *(PackageNameBlock.Get() + AssetIndex); // -C6385
		(AssetDataBlock.Get() + AssetIndex)->SerializeForCache(Ar); // -C6385
		if (Ar.IsError())
		{
			// There was an error reading the cache. Bail out.
			break;
		}
	}

	if (Ar.IsError())
	{
		return FCachePayload();
	}
	FCachePayload Result;
	Result.PackageNames = MoveTemp(PackageNameBlock);
	Result.AssetDatas = MoveTemp(AssetDataBlock);
	Result.NumAssets = LocalNumAssets;
	Result.bSucceeded = true;
	return Result;
}

TArray<FCachePayload> LoadCacheFiles(TConstArrayView<FString> InCacheFilenames, bool bCacheWriteEnabled)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LoadCacheFiles);
	TArray<FCachePayload> Results;
	if (UNLIKELY(InCacheFilenames.IsEmpty()))
	{
		return MoveTemp(Results);
	}
	Results.AddDefaulted(InCacheFilenames.Num());

	// Binary cache files are never overwritten (write-once, named with timestamp+suffix), so memory-mapping
	// them is safe even on builder machines. Only the escape hatch flag can disable it.
	static bool bNoAssetRegistryMemoryMapping = FParse::Param(FCommandLine::Get(), TEXT("NoAssetRegistryMemoryMapping"));
	if (bNoAssetRegistryMemoryMapping)
	{
		UE::AssetDataGather::Private::bMemoryMappedCacheFiles = false;
	}

	UE_LOGF(LogAssetRegistry, Display, "LoadCacheFiles: loading %d cache files, memory-mapping=%ls",
		InCacheFilenames.Num(),
		UE::AssetDataGather::Private::bMemoryMappedCacheFiles ? TEXT("enabled") : TEXT("disabled"));

	auto DoLoad = [](FArchive& FilemappedReader, TSharedPtr<UE::AssetRegistry::FMemoryMappedFile> BackingFile, FAssetRegistryVersion::Type VersionToRead = FAssetRegistryVersion::LatestVersion)
		{
			// We are already using AsyncTasks for our own ParallelFor, passing non-zero NumAsyncWorkers into
			// FAssetRegistryReader will cause our current task to block waiting for another task to run
			// the name batch loading. That can cause a deadlock due to running out of task threads
			// if we don't have a large number of threads, so disable the parallelism.
			constexpr int32 NumAsyncWorkers = 0;

			// The discovery cache always includes editor-only data.
			FAssetRegistryHeader Header;
			Header.Version = VersionToRead;
			Header.bFilterEditorOnlyData = false;

			FAssetRegistryReader RegistryReader(FilemappedReader,
				UE::AssetDataGather::Private::bMemoryMappedCacheFiles ? BackingFile : nullptr,
				FixedTagPrivate::FWeakStorePtr(), NumAsyncWorkers, Header);
			return RegistryReader.IsError() ? FCachePayload() : SerializeCacheLoad(RegistryReader);
		};

	auto VerifyIntegrity = [](FArchive* Ar)
		{
			// same size as in FMD5Hash::HashFileFromArchive
			constexpr int32 kLocalScratchSize = 64 * 1024;
			TArray<uint8> LocalScratch;
			LocalScratch.SetNumUninitialized(kLocalScratchSize);

			// we're going to hash all the file except the last N bytes which hold the hash we need to compare
			constexpr int32 kHashSize = sizeof(FBlake3Hash);
			const int64 SizeToHash = Ar->TotalSize() - kHashSize;

			if (SizeToHash > 0)
			{
				int64 Position = 0;
				FBlake3Hash ComputedHash = [&]()
					{
						THashingArchiveProxy<FBlake3> HashingAr(*Ar);
						// Read in BufferSize chunks
						while (Position < SizeToHash)
						{
							const auto ReadNum = FMath::Min(SizeToHash - Position, (int64)LocalScratch.Num());
							HashingAr.Serialize(LocalScratch.GetData(), ReadNum);

							Position += ReadNum;
						};

						return HashingAr.GetHash();
					}();

				int64 HashToRead = Ar->TotalSize() - Position;
				if (HashToRead == sizeof(ComputedHash))
				{
					FBlake3Hash ReadHash;
					Ar->Serialize(&ReadHash, sizeof(ReadHash));
					Ar->Seek(0);	// return to the beginning so if verification succeeds, we can proceed to read the file as usual
					return (ReadHash == ComputedHash);
				}
			}

			return false;
		};

	if (FPlatformProperties::SupportsMemoryMappedFiles())
	{
		const int32 NumTasks = InCacheFilenames.Num();
		const int32 MaxNumThreads = FGatherSettings::GARGatherCacheParallelism;
		const int32 MinBatchSize = FMath::DivideAndRoundUp(NumTasks, MaxNumThreads);

		ParallelFor(TEXT("AssetDataGatherer::LoadCacheFiles"), NumTasks, MinBatchSize,
			[&DoLoad, &InCacheFilenames, &Results, &VerifyIntegrity](int32 Index)
		{
			const FString& CacheFilename = InCacheFilenames[Index];

			FCachePayload Payload;
			bool bVerifiedOk = false;

			// First, verify the file and refuse to read if hashes do not agree
			// We do it before memory mapping the file to avoid bringing into the process memory areas of the file
			// that we do not expect to touch, and would have to explicitly evict later.
			TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(*CacheFilename));
			if (Ar)
			{
				bVerifiedOk = VerifyIntegrity(Ar.Get());
			}

			// now read the file if it is valid
			if (bVerifiedOk)
			{
				TSharedPtr<UE::AssetRegistry::FMemoryMappedFile> File = MakeShared<UE::AssetRegistry::FMemoryMappedFile>(*CacheFilename);

				// we already know (from the verification) that Ar is non-empty
				uint64 Version;
				*Ar << Version;

				if (Version == FGatherSettings::CurrentVersion)
				{
					Payload = DoLoad(*Ar, File);
					UE_CLOGF(Payload.bSucceeded, LogAssetRegistry, Display,
						"Asset registry cache read as %.1f MiB from %ls (memory-mapped=%ls)",
						static_cast<float>(Ar->TotalSize()) / 1024.f / 1024.f, *CacheFilename,
						UE::AssetDataGather::Private::bMemoryMappedCacheFiles ? TEXT("yes") : TEXT("no"));
					UE_CLOGF(!Payload.bSucceeded, LogAssetRegistry, Warning,
						"There was an error loading the asset registry cache from %ls.",
						*CacheFilename);
				}
				else
				{
					// We do not support backwards compatibility after FAssetRegistryVersion::MemoryMappedTagDataStore
					UE_LOGF(LogAssetRegistry, Display,
						"Asset registry older version (%" UINT64_FMT ") cache is too old to read; discarding %ls and regathering its data.",
						Version, *CacheFilename);
				}
			}
			else // check if it was a past version
			{
				Ar.Reset();

				TSharedPtr<UE::AssetRegistry::FMemoryMappedFile> File = MakeShared<UE::AssetRegistry::FMemoryMappedFile>(*CacheFilename);
				if (File)
				{
					AssetRegistryBackwardCompat::FMemoryViewReader FileReader(File->GetWholeFileView());
					TOptional<uint64> VersionOpt = FileReader.TryLoad<uint64>();
					// all later versions should have a valid integrity hash, so if we're here with a later version archive, it is genuinely corrupted and this is not a compat issue.
					if (VersionOpt.IsSet() && static_cast<FAssetRegistryVersion::Type>(VersionOpt.GetValue()) < FAssetRegistryVersion::MemoryMappedTagDataStore)
					{
						FAssetRegistryVersion::Type Version = static_cast<FAssetRegistryVersion::Type>(VersionOpt.GetValue());
						// For a cooked editor, we don't support any older caches
						#if UE_EDITOR && !UE_IS_COOKED_EDITOR
						// For uncooked editor, we only support backwards compatibility for a single version
						if (Version >= FAssetRegistryVersion::ManageDependenciesCookRule)
						{
							AssetRegistryBackwardCompat::FChecksumViewReader ChecksummingReader(MoveTemp(FileReader), *CacheFilename);
							Payload = DoLoad(ChecksummingReader, nullptr, Version);	// cannot pass the mmaped file, as we cannot map the data in old format
							UE_CLOGF(Payload.bSucceeded, LogAssetRegistry, Display,
								"Asset registry older version (%d) cache read as %.1f MiB from %ls (memory-mapped=no, old format).",
								Version,
								static_cast<float>(FileReader.GetTotalSize()) / 1024.f / 1024.f, *CacheFilename);
							UE_CLOGF(!Payload.bSucceeded, LogAssetRegistry, Warning,
								"There was an error loading the asset registry cache in old format (version %d) from %ls.",
								Version,
								*CacheFilename);
						}
						else
						#endif // UE_EDITOR && !UE_IS_COOKED_EDITOR
						{
							UE_LOGF(LogAssetRegistry, Display,
								"Asset registry older version (%d) cache is too old to read; discarding %ls and regathering its data.",
								Version, *CacheFilename);
						}
					}
					else
					{
						UE_LOGF(LogAssetRegistry, Warning, "Corrupted AssetRegistry cache file %ls",
							*CacheFilename
						);
					}
				}
			}
			Results[Index] = MoveTemp(Payload);

		}, EParallelForFlags::BackgroundPriority);
	}
	else
	{
		ParallelFor(InCacheFilenames.Num(), [&DoLoad, &InCacheFilenames, &Results, &VerifyIntegrity](int32 Index)
		{
			const FString& CacheFilename = InCacheFilenames[Index];

			FCachePayload Payload;
			TUniquePtr<FArchive> FileAr(IFileManager::Get().CreateFileReader(*CacheFilename, FILEREAD_Silent));
			// old archive versions (predating the hash) may fail verification, but this path only supports current version anyway
			if (FileAr && !FileAr->IsError() && FileAr->TotalSize() > sizeof(uint64) && VerifyIntegrity(FileAr.Get()))
			{
				uint64 Version = 0;
				*FileAr << Version;
				if (Version == FGatherSettings::CurrentVersion)
				{
					Payload = DoLoad(*FileAr, nullptr);
					UE_CLOGF(Payload.bSucceeded, LogAssetRegistry, Display,
						"Asset registry cache read as %.1f MiB from %ls (memory-mapped=no, platform unsupported)",
						static_cast<float>(FileAr->TotalSize()) / 1024.f / 1024.f, *CacheFilename);
					UE_CLOGF(!Payload.bSucceeded, LogAssetRegistry, Warning,
						"There was an error loading the asset registry cache from %ls.",
						*CacheFilename);
				}
				else
				{
					UE_LOGF(LogAssetRegistry, Display,
						"Asset registry older version (%" UINT64_FMT ") cache is too old to read; discarding %ls and regathering its data.",
						Version, *CacheFilename);
				}
			}

			Results[Index] = MoveTemp(Payload);
		}, EParallelForFlags::BackgroundPriority);
	}

	return MoveTemp(Results);
}

void DeleteCacheFilesWithinLock()
{
	// Delete binary files discovered via .ref indirection and old-style migration fallback
	TArray<FString> CacheFiles = GGatherSettings.FindShardedCacheFiles();
	for (const FString& CacheFile : CacheFiles)
	{
		IFileManager::Get().Delete(*CacheFile);
	}

	// Also delete all .ref files and any orphaned binaries not reachable via FindShardedCacheFiles
	const FString CacheBase = GGatherSettings.GetCacheBaseFilename();
	const FString Directory = FPaths::GetPath(CacheBase);

	TArray<FString> RefFiles;
	IFileManager::Get().FindFiles(RefFiles, *(CacheBase + TEXT("_*.ref")), true, false);
	for (const FString& RefFile : RefFiles)
	{
		IFileManager::Get().Delete(*(Directory / RefFile));
	}

	TArray<FString> AllBinaries;
	IFileManager::Get().FindFiles(AllBinaries, *(CacheBase + TEXT("_*.bin")), true, false);
	for (const FString& BinaryFile : AllBinaries)
	{
		IFileManager::Get().Delete(*(Directory / BinaryFile));
	}

	// Also delete legacy location files (Intermediate/CachedAssetRegistry_N.bin)
	TArray<FString> LegacyCacheFiles = GGatherSettings.FindLegacyShardedCacheFiles();
	for (const FString& LegacyCacheFile : LegacyCacheFiles)
	{
		IFileManager::Get().Delete(*LegacyCacheFile);
	}
}

} // namespace UE::AssetDataGather::Private

SIZE_T FAssetDataGatherer::GetAllocatedSize() const
{
	using namespace UE::AssetDataGather::Private;
	auto GetArrayRecursiveAllocatedSize = [](auto Container)
	{
		SIZE_T Result = Container.GetAllocatedSize();
		for (const auto& Value : Container)
		{
			Result += Value.GetAllocatedSize();
		}
		return Result;
	};

	SIZE_T Result = 0;
	if (Thread)
	{
		// TODO: Add size of Thread->GetAllocatedSize()
		Result += sizeof(*Thread);
	}

	Result += sizeof(*Discovery) + Discovery->GetAllocatedSize();

	FScopedGatheringPause ScopedPause(*this);
	CHECK_IS_NOT_LOCKED_CURRENT_THREAD(ResultsLock);
	FGathererScopeLock TickScopeLock(&TickLock);
	FGathererScopeLock ResultsScopeLock(&ResultsLock);

	Result += sizeof(*FilesToSearch) + FilesToSearch->GetAllocatedSize();

	Result += AssetResults.GetAllocatedSize();
	FAssetDataTagMapSharedView::FMemoryCounter TagMemoryUsage;
	for (const TUniquePtr<FAssetData>& Value : AssetResults)
	{
		Result += sizeof(*Value);
		TagMemoryUsage.Include(Value->TagsAndValues);
	}
	Result += FAssetData::GetChunkArrayRegistryAllocatedSize();
	Result += TagMemoryUsage.GetFixedSize() + TagMemoryUsage.GetLooseSize();

	Result += GetArrayRecursiveAllocatedSize(DependencyResults);
	Result += GetArrayRecursiveAllocatedSize(CookedPackageNamesWithoutAssetDataResults);
	Result += VerseResults.GetAllocatedSize();
	Result += BlockedResults.GetAllocatedSize();
	Result += SearchTimes.GetAllocatedSize();
	Result += GetArrayRecursiveAllocatedSize(DiscoveredPaths);
	Result += GGatherSettings.GetCacheBaseFilename().GetAllocatedSize();

	Result += NewCachedAssetData.GetAllocatedSize();
	for (const FDiskCachedAssetData* Value : NewCachedAssetData)
	{
		Result += sizeof(*Value);
		Result += Value->GetAllocatedSize();
	}
	Result += DiskCachedAssetBlocks.GetAllocatedSize();
	for (const TPair<int32, FDiskCachedAssetData*>& ArrayData : DiskCachedAssetBlocks)
	{
		Result += ArrayData.Get<0>() * sizeof(FDiskCachedAssetData);
	}
	Result += DiskCachedAssetDataMap.GetAllocatedSize();
	Result += NewCachedAssetDataMap.GetAllocatedSize();

	return Result;
}

void FAssetDataGatherer::Shrink()
{
	CHECK_IS_LOCKED_CURRENT_THREAD(TickLock);
	CHECK_IS_LOCKED_CURRENT_THREAD(ResultsLock);
	FilesToSearch->Shrink();
	AssetResults.Shrink();
	DependencyResults.Shrink();
	CookedPackageNamesWithoutAssetDataResults.Shrink();
	VerseResults.Shrink();
	BlockedResults.Shrink();
	SearchTimes.Shrink();
	DiscoveredPaths.Shrink();
	FileReadScheduler->Shrink();
}

void FAssetDataGatherer::AddMountPoint(FStringView LocalPath, FStringView LongPackageName)
{
	bool bAlreadyExisted = false;
	Discovery->AddMountPoint(NormalizeLocalPath(LocalPath), NormalizeLongPackageName(LongPackageName), bAlreadyExisted);

	if (!bAlreadyExisted)
	{
		FGathererScopeLock ResultsScopeLock(&ResultsLock);
		SetIsIdle(false);
	}
}

void FAssetDataGatherer::RemoveMountPoint(FStringView LocalPath)
{
	Discovery->RemoveMountPoint(NormalizeLocalPath(LocalPath));
}

void FAssetDataGatherer::AddRequiredMountPoints(TArrayView<FString> LocalPaths)
{
	TStringBuilder<128> MountPackageName;
	TStringBuilder<128> MountFilePath;
	TStringBuilder<128> RelPath;
	bool bAllExisted = true;
	for (const FString& LocalPath : LocalPaths)
	{
		if (FPackageName::TryGetMountPointForPath(LocalPath, MountPackageName, MountFilePath, RelPath))
		{
			bool bAlreadyExisted = false;
			Discovery->AddMountPoint(NormalizeLocalPath(MountFilePath), NormalizeLongPackageName(MountPackageName), bAlreadyExisted);
			bAllExisted = bAllExisted && bAlreadyExisted;
		}
	}

	if (!bAllExisted)
	{
		FGathererScopeLock ResultsScopeLock(&ResultsLock);
		SetIsIdle(false);
	}
}

void FAssetDataGatherer::OnDirectoryCreated(FStringView LocalPath)
{
	Discovery->OnDirectoryCreated(NormalizeLocalPath(LocalPath));
	{
		FGathererScopeLock ResultsScopeLock(&ResultsLock);
		SetIsIdle(false);
	}
}

void FAssetDataGatherer::OnFilesCreated(TConstArrayView<FString> LocalPaths)
{
	TArray<FString> LocalAbsPaths;
	LocalAbsPaths.Reserve(LocalPaths.Num());
	for (const FString& LocalPath : LocalPaths)
	{
		LocalAbsPaths.Add(NormalizeLocalPath(LocalPath));
	}
	Discovery->OnFilesCreated(LocalAbsPaths);
	{
		FGathererScopeLock ResultsScopeLock(&ResultsLock);
		SetIsIdle(false);
	}
}

void FAssetDataGatherer::PrioritizeSearchPath(const FString& PathToPrioritize)
{
	using namespace UE::AssetDataGather::Private;

	FString LocalFilenamePathToPrioritize;
	if (FPackageName::TryConvertLongPackageNameToFilename(PathToPrioritize, LocalFilenamePathToPrioritize))
	{
		LocalFilenamePathToPrioritize = NormalizeLocalPath(LocalFilenamePathToPrioritize);
		if (LocalFilenamePathToPrioritize.Len() == 0)
		{
			return;
		}
		EPriority Priority = EPriority::High;
		Discovery->PrioritizeSearchPath(LocalFilenamePathToPrioritize, Priority);

		{
			FGathererScopeLock ResultsScopeLock(&ResultsLock);
			SetIsIdle(false);
			int32 NumPrioritizedPaths;
			UE::AssetDataGather::Private::FPathExistence QueryPath(LocalFilenamePathToPrioritize);
			SortPathsByPriority(TArrayView<UE::AssetDataGather::Private::FPathExistence>(&QueryPath, 1),
				Priority, NumPrioritizedPaths);
		}
	}
}

void FAssetDataGatherer::SetDirectoryProperties(FStringView LocalPath, const UE::AssetDataGather::Private::FSetPathProperties& InProperties)
{
	FString LocalAbsPath = NormalizeLocalPath(LocalPath);
	if (LocalAbsPath.Len() == 0)
	{
		return;
	}

	Discovery->TrySetDirectoryProperties(LocalAbsPath, InProperties, false);

	{
		FGathererScopeLock ResultsScopeLock(&ResultsLock);
		SetIsIdle(false);
	}
}

void FAssetDataGatherer::SortPathsByPriority(
	TArrayView<UE::AssetDataGather::Private::FPathExistence> LocalAbsPathsToPrioritize,
	UE::AssetDataGather::Private::EPriority Priority, int32& OutNumPaths)
{
	CHECK_IS_LOCKED_CURRENT_THREAD(ResultsLock);
	using namespace UE::AssetDataGather::Private;

	for (UE::AssetDataGather::Private::FPathExistence& QueryPath : LocalAbsPathsToPrioritize)
	{
		FilesToSearch->PrioritizePath(QueryPath, Priority);
	}
	OutNumPaths = FilesToSearch->NumBlockingFiles();
}

void FAssetDataGatherer::SetIsOnAllowList(FStringView LocalPath, bool bIsAllowed)
{
	using namespace UE::AssetDataGather::Private;

	FSetPathProperties Properties;
	Properties.IsOnAllowList = bIsAllowed;
	SetDirectoryProperties(LocalPath, Properties);
}

bool FAssetDataGatherer::IsOnAllowList(FStringView LocalPath) const
{
	return Discovery->IsOnAllowList(NormalizeLocalPath(LocalPath));
}

bool FAssetDataGatherer::IsOnDenyList(FStringView LocalPath) const
{
	return Discovery->IsOnDenyList(NormalizeLocalPath(LocalPath));
}

bool FAssetDataGatherer::IsMonitored(FStringView LocalPath) const
{
	return Discovery->IsMonitored(NormalizeLocalPath(LocalPath));
}

static const TCHAR* VerseExtensions[] = {TEXT(".verse"), TEXT(".vmodule")};
// NOTE: If you want this to check against Verse naming conventions for filenames, this isn't what you want.
// Call `UE::AssetDataGather::Private::IsVerseFile` instead.
bool FAssetDataGatherer::IsVerseFile(FStringView FilePath)
{
	for (const TCHAR* Extension : VerseExtensions)
	{
		if (FilePath.EndsWith(Extension))
		{
			return true;
		}
	}
	return false;
}

TConstArrayView<const TCHAR*> FAssetDataGatherer::GetVerseFileExtensions()
{
	return TConstArrayView<const TCHAR*>(VerseExtensions, UE_ARRAY_COUNT(VerseExtensions));
}

void FAssetDataGatherer::SetIsIdle(bool bInIsIdle)
{
	double TickStartTime = -1.;
	SetIsIdle(bInIsIdle, TickStartTime);
}

void FAssetDataGatherer::SetIsIdle(bool bInIsIdle, double& TickStartTime)
{
	CHECK_IS_LOCKED_CURRENT_THREAD(ResultsLock);
	if (bInIsIdle == bIsIdle)
	{
		return;
	}

	bIsIdle = bInIsIdle;
	if (bIsIdle)
	{
		// bIsComplete will be set in GetAndTrimSearchResults
		if (TickStartTime >= 0.)
		{
			CurrentSearchTime += FPlatformTime::Seconds() - TickStartTime;
			TickStartTime = -1.;
		}
		// Finishing the initial discovery is blocked until IsEngineStartupModuleLoadingComplete because plugins can
		// be mounted during startup up until that point, and we need to wait for all the plugins that will load
		// before declaring completion.
		if (!bFinishedInitialDiscovery && IsEngineStartupModuleLoadingComplete())
		{
			bFinishedInitialDiscovery = true;

			UE_LOGF(LogAssetRegistry, Verbose, "Initial scan took %0.6f seconds (found %d cached assets, and loaded %d)",
				(float)CurrentSearchTime, NumCachedAssetFiles, NumUncachedAssetFiles);
		}
		SearchTimes.Add(CurrentSearchTime);
		CumulativeGatherTime += static_cast<float>(CurrentSearchTime);
		CurrentSearchTime = 0.;
	}
	else
	{
		bIsComplete = false;
		bDiscoveryIsComplete = false;
		bFirstTickAfterIdle = true;
	}
}

FString FAssetDataGatherer::NormalizeLocalPath(FStringView LocalPath)
{
	FString LocalAbsPath(LocalPath);
	LocalAbsPath = FPaths::ConvertRelativePathToFull(MoveTemp(LocalAbsPath));
	while (FPathViews::HasRedundantTerminatingSeparator(LocalAbsPath))
	{
		LocalAbsPath.LeftChopInline(1);
	}
	return LocalAbsPath;
}

FStringView FAssetDataGatherer::NormalizeLongPackageName(FStringView LongPackageName)
{
	// Conform LongPackageName to our internal format, which does not have a terminating redundant /
	if (LongPackageName.EndsWith(TEXT('/')))
	{
		LongPackageName = LongPackageName.LeftChop(1);
	}
	return LongPackageName;
}

namespace UE::AssetDataGather::Private
{

FFileReadScheduler::FFileReadScheduler(int32 InNumReadThreads, bool InGatherAssetPackageData, bool InGatherDependsData)
	: ReadConcurrencyLimiter(InNumReadThreads, UE::Tasks::ETaskPriority::BackgroundLow)
	, NumReadTaskThreads(InNumReadThreads)
	, bGatherAssetPackageData(InGatherAssetPackageData)
	, bGatherDependsData(InGatherDependsData)
{
	ScheduledDirectories = MakeUnique<TDirectoryTree<TSharedPtr<FDirectoryReadTaskData>>>();
}

FFileReadScheduler::~FFileReadScheduler()
{
	// Ensure all read tasks are complete
	ReadConcurrencyLimiter.Wait();

	// Free all allocated read buffers
	FreeReadTaskLoaders();
}

void FFileReadScheduler::FreeReadTaskLoaders()
{
	TArray<FNonBufferingReadOnlyArchive*> Loaders;
	ReadTaskLoaderFreeList.PopAll(Loaders);
	for (FNonBufferingReadOnlyArchive* Loader : Loaders)
	{
		delete Loader;
	}
}

void FFileReadScheduler::ProcessReadContexts(TArray<TUniquePtr<UE::AssetDataGather::Private::FReadContext>>&& ReadContexts, bool bIsTimeSliced)
{
	using namespace UE::AssetDataGather::Private;
	using namespace UE::Tasks;

	TRACE_CPUPROFILER_EVENT_SCOPE(FAssetDataGatherer::ReadDirectoryAssetFiles);

	// We only use a non-buffering loader if we fully control how archive loading will be done from disk.
	bool bIsPhysicalPlatformFile = &IPlatformFile::GetPlatformPhysical() == &FPlatformFileManager::Get().GetPlatformFile();
	double StartTime = FPlatformTime::Seconds();
	const double TimeLimit = 1.0;
	int32 MinBatchSize = 32;
	int32 Iteration = 0;
	
	FNonBufferingReadOnlyArchive* Loader = nullptr;
	if (bIsPhysicalPlatformFile)
	{
		Loader = ReadTaskLoaderFreeList.Pop();
		if (!Loader)
		{
			const int64 BufferSize = 1024 * 1024;
			Loader = new FNonBufferingReadOnlyArchive(BufferSize);
		}
	}

	for (int32 i = 0; i < ReadContexts.Num(); ++i)
	{
		if (bIsTimeSliced && Iteration++ > MinBatchSize)
		{
			if ((FPlatformTime::Seconds() - StartTime) >= TimeLimit)
			{
				for (int32 j = i; j < ReadContexts.Num(); ++j)
				{
					TUniquePtr<FReadContext>& ReadContext = ReadContexts[j];
					ReadContext->bCanceled = true;
					ReadyReadContexts.Enqueue(MoveTemp(ReadContext));
				}
				break;
			}
			Iteration = 0;
		}

		TUniquePtr<FReadContext>& ReadContext = ReadContexts[i];
		FPackageReader::EOpenPackageResult OpenPackageResult;
		FPackageReader PackageReader;
		auto ProcessPackageFile = [this](FPackageReader& PackageReader, FPackageReader::EOpenPackageResult OpenPackageResult, TUniquePtr<FReadContext>& ReadContext)
			{
				if (OpenPackageResult != FPackageReader::EOpenPackageResult::Success)
				{
					// If we're missing a custom version, we might be able to load this package later once the module containing that version is loaded...
					//   -	Attempting a retry is only useful when engine startup module is not yet complete and therefore more plugins are expected.
					const bool bAllowRetry = !IsEngineStartupModuleLoadingComplete();
					if (OpenPackageResult == FPackageReader::EOpenPackageResult::CustomVersionMissing)
					{
						ReadContext->bCanAttemptAssetRetry = bAllowRetry;
						if (!bAllowRetry)
						{
							UE_LOGF(LogAssetRegistry, Display, "Package %ls uses an unknown custom version and cannot be loaded for the AssetRegistry", *ReadContext->AssetFileData.LocalAbsPath);
						}
					}
					else
					{
						ReadContext->bCanAttemptAssetRetry = false;
					}
					ReadContext->bResult = false;
				}
				else
				{
					ReadContext->bResult = FAssetDataGatherer::ReadAssetFile(PackageReader, ReadContext->AssetDataFromFile,
						ReadContext->DependencyData, ReadContext->CookedPackageNamesWithoutAssetData,
						(bGatherAssetPackageData ? FPackageReader::EReadOptions::PackageData : FPackageReader::EReadOptions::None) |
						(bGatherDependsData ? FPackageReader::EReadOptions::Dependencies : FPackageReader::EReadOptions::None));
				}

				if (ReadContext->bResult && bBlockPackagesWithMarkOfTheWeb)
				{
					ReadContext->HasMarkOfTheWeb = BoolToOptionalBool(
						IPlatformFile::GetPlatformPhysical().HasMarkOfTheWeb(ReadContext->AssetFileData.LocalAbsPath));
				}
			};

		if (!bIsPhysicalPlatformFile)
		{		
			PackageReader.OpenPackageFile(ReadContext->AssetFileData.LongPackageName, 
				ReadContext->AssetFileData.LocalAbsPath, &OpenPackageResult);
			ProcessPackageFile(PackageReader, OpenPackageResult, ReadContext);
		}
		else if(Loader->OpenFile(*ReadContext->AssetFileData.LocalAbsPath))
		{
			PackageReader.OpenPackageFile(Loader, ReadContext->AssetFileData.LongPackageName, &OpenPackageResult);
			ProcessPackageFile(PackageReader, OpenPackageResult, ReadContext);
			Loader->Close();
		}
		else
		{
			UE_LOGF(LogAssetRegistry, Warning, "Package %ls could not be opened during gathering. Package will be ignored.",
				*ReadContext->AssetFileData.LocalAbsPath);
			ReadContext->bResult = false;
		}

		ReadyReadContexts.Enqueue(MoveTemp(ReadContext));
	}

	if (bIsPhysicalPlatformFile)
	{
		ReadTaskLoaderFreeList.Push(Loader);
	}
}

void FFileReadScheduler::AddFile(const FName PackageName, FGatheredPathData&& AssetFileData)
{
	FString DirectoryPath = FPaths::GetPath(AssetFileData.LocalAbsPath);
	check(!AssetFileData.LocalAbsPath.IsEmpty());
	check(!DirectoryPath.IsEmpty());

	bool bAlreadyExisted = false;
	TSharedPtr<FDirectoryReadTaskData>& DirectoryReadTaskData = ScheduledDirectories->FindOrAdd(DirectoryPath, &bAlreadyExisted);
	if (!bAlreadyExisted)
	{
		check(!DirectoryReadTaskData.IsValid());
		DirectoryReadTaskData = MakeShared<FDirectoryReadTaskData>();
	}

	TArray<TUniquePtr<FReadContext>>& ReadContextsToProcess = DirectoryReadsToSchedule.FindOrAdd(DirectoryPath);
	ReadContextsToProcess.Emplace(new FReadContext(PackageName,
		FName(FPathViews::GetExtension(AssetFileData.LocalAbsPath)), MoveTemp(AssetFileData), DirectoryReadTaskData));
}

void FFileReadScheduler::ScheduleDirectory(UE::Tasks::FTaskConcurrencyLimiter& TaskLimiter, const FString& DirectoryPath, 
	TArray<TUniquePtr<FReadContext>>&& ReadContexts, bool bUnblockImmediately)
{
	using namespace UE::Tasks;
	TSharedPtr<FDirectoryReadTaskData>* DirectoryReadTaskData = ScheduledDirectories->Find(DirectoryPath);
	if (!DirectoryReadTaskData)
	{
		return;
	}
	check(DirectoryReadTaskData->IsValid());

	FTask NewReadTask;
	FTask ExistingTask = (*DirectoryReadTaskData)->LastScheduledReadTask;
	if (!ReadContexts.IsEmpty())
	{
		// Schedule a new task that depends on our event being triggered (so the task can be limited) 
		// but also depends on the directory's existing scheduled task (this is a case where files 
		// in the directory come in after we have already started loading files of that directory 
		// previously added to the gatherer for reading)
		NewReadTask = Launch(TEXT("ReadDirectoryAssetFiles"),
			[this, ReadContexts = MoveTemp(ReadContexts)]() mutable
			{
				ProcessReadContexts(MoveTemp(ReadContexts), true /*bIsTimeSliced*/);
			}, UE::Tasks::Prerequisites(ExistingTask, (*DirectoryReadTaskData)->UnblockRead),
				bUnblockImmediately ? LowLevelTasks::ETaskPriority::BackgroundNormal : LowLevelTasks::ETaskPriority::BackgroundLow);
		(*DirectoryReadTaskData)->LastScheduledReadTask = NewReadTask;
	}

	if (bUnblockImmediately)
	{
		// Kick off the task immediately, allowing it to run in parallel to
		// tasks in the task limiter but at a higher priority
		(*DirectoryReadTaskData)->UnblockRead.Trigger();
	}
	else if(NewReadTask.IsValid()) // If we scheduled a new job
	{
		// This task is made such that we can run our work under the limiter's concurrency limit
		// but still let someone explicitly wait on the task outside of the limiter (possibly surpassing the limiter limit)
		// Note, we pass in the read task to wait on since if we have a chain of tasks, DirectoryReadTaskData->ReadTask
		// could change during the execution of our task.
		TaskLimiter.Push(TEXT("TriggerReadDirectoryAssetFiles"),
			[DirectoryReadTaskData = *DirectoryReadTaskData, ReadTask = NewReadTask](int32)
			{
				DirectoryReadTaskData->UnblockRead.Trigger();
				ReadTask.Wait();
			});
	}
}

void FFileReadScheduler::WaitOnDirectoriesAndUpdateWaitList(FWaitBatchDirectorySet& WaitBatchDirectories)
{
	using namespace UE::Tasks;

	if (WaitBatchDirectories.IsEmpty())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FFileReadScheduler::WaitOnDirectoriesAndUpdateWaitList)
	// Collect all child directories we might need to wait on
	FWaitBatchDirectorySet NewWaitBatchDirectories(WaitBatchDirectories);
	for (FWaitBatchDirectory& WaitBatchDirectory : WaitBatchDirectories)
	{
		if (!WaitBatchDirectory.bIsRecursive)
		{
			continue;
		}

		TArray<FString> ChildDirectories;
		if (ScheduledDirectories->TryGetChildren(WaitBatchDirectory.Path, ChildDirectories, 
			EDirectoryTreeGetFlags::ImpliedParent | EDirectoryTreeGetFlags::Recursive) && !ChildDirectories.IsEmpty())
		{
			TStringBuilder<512> AbsChildPath;
			for (FString& ChildDir : ChildDirectories)
			{
				AbsChildPath.Reset();
				AbsChildPath.Append(FPaths::Combine(WaitBatchDirectory.Path, ChildDir));
				NewWaitBatchDirectories.Add(FWaitBatchDirectory(AbsChildPath, false /*bIsRecursive*/));
			}
		}
	}

	FWaitBatchDirectorySet EmptyWaitBatchDirectories;
	FTaskConcurrencyLimiter WaitBatchLimiter(NumReadTaskThreads, LowLevelTasks::ETaskPriority::BackgroundHigh);
	for (FWaitBatchDirectory& WaitBatchDirectory : NewWaitBatchDirectories)
	{
		// Schedule any recently added files for the wait directory
		TArray<TUniquePtr<FReadContext>> UnscheduledReadContexts;
		if (DirectoryReadsToSchedule.RemoveAndCopyValue(WaitBatchDirectory.Path, UnscheduledReadContexts) && !UnscheduledReadContexts.IsEmpty())
		{
			ScheduleDirectory(WaitBatchLimiter, WaitBatchDirectory.Path, MoveTemp(UnscheduledReadContexts), false /*bUnblockImmediately*/);
		}
		// If we have existing scheduled directories, find and wait for them
		else if (TSharedPtr<FDirectoryReadTaskData>* DirectoryReadTaskData = ScheduledDirectories->Find(WaitBatchDirectory.Path))
		{
			check(DirectoryReadTaskData->IsValid());

			// Push a task to unblock our read task. We use a limiter so that we don't trigger all waiting directory reads as once
			WaitBatchLimiter.Push(TEXT("WaitBatchTriggerReadDirectoryAssetFiles"),
				[DirectoryReadTaskData = *DirectoryReadTaskData, ReadTask = (*DirectoryReadTaskData)->LastScheduledReadTask](int32)
				{
					DirectoryReadTaskData->UnblockRead.Trigger();
					ReadTask.Wait();
				});
		}
		else
		{
			// If there was no directory scheduled, remove this directory from the wait batch;
			// we likely processed all files for this directory already
			EmptyWaitBatchDirectories.Add(FWaitBatchDirectory(WaitBatchDirectory.Path, false /*bIsRecursive*/));
		}
	}
	WaitBatchLimiter.Wait();

	// Update the passed in wait batch since it may have grown based on child directories, but also remove the empty directories
	WaitBatchDirectories = NewWaitBatchDirectories.Difference(EmptyWaitBatchDirectories);
}

void FFileReadScheduler::Schedule(const TArray<FString>& PriorityDirectories)
{
	for (const FString& PriorityDirectory : PriorityDirectories)
	{
		TArray<TUniquePtr<FReadContext>> ReadContexts;
		if (DirectoryReadsToSchedule.RemoveAndCopyValue(PriorityDirectory, ReadContexts))
		{
			ScheduleDirectory(ReadConcurrencyLimiter, PriorityDirectory, MoveTemp(ReadContexts), true /*bUnblockImmediately*/);
		}
		else if (ScheduledDirectories->Find(PriorityDirectory))
		{
			ScheduleDirectory(ReadConcurrencyLimiter, PriorityDirectory, {}, true /* bUnblockImmediately */);
		}
	}

	// If there are any remaining directories, schedule them for background processing
	for (TPair<FString, TArray<TUniquePtr<FReadContext>>>& Pair : DirectoryReadsToSchedule)
	{
		const FString& Directory = Pair.Key;
		TArray<TUniquePtr<FReadContext>> ReadContexts = MoveTemp(Pair.Value);
		ScheduleDirectory(ReadConcurrencyLimiter, Directory, MoveTemp(ReadContexts), false /*bUnblockImmediately*/);
	}
	DirectoryReadsToSchedule.Empty();
}

void FFileReadScheduler::CollectCompletedReads(TArray<TUniquePtr<FReadContext>>& OutCompletedReadContexts)
{
	TUniquePtr<FReadContext> Context;
	while (ReadyReadContexts.Dequeue(Context))
	{
		OutCompletedReadContexts.Add(MoveTemp(Context));
	}
}

void FFileReadScheduler::RemoveScheduledDirectory(const FString& Directory)
{
	ScheduledDirectories->Remove(Directory);
}

void FFileReadScheduler::Shrink()
{
	ScheduledDirectories->Shrink();
	DirectoryReadsToSchedule.Shrink();
}

void FFilesToSearch::AddPriorityFile(FGatheredPathData&& FilePath)
{
	++AvailableFilesNum;
	BlockingFiles.Add(MoveTemp(FilePath));
}

void FFilesToSearch::AddFiles(TArray<FGatheredPathData>&& FilePaths)
{
	if (FilePaths.Num() == 0)
	{
		return;
	}
	
	AvailableFilesNum += FilePaths.Num();
	NormalPriorityFiles.MoveAppendRange(FilePaths.GetData(), FilePaths.Num());
}

void FFilesToSearch::AddFileAgainAfterTimeout(FGatheredPathData&& FilePath)
{
	++AvailableFilesNum;
	BlockingFiles.AddFront(MoveTemp(FilePath));
}

void FFilesToSearch::AddFileForLaterRetry(FGatheredPathData&& FilePath)
{
	LaterRetryFiles.Add(FilePath);
}

void FFilesToSearch::RetryLaterRetryFiles()
{
	while (!LaterRetryFiles.IsEmpty())
	{
		FGatheredPathData FilePath = LaterRetryFiles.PopFrontValue();
		++AvailableFilesNum;
		NormalPriorityFiles.Add(MoveTemp(FilePath));
	}
}

template <typename AllocatorType>
void FFilesToSearch::PopFront(TArray<FGatheredPathData, AllocatorType>& Out, int32 NumToPop)
{
	int32 InitialNumToPop = NumToPop;
	while (NumToPop > 0 && !BlockingFiles.IsEmpty())
	{
		Out.Add(BlockingFiles.PopFrontValue());
		--NumToPop;
	}
	while (NumToPop > 0 && !NormalPriorityFiles.IsEmpty())
	{
		Out.Add(NormalPriorityFiles.PopFrontValue());
		--NumToPop;
	}
	AvailableFilesNum += NumToPop - InitialNumToPop;
	check(AvailableFilesNum >= 0);
}

void FFilesToSearch::PrioritizePath(FPathExistence& QueryPath, EPriority Priority)
{
	// We may need to prioritize a LaterRetryFile that is now loadable, so add them all into the Root
	RetryLaterRetryFiles();

	if (Priority > EPriority::Blocking)
	{
		// TODO: Implement another tree that is searched first for the High Priority 
		// We cannot add the High Priority files to the BlockingFiles array, because
		// then blocking on BlockingFiles to be empty could be slow. We cannot add them
		// as a separate simple array, because we would have to search that (sometimes large)
		// array linearly when looking for files to accomodate a blocking priority request
		return;
	}

	FString PriorityDirectory = QueryPath.GetLocalAbsPath();
	const bool bIsDirectory = QueryPath.GetType() == UE::AssetDataGather::Private::FPathExistence::Directory;
	if (!bIsDirectory)
	{
		FStringView DirectoryPath = FPathViews::GetPath(PriorityDirectory);
		PriorityDirectory.LeftInline(DirectoryPath.Len());
	}
	PriorityDirectories.Add(PriorityDirectory);
}

void FFilesToSearch::PopPriorityDirectories(TArray<FString>& OutPriorityDirectories)
{
	while (!PriorityDirectories.IsEmpty())
	{
		OutPriorityDirectories.Add(PriorityDirectories.PopFrontValue());
	}
}

int32 FFilesToSearch::NumBlockingFiles() const
{
	return BlockingFiles.Num();
}

void FFilesToSearch::Shrink()
{
	// TODO: Make TRingBuffer::Shrink
	TRingBuffer<UE::AssetDataGather::Private::FGatheredPathData> Buffer;

	Buffer.Empty(NormalPriorityFiles.Num());
	for (UE::AssetDataGather::Private::FGatheredPathData& File : NormalPriorityFiles)
	{
		Buffer.Add(MoveTemp(File));
	}
	Swap(Buffer, NormalPriorityFiles);

	Buffer.Empty(BlockingFiles.Num());
	for (UE::AssetDataGather::Private::FGatheredPathData& File : BlockingFiles)
	{
		Buffer.Add(MoveTemp(File));
	}
	Swap(Buffer, BlockingFiles);

	Buffer.Empty(LaterRetryFiles.Num());
	for (UE::AssetDataGather::Private::FGatheredPathData& File : LaterRetryFiles)
	{
		Buffer.Add(MoveTemp(File));
	}
	Swap(Buffer, LaterRetryFiles);
}

int32 FFilesToSearch::Num() const
{
	return AvailableFilesNum + LaterRetryFiles.Num();
}

int32 FFilesToSearch::GetNumAvailable() const
{
	return AvailableFilesNum;
}

SIZE_T FFilesToSearch::GetAllocatedSize() const
{
	SIZE_T Size = 0;
	Size += NormalPriorityFiles.GetAllocatedSize();
	for (const FGatheredPathData& PathData : NormalPriorityFiles)
	{
		Size += PathData.GetAllocatedSize();
	}
	Size += BlockingFiles.GetAllocatedSize();
	for (const FGatheredPathData& PathData : BlockingFiles)
	{
		Size += PathData.GetAllocatedSize();
	}
	Size += LaterRetryFiles.GetAllocatedSize();
	for (const FGatheredPathData& PathData : LaterRetryFiles)
	{
		Size += PathData.GetAllocatedSize();
	}
	return Size;
}

bool IsVerseFile(const EGatherableFileType FileType)
{
	return (FileType == EGatherableFileType::VerseFile) | (FileType == EGatherableFileType::VerseModule);
}

bool DoesPathContainInvalidCharacters(const EGatherableFileType FileType, FStringView FilePath)
{
	if (FilePath.IsEmpty())
	{
		return true;
	}

	// NOTE: This is replicating the logic in
	// `CSourceFileProject::IsValidModuleName`/`CSourceFileProject::IsValidSnippetFileName` because we cannot bring in
	// the `uLang` string utilities here (as they assume `uLang` is initialized, which may not be the case).  However,
	// one difference is that here, we are more permissive with Verse snippet filenames than what should be allowed -
	// the reason for that is we previously shipped with this behaviour, and need to continue supporting it. We reject
	// these files later on in `CSourceFilePackage::GatherPackageSourceFiles` instead if needed, as we know the
	// package's uploaded version at that point.
	if (FileType == EGatherableFileType::VerseFile)
	{
		for (const TCHAR* InvalidCharacters = INVALID_LONGPACKAGE_CHARACTERS; *InvalidCharacters; ++InvalidCharacters)
		{
			if (*InvalidCharacters == '.')
			{
				continue;
			}
			int32 OutIndex;
			if (FilePath.FindChar(*InvalidCharacters, OutIndex))
			{
				return true;
			}
		}
		return false;
	}
	else if (FileType == EGatherableFileType::VerseModule)
	{
		if (!FChar::IsAlpha(FilePath[0]) && FilePath[0] != '_')
		{
			return true;
		}

		for (const auto& Char : FilePath)
		{
			if (!FChar::IsAlnum(Char) && Char != '_')
			{
				return true;
			}
		}
		return false;
	}
	else
	{
		return FPackageName::DoesPackageNameContainInvalidCharacters(FilePath);
	}
}
} // namespace UE::AssetDataGather::Private

// ============================================================================
// Tests for cache ref-file indirection (write-once binaries + .ref pointers)
// ============================================================================
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCacheRefFileLifecycleTest,
	"System.AssetRegistry.CacheRefFile.Lifecycle",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);

bool FCacheRefFileLifecycleTest::RunTest(const FString& Parameters)
{
	using namespace UE::AssetDataGather::Private;

	// --- Setup: create an isolated temp directory ---
	const FString TempDir = FPaths::ProjectIntermediateDir() / FString::Printf(TEXT("CacheRefLifecycleTest_%s"), *FGuid::NewGuid().ToString());
	IFileManager::Get().MakeDirectory(*TempDir, true);
	ON_SCOPE_EXIT
	{
		IFileManager::Get().DeleteDirectory(*TempDir, false, true);
	};

	const FString CacheBase = TempDir / TEXT("TestCache");

	// Helper: write a valid cache binary file identical to SaveCacheFileInternal's format.
	auto WriteCacheBinary = [this](const FString& TargetPath, const TArray<TPair<FName, FDiskCachedAssetData*>>& Assets) -> bool
	{
		const FString TempPath = TargetPath + TEXT(".tmp");
		{
			TUniquePtr<FArchive> FileAr(IFileManager::Get().CreateFileWriter(*TempPath, 0));
			if (!FileAr)
			{
				AddError(FString::Printf(TEXT("Failed to create file writer for %s"), *TempPath));
				return false;
			}
			UE::THashingArchiveProxy<FBlake3> HashAr(*FileAr);
			uint64 Version = FGatherSettings::CurrentVersion;
			HashAr << Version;
			{
				FAssetRegistryWriter Writer(FAssetRegistryWriterOptions(), HashAr);
				UE::AssetDataGather::Private::SerializeCacheSave(Writer, Assets);
			}
			FBlake3Hash Hash = HashAr.GetHash();
			FileAr->Serialize(&Hash, sizeof(Hash));
		}
		return IFileManager::Get().Move(*TargetPath, *TempPath);
	};

	// Helper: count .bin files matching the cache base pattern
	auto CountBinFiles = [&CacheBase]() -> int32
	{
		TArray<FString> Files;
		IFileManager::Get().FindFiles(Files, *(CacheBase + TEXT("_*.bin")), true, false);
		return Files.Num();
	};

	// Helper: count .ref files
	auto CountRefFiles = [&CacheBase]() -> int32
	{
		TArray<FString> Files;
		IFileManager::Get().FindFiles(Files, *(CacheBase + TEXT("_*.ref")), true, false);
		return Files.Num();
	};

	// --- Step 1: Build test asset data ---
	FDiskCachedAssetData AssetA(FDateTime::Now(), FDateTime::Now(), FName(TEXT("uasset")));
	FDiskCachedAssetData AssetB(FDateTime::Now(), FDateTime::Now(), FName(TEXT("uasset")));

	TArray<TPair<FName, FDiskCachedAssetData*>> TestAssets;
	TestAssets.Emplace(FName(TEXT("/Test/PackageA")), &AssetA);
	TestAssets.Emplace(FName(TEXT("/Test/PackageB")), &AssetB);

	// --- Step 2: Write binary A + .ref ---
	const FString BinaryPathA = MakeShardBinaryFilename(CacheBase, 0);
	TestTrue(TEXT("Write binary A"), WriteCacheBinary(BinaryPathA, TestAssets));

	const FString RefPath = GetShardRefFilename(CacheBase, 0);
	WriteCacheRefFile(RefPath, FPaths::GetCleanFilename(BinaryPathA));

	TestEqual(TEXT("Step 2: 1 .bin file"), CountBinFiles(), 1);
	TestEqual(TEXT("Step 2: 1 .ref file"), CountRefFiles(), 1);
	TestEqual(TEXT("Step 2: .ref points to A"), ReadCacheRefFile(RefPath), FPaths::GetCleanFilename(BinaryPathA));

	// --- Step 3: Load and verify round-trip ---
	{
		TArray<FCachePayload> Payloads = LoadCacheFiles({BinaryPathA}, false);
		TestEqual(TEXT("Step 3: LoadCacheFiles returned 1 payload"), Payloads.Num(), 1);
		TestTrue(TEXT("Step 3: Load succeeded"), Payloads[0].bSucceeded);
		TestEqual(TEXT("Step 3: 2 assets loaded"), Payloads[0].NumAssets, 2);
	}

	// --- Step 4: Write binary B, update .ref ---
	const FString BinaryPathB = MakeShardBinaryFilename(CacheBase, 0);
	TestTrue(TEXT("Write binary B"), WriteCacheBinary(BinaryPathB, TestAssets));
	WriteCacheRefFile(RefPath, FPaths::GetCleanFilename(BinaryPathB));

	// --- Step 5: Assert two .bin files, .ref points to B ---
	TestEqual(TEXT("Step 5: 2 .bin files"), CountBinFiles(), 2);
	TestEqual(TEXT("Step 5: 1 .ref file"), CountRefFiles(), 1);
	TestEqual(TEXT("Step 5: .ref points to B"), ReadCacheRefFile(RefPath), FPaths::GetCleanFilename(BinaryPathB));

	// --- Step 6: Write binary C, update .ref ---
	const FString BinaryPathC = MakeShardBinaryFilename(CacheBase, 0);
	TestTrue(TEXT("Write binary C"), WriteCacheBinary(BinaryPathC, TestAssets));
	WriteCacheRefFile(RefPath, FPaths::GetCleanFilename(BinaryPathC));

	TestEqual(TEXT("Step 6: 3 .bin files"), CountBinFiles(), 3);

	// --- Step 7: Run cleanup; A and B are orphaned, C is referenced ---
	CleanupOrphanedCacheFiles(CacheBase, ECacheCleanupContext::PostWrite);

	TestEqual(TEXT("Step 7: 1 .bin file remains"), CountBinFiles(), 1);
	TestEqual(TEXT("Step 7: 1 .ref file remains"), CountRefFiles(), 1);
	TestTrue(TEXT("Step 7: Binary C still exists"), IFileManager::Get().FileExists(*BinaryPathC));
	TestFalse(TEXT("Step 7: Binary A deleted (orphan)"), IFileManager::Get().FileExists(*BinaryPathA));
	TestFalse(TEXT("Step 7: Binary B deleted (orphan)"), IFileManager::Get().FileExists(*BinaryPathB));
	TestEqual(TEXT("Step 7: .ref points to C"), ReadCacheRefFile(RefPath), FPaths::GetCleanFilename(BinaryPathC));

	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
