// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/PackageReader.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/DirectoryTree.h"
#include "Containers/LockFreeList.h"
#include "Containers/Map.h"
#include "Containers/MpscQueue.h"
#include "Containers/RingBuffer.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CriticalSectionQueryable.h"
#include "DiskCachedAssetData.h"
#include "HAL/CriticalSection.h"
#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "HAL/Runnable.h"
#include "Misc/DateTime.h"
#include "Misc/Optional.h"
#include "NonBufferingReadOnlyArchive.h"
#include "PackageDependencyData.h"
#include "Tasks/TaskConcurrencyLimiter.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

#include <atomic>

class FArchive;
class FAssetRegistryReader;
class FAssetRegistryWriter; // Not defined if !ALLOW_NAME_BATCH_SAVING
class FDiskCachedAssetData;
struct FAssetData;
namespace UE::AssetDataGather { struct FResults; }
namespace UE::AssetDataGather::Private { class FAssetDataDiscovery; }
namespace UE::AssetDataGather::Private { class FFileReadScheduler; }
namespace UE::AssetDataGather::Private { class FFilesToSearch; }
namespace UE::AssetDataGather::Private { struct FCachePayload; }
namespace UE::AssetDataGather::Private { struct FDirectoryReadTaskData; }
namespace UE::AssetDataGather::Private { struct FGatheredPathData; }
namespace UE::AssetDataGather::Private { struct FPathExistence; }
namespace UE::AssetDataGather::Private { struct FReadContext; }
namespace UE::AssetDataGather::Private { struct FSetPathProperties; }
namespace UE::AssetDataGather::Private { enum class EPriority : uint8; }
namespace UE::AssetDataGather::Private 
{ 
struct FWaitBatchDirectory
{
	FWaitBatchDirectory(const FStringView InPath, const bool InIsRecursive)
		: Path(InPath)
		, bIsRecursive(InIsRecursive)
	{
	}

	friend int32 GetTypeHash(const FWaitBatchDirectory& WaitBatchDirectory)
	{
		return GetTypeHash(WaitBatchDirectory.Path);
	}

	FString Path;
	bool bIsRecursive;
};

struct FWaitBatchDirectorySetFuncs : BaseKeyFuncs<FWaitBatchDirectory, FString>
{
	static FORCEINLINE const FString& GetSetKey(const FWaitBatchDirectory& Element)
	{
		return Element.Path;
	}
	static FORCEINLINE bool Matches(const FString& A, const FString& B)
	{
		return A.Equals(B, ESearchCase::CaseSensitive);
	}
	static FORCEINLINE uint32 GetKeyHash(const FString& Key)
	{
		return GetTypeHash(Key);
	}
};

typedef TSet<FWaitBatchDirectory, FWaitBatchDirectorySetFuncs> FWaitBatchDirectorySet;
}
namespace UE::AssetRegistry { class FAssetRegistryImpl; }
namespace UE::AssetRegistry::Impl { enum class EGatherStatus : uint8; }

#if DO_CHECK
typedef FCriticalSectionQueryable FGathererCriticalSection;
typedef FScopeLockQueryable FGathererScopeLock;
#define CHECK_IS_LOCKED_CURRENT_THREAD(CritSec) check(CritSec.IsLockedOnCurrentThread())
#define CHECK_IS_NOT_LOCKED_CURRENT_THREAD(CritSec) check(!CritSec.IsLockedOnCurrentThread())
#else
typedef FCriticalSection FGathererCriticalSection;
typedef FScopeLock FGathererScopeLock;
#define CHECK_IS_LOCKED_CURRENT_THREAD(CritSec) do {} while (false)
#define CHECK_IS_NOT_LOCKED_CURRENT_THREAD(CritSec) do {} while (false)
#endif

DECLARE_TS_MULTICAST_DELEGATE_OneParam(FGatheredResultsEvent, const UE::AssetDataGather::FResults&);

struct FAssetGatherDiagnostics
{
	/** Time spent identifying asset files on disk */
	float DiscoveryTimeSeconds;
	/** Time spent reading asset files on disk / from cache */
	float GatherTimeSeconds;
	/** Time in between gatherer start and the call to GetDiagnostics. */
	float WallTimeSeconds;
	/** How many directories in the search results were read from the cache. */
	int32 NumCachedDirectories;
	/** How many directories in the search results were not in the cache and were read by scanning the disk. */
	int32 NumUncachedDirectories;
	/** How many files in the search results were read from the cache. */
	int32 NumCachedAssetFiles;
	/** How many files in the search results were not in the cache and were read by parsing the file. */
	int32 NumUncachedAssetFiles;
};

namespace UE::AssetDataGather
{

/** Structure to accumulate the results of the gather. Appended to from calls to FAssetDataGatherer::GetAndTrimSearchResults. */
struct FResults
{
	TMultiMap<FName, TUniquePtr<FAssetData>> Assets;
	TMultiMap<FName, TUniquePtr<FAssetData>> AssetsForGameThread;
	TRingBuffer<FString> Paths;
	TMultiMap<FName, FPackageDependencyData> Dependencies;
	TMultiMap<FName, FPackageDependencyData> DependenciesForGameThread;
	TRingBuffer<FString> CookedPackageNamesWithoutAssetData;
	TRingBuffer<FName> VerseFiles;
	TArray<FString> BlockedFiles;

	SIZE_T GetAllocatedSize() const
	{
		return Assets.GetAllocatedSize() + AssetsForGameThread.GetAllocatedSize() + Paths.GetAllocatedSize() + Dependencies.GetAllocatedSize() +
			DependenciesForGameThread.GetAllocatedSize() + CookedPackageNamesWithoutAssetData.GetAllocatedSize() + VerseFiles.GetAllocatedSize() +
			BlockedFiles.GetAllocatedSize();
	}
	void Shrink()
	{
		Assets.Shrink();
		AssetsForGameThread.Shrink();
		Paths.Trim();
		Dependencies.Shrink();
		DependenciesForGameThread.Shrink();
		CookedPackageNamesWithoutAssetData.Trim();
		VerseFiles.Trim();
		BlockedFiles.Shrink();
	}
};

/**
 * Structure to receive transient data about the current state of the gather. Repopulated during every call to
 * FAssetDataGatherer::GetAndTrimSearchResults.
 */
struct FResultContext
{
	bool bIsSearching = false;
	bool bAbleToProgress = false;
	TArray<double> SearchTimes;
	int32 NumFilesToSearch = 0;
	int32 NumPathsToSearch = 0;
	bool bIsDiscoveringFiles = false;
};

} // namespace UE::AssetDataGather


/**
 * Async task for gathering asset data from from the file list in FAssetRegistry
 */
class FAssetDataGatherer : public FRunnable
{
public:
	FAssetDataGatherer(UAssetRegistryImpl& InRegistryImpl);
	virtual ~FAssetDataGatherer();

	/** 
	* Update the the Gatherer's cache using in-memory state information before serializing. 
	* This method must be called while owning the InterfaceLock.
	*/
	void UpdateCacheForSaving();
	void OnInitialSearchCompleted();
	void OnAdditionalMountSearchCompleted();
	/** Requests a save of the cache as soon as possible. */
	void RequestAsyncCacheSave();
	/** Requests a delayed save of the cache. If a delayed save has already been requested, the delayed save timer is not reset. */
	void RequestDelayedAsyncCacheSave();
	/** Removes any existing request to save the cache. */
	void ClearRequestAsyncCacheSave();
	/** 
	* Returns true if an immediate or delayed save has been requested. 
	* Sets OutSaveRequestTime to the time the save was requested (value may 
	* be negative to indicate a save should occur immediately), zero otherwise.
	*/
	bool IsCacheSaveRequested(double* OutSaveRequestTime = nullptr) const;
	/**
	* Returns true when a save has been requested and is ready to be handled. Delayed save requests are ready once
	* FGatherSettings::MinSecondsToElapseBeforeCacheWrite seconds have elapsed since the time of the save request. 
	*/
	bool IsCacheSaveReady() const;

	// Controlling Async behavior

	/** Start the async thread, if this Gatherer was created async. Does nothing if not async or already started. */
	void StartAsync();

	// FRunnable implementation
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;

	bool IsAsyncEnabled() const;
	bool IsSynchronous() const;
	/** Signals to end the thread and waits for it to close before returning */
	void EnsureCompletion();

	/** Returns reference to delegate invoked when search results have been gathered in GetAndTrimSearchResults. */
	FGatheredResultsEvent& GetGatheredResultsEvent();

	/** Gets search results from the data gatherer. */
	void GetAndTrimSearchResults(UE::AssetDataGather::FResults& InOutResults,
		UE::AssetDataGather::FResultContext& OutContext);
	/** Get diagnostics for telemetry or logging. */
	FAssetGatherDiagnostics GetDiagnostics();
	/** Gets just the Assets, AssetsForGameThread, Dependencies, and DependenciesForGameThread from the data gatherer. */
	void GetPackageResults(UE::AssetDataGather::FResults& InOutResults);
	/**
	 * Wait for all monitored assets under the given path to be added to search results.
	 * Returns immediately if the given path is not monitored.
	 */
	void WaitOnPath(FStringView LocalPath);
	/**
	 * Saves the current cache state to disk, then empties the caches holding data read from disk and new data to be 
	 * written to disk. Disables further caching. Used to save memory when cooking after the scan is complete.
	 * Saving is best-effort and may fail if the cache files are open for read on the system.
	*/
	void ClearCache();

	/**
	 * Add a set of paths to the allow list, optionally force rescanning and ignore deny list on them,
	 * and wait for all assets in the paths to be added to search results.
	 * Wait time is minimized by prioritizing the paths and transferring async scanning to the current thread.
	 */
	void ScanPathsSynchronous(const TArray<FString>& InPaths, bool bForceRescan, bool bIgnoreDenyListScanFilters);
	/** Wait for all monitored assets to be added to search results. */
	void WaitForIdle(float TimeoutSeconds = -1.0f);
	/**
	 * Report whether all monitored assets have been added to search results, AND these results have been gathered
	 * through a GetAndTrimSearchResults call.
	 * This function can be used to check whether there is any work to be done on the gatherer.
	 */
	bool IsComplete() const;
	/** Returns true if the asset registry has written out the discovery cache after discovery has completed. 
	* This function will return false if discovery is not yet complete, discovery caching isn't supported by the current platform 
	* or cache writing has been disabled such as when using -NoAssetRegistryCacheWrite. */
	bool HasSerializedDiscoveryCache() const;

	// Reading/writing/triggering depot-wide properties and events (possibly while tick is running)

	/** Set after initial plugins have loaded and we should not retry failed loads with missing custom versions. */
	void SetInitialPluginsLoaded();
	/** Report whether the gatherer is configured to load depends data in addition to asset data. */
	bool IsGatheringDependencies() const;
	/** Return whether the current process enables reading AssetDataGatherer cache files. */
	bool IsCacheReadEnabled() const;
	/** Return whether the current process enables writing AssetDataGatherer cache files. */
	bool IsCacheWriteEnabled() const;
	/** Return the memory used by the gatherer. Used for performance metrics. */
	SIZE_T GetAllocatedSize() const;


	// Configuring mount points (possibly while tick is running)

	/** Add a mountpoint to the gatherer after it has been registered with FPackageName .*/
	void AddMountPoint(FStringView LocalPath, FStringView LongPackageName);
	/** Remove a previously added mountpoint. */
	void RemoveMountPoint(FStringView LocalPath);
	/** Add MountPoints in LocalPaths to the gatherer. */
	void AddRequiredMountPoints(TArrayView<FString> LocalPaths);


	// Reading/Writing properties of files and directories (possibly while tick is running)

	/** Called from DirectoryWatcher. Update the directory for reporting in future search results. */
	void OnDirectoryCreated(FStringView LocalPath);
	/** Called from DirectoryWatcher. Update the files for reporting in future search results. */
	void OnFilesCreated(TConstArrayView<FString> LocalPaths);
	/** Mark a file or directory to be scanned before unprioritized assets. */
	void PrioritizeSearchPath(const FString& PathToPrioritize);
	/**
	 * Mark whether a given path is in the scanning allow list.
	 *
	 * By default no paths are scanned; adding a path to the allow list causes it and its subdirectories to be scanned.
	 * Note that the deny list (InLongPackageNameDenyList) overrides the allow list.
	 * Allow list settings are recursive. Attempting to mark a path as allowed if a parent path is on the allow list
	 * will have no effect. This means the scenario ((1) add allow list A (2) add allow list A/Child (3) remove allow
	 * list A) will therefore not result in A/Child being allowed.
	 */
	void SetIsOnAllowList(FStringView LocalPath, bool bIsAllowed);
	/** Report whether the path is in the allow list. Only paths in AllowList AND not in DenyList will be scanned. */
	bool IsOnAllowList(FStringView LocalPath) const;
	/** Report whether the path is in the deny list. Paths in DenyList are not scanned. */
	bool IsOnDenyList(FStringView LocalPath) const;
	/** Report whether the path is both in the allow list and not in the deny list. */
	bool IsMonitored(FStringView LocalPath) const;

	/** Determine, based on the file extension, if the given file path is a Verse file */
	static bool IsVerseFile(FStringView FilePath);
	/** Return the list of extensions that indicate verse files. */
	static TConstArrayView<const TCHAR*> GetVerseFileExtensions();

	/**
	 * Reads FAssetData information out of a previously initialized package reader
	 *
	 * @param PackageReader the previously opened package reader
	 * @param AssetDataList the FAssetData for every asset found in the file
	 * @param DependencyData the FPackageDependencyData for every asset found in the file
	 * @param CookedPackagesToLoadUponDiscovery the list of cooked packages to be loaded if any
	 * @param Options Which bits of data to read
	 */
	static bool ReadAssetFile(FPackageReader& PackageReader, TArray<FAssetData*>& AssetDataList,
		FPackageDependencyData& DependencyData, TArray<FString>& CookedPackagesToLoadUponDiscovery,
		FPackageReader::EReadOptions Options);

	/** Callable by the main thread to request that this thread pause/resume processing data. Gathering can 
	 *  still proceed during this time.
	 */
	void PauseProcessing() { IsProcessingPaused.fetch_add(1, std::memory_order_relaxed); }
	void ResumeProcessing() { IsProcessingPaused.fetch_sub(1, std::memory_order_relaxed); }
	bool IsProcessingPauseRequested() const { return IsProcessingPaused.load(std::memory_order_relaxed) != 0; }

	void SetGatherOnGameThreadOnly(bool bValue);
	bool IsGatherOnGameThreadOnly() const;

	/*
	* Mark that the gatherer is in the process of handling an additional search.
	*/
	void SetIsAdditionalMountSearchInProgress(bool bIsInProgress);

private:
	enum class ETickResult
	{
		KeepTicking,
		PollDiscovery,
		Idle,
		Interrupt,
	};
	/**
	 * Helper function to run the tick in a loop-within-a-loop to minimize critical section entry, and to move expensive
	 * operations out of the critical section
	 */
	ETickResult InnerTickLoop(bool bInSynchronousTick, bool bContributeToCacheSave, double EndTimeSeconds, bool* bShouldRequestAssetRegistryTick = nullptr);
	/**
	 * Tick function to pump scanning and push results into the search results structure. May be called from devoted
	 * thread or inline from synchronous functions on other threads.
	 */
	ETickResult TickInternal(double& TickStartTime, bool bPollDiscovery, bool* bShouldRequestAssetRegistryTick = nullptr);
	/**
	* Conditionally Ticks the AssetRegistry from the background thread returning the result of gathering. Return the new status of the
	* AssetRegistry's gather. If the background thread should not currently perfom ticking, EGatherStatus::Complete is returned.
	*/
	UE::AssetRegistry::Impl::EGatherStatus TryTickOnBackgroundThread();

	/** Called outside the TickLock or ResultLock; it takes the ResultLock internally. */
	bool ShouldBackgroundGatherThreadPauseGathering(bool& bOutGathererIsIdle);

	/** Add any new package files from the background directory scan to our work list **/
	void IngestDiscoveryResults();

	/** Helper for OnFilesCreated. Update the file for reporting in future search results. */
	void OnFileCreated(FStringView LocalPath);

	/**
	 * Set a selection of directory-scanning properties on a given LocalPath.
	 * This function is used when multiple properties need to be set on the path and we want to avoid redundant
	 * tree-traversal costs.
	 */
	void SetDirectoryProperties(FStringView LocalPath, const UE::AssetDataGather::Private::FSetPathProperties& Properties);

	/**
	 * Wait for all monitored assets under the given path to be added to search results.
	 * Returns immediately if the given path are not monitored.
	 */
	void WaitOnPathsInternal(TArrayView<UE::AssetDataGather::Private::FPathExistence> QueryPaths);

	/** Sort the pending list of filepaths so that assets under the given directory/filename are processed first. */
	void SortPathsByPriority(TArrayView<UE::AssetDataGather::Private::FPathExistence> QueryPaths,
		UE::AssetDataGather::Private::EPriority Priority, int32& OutNumPaths);
	/**
	 * Reads FAssetData information out of a file
	 *
	 * @param AssetLongPackageName the package name of the file to read
	 * @param AssetFilename the local path of the file to read
	 * @param AssetDataList the FAssetData for every asset found in the file
	 * @param DependencyData the FPackageDependencyData for every asset found in the file
	 * @param CookedPackagesToLoadUponDiscovery the list of cooked packages to be loaded if any
	 * @param OutCanRetry Set to true if this file failed to load, but might be loadable later (due to missing modules)
	 *
	 * @return true if the file was successfully read
	 */
	bool ReadAssetFile(const FString& AssetLongPackageName, const FString& AssetFilename,
		TArray<FAssetData*>& AssetDataList, FPackageDependencyData& DependencyData,
		TArray<FString>& CookedPackagesToLoadUponDiscovery, bool& OutCanRetry) const;

	/** Add the given AssetDatas into DiskCachedAssetDataMap and DiskCachedAssetBlocks. */
	void ConsumeCacheFiles(TArray<UE::AssetDataGather::Private::FCachePayload> Payloads);
	/**
	 * If a cache save has been triggered, get the cache filename and pointers to all elements that
	 * should be saved, for later saving outside of the critical section.
	 */
	void TryReserveSaveCache(bool& bOutShouldSave, TArray<TPair<FName,FDiskCachedAssetData*>>& AssetsToSave);
	/** 
	* Tries to save the cache file for the assetdatas read from package headers, possibly sharded into multiple files. 
	* If the save fails, bRequestSaveOnFailure specifies whether a save will be requested again to be handled in a future gatherer tick.
	* Returns whether the save was successful or not.
	*/
	bool TrySaveCacheFile(const TArray<TPair<FName,FDiskCachedAssetData*>>& AssetsToSave, bool bRequestSaveOnFailure = false);
	/**
	 * If the CacheFilename/AssetsToSave are non empty, save the cache file. 
	 * This function reads the read-only-after-creation data from each FDiskCachedAssetData*, but otherwise does not use
	 * data from this Gatherer and so can be run outside any critical section.
	 * Returns the size of the saved file, or 0 if nothing was saved for any reason.
	 */
	int64 SaveCacheFileInternal(const FString& CacheFilename,
		const TArray<TPair<FName,FDiskCachedAssetData*>>& AssetsToSave);
	/**
	 * Get the list of FDiskCachedAssetData* for saving into the cache.
	 * Includes both assets that were loaded in the gatherer and assets which were loaded from the cache and have not been pruned.
	 */
	void GetCacheAssetsToSave(TArray<TPair<FName,FDiskCachedAssetData*>>& OutAssetsToSave);

	/* Adds the given pair into NewCachedAssetDataMap. Detects collisions for multiple files with the same PackageName */
	void AddToCache(FName PackageName, FDiskCachedAssetData* DiskCachedAssetData);

	/**
	 * Mark that the gatherer has become idle or has become active. Called from tick function and configuration functions
	 * when they note a possible state change. Caller is responsible for holding the ResultsLock.
	 */
	void SetIsIdle(bool IsIdle);
	void SetIsIdle(bool IsIdle, double& TickStartTime);

	/** Minimize memory usage in the buffers used during gathering. */
	void Shrink();

	/** Scoped guard for pausing the asynchronous tick. */
	struct FScopedGatheringPause
	{
		FScopedGatheringPause(const FAssetDataGatherer& InOwner);
		~FScopedGatheringPause();
		const FAssetDataGatherer& Owner;
	};

	/** Convert the LocalPath into our normalized version. */
	static FString NormalizeLocalPath(FStringView LocalPath);
	/** Convert the LongPackageName into our normalized version. */
	static FStringView NormalizeLongPackageName(FStringView LongPackageName);

private:

	/**
	 * Critical section to allow Tick to be called from worker thread or from synchronous functions on other threads.
	 * To prevent DeadLocks, TickLock can not be entered from within any of the other locks on this class.
	 */
	mutable FGathererCriticalSection TickLock;
	/**
	 * A critical section to protect data transfer to GetAndTrimSearchResults.
	 * ResultsLock can be entered while holding TickLock.
	 */
	mutable FGathererCriticalSection ResultsLock;

	/** Delegates invoked when results are gathered in GetAndTrimSearchResults */
	FGatheredResultsEvent GatheredResultsEvent;

	// Variable section for variables that are constant during threading.
	UAssetRegistryImpl& AssetRegistry;

	/**
	 * Thread to run async Ticks on. Constant during threading.
	 * Activated when StartAsync is called and bAsyncEnabled is true.
	 * If null, results will only be added when Wait functions are called. Constant during threading.
	 */
	FRunnableThread* Thread;

	/**
	 * True if async gathering is enabled, false if e.g. singlethreaded or disabled by commandline.
 	 * Even when enabled, gathering is still synchronous until StartAsync is called.
	 */
	bool bAsyncEnabled;
	/** True if AssetPackageData should be gathered. Constant during threading. */
	bool bGatherAssetPackageData;
	/** True if dependency data should be gathered. Constant during threading. */
	bool bGatherDependsData;

	/** Timestamp of the start of the gather for consistent marking of 'last discovered' time in caching */
	FDateTime GatherStartTime;

	// Variable section for variables that are atomics read/writable from outside critical sections.

	/** > 0 if we've been asked to abort gathering work in progress at the next opportunity. */
	std::atomic<uint32> IsStopped;
	/** > 0 if we've been asked to pause the worker thread gathering work so a synchronous function can take over the tick. */
	mutable std::atomic<uint32> IsGatheringPaused;
	
	/** > 0 if we've been asked to pause processing work (but not gathering work) at the next opportunity */
	mutable std::atomic<uint32> IsProcessingPaused;

	/**
	 * Discovery subsystem; decides which paths to search and queries the FileManager to search directories.
	 * Pointer is constant during threading. Object pointed to internally provides threadsafety.
	 */
	TUniquePtr<UE::AssetDataGather::Private::FAssetDataDiscovery> Discovery;
	/** 
	* Start time a delayed request to save the async cache was made. 
	* A save will be triggered after FGatherSettings::MinSecondsToElapseBeforeCacheWrite have elapsed. 
	* When an immediate save request is made this time will be negative.
	*/
	std::atomic<double> DelayedSaveAsyncCacheStartTime;
	/** True if the current process allows reading of AssetDataGatherer cache files. */
	std::atomic<bool> bCacheReadEnabled;
	/** True if the current process allows writing of AssetDataGatherer cache files. */
	std::atomic<bool> bCacheWriteEnabled;

	// Variable section for variables that are read/writable only within ResultsLock.

	/** List of files that need to be processed by the search. */
	TUniquePtr<UE::AssetDataGather::Private::FFilesToSearch> FilesToSearch;

	/** The asset data gathered from the searched files. */
	TArray<TUniquePtr<FAssetData>> AssetResults;
	/** Like AssetResults but for assets that must be processed on the game thread */
	TArray<TUniquePtr<FAssetData>> AssetResultsForGameThread;
	/** Dependency data gathered from the searched files packages. */
	TArray<FPackageDependencyData> DependencyResults;
	/** Like DependencyResults but for assets that must be processed on the game thread */
	TArray<FPackageDependencyData> DependencyResultsForGameThread;
	/**
	 * A list of cooked packages that did not have asset data in them.
	 * These assets may still contain assets (if they were older for example). 
	 */
	TArray<FString> CookedPackageNamesWithoutAssetDataResults;
	/** File paths (in UE LongPackagePath notation) of the Verse source code gathered from the searched files. */
	TArray<FName> VerseResults;
	/** File paths (in regular filesystem notation) of blocked packages from the searched files. */
	TArray<FString> BlockedResults;

	/** All the search times since the last call to GetAndTrimSearchResults. */
	TArray<double> SearchTimes;
	/** Sum of all SearchTimes. */
	float CumulativeGatherTime = 0.f;

	/** The directories found during the search. */
	TArray<FString> DiscoveredPaths;

	/** The time spent in TickInternal since the last idle time. Used for performance metrics when reporting results. */
	double CurrentSearchTime = 0.;
	/** The last time at which the cache file was written, used to periodically update the cache. */
	double LastCacheWriteTime;
	/** The cached value of the NumPathsToSearch returned by Discovery the last time we synchronized with it. */
	int32 NumPathsToSearchAtLastSyncPoint;
	/** The total number of files in the search results that were read from the cache. */
	int32 NumCachedAssetFiles = 0;
	/** The total number of files in the search results that were not in the cache and were read by parsing the file. */
	int32 NumUncachedAssetFiles = 0;
	/** The total number of files in the search results that were not in the cache and are currently being read off disk. */
	int32 NumUncachedAssetFilesOutstanding = 0;
	/** Track whether the cache has been loaded. */
	bool bHasLoadedCache;
	/** Track whether the Discovery subsystem has gone idle and we have read all filenames from it. */
	bool bDiscoveryIsComplete;
	/** Track whether this Gather has gone idle and a caller has read all search data from it. */
	bool bIsComplete;
	/** Track whether this Gatherer has gone idle, either it has no more work or it's blocked on external events. */
	bool bIsIdle;
	/** Track the first tick after idle to set up e.g. timing data. */
	bool bFirstTickAfterIdle;
	/** True if we have finished discovering our first wave of files, to report metrics for that most-important wave. */
	bool bFinishedInitialDiscovery;
	/** True if OnInitialSearchCompleted has been called. */
	std::atomic<bool> bIsInitialSearchCompleted;
	/** True if we have begun discovering files after the initial search. */
	std::atomic<bool> bIsAdditionalMountSearchInProgress;
	std::atomic<bool> bGatherOnGameThreadOnly;

	// Variable section for variables that are read/writable only within TickLock.

	/**
	 * An array of all cached data that was newly discovered this run. This array is just used to make sure they are all
	 * deleted at shutdown.
	 */
	TArray<FDiskCachedAssetData*> NewCachedAssetData;
	TArray<TPair<int32, FDiskCachedAssetData*>> DiskCachedAssetBlocks;
	/**
	 * Map of PackageName to cached discovered assets that were loaded from disk.
	 * This should only be modified by ConsumeCacheFiles.
	 */
	TMap<FName, FDiskCachedAssetData*> DiskCachedAssetDataMap;
	/** Map of PackageName to cached discovered assets that will be written to disk at shutdown. */
	TMap<FName, FDiskCachedAssetData*> NewCachedAssetDataMap;
	/** How many uncached asset files had been discovered at the last async cache save */
	int32 LastCacheSaveNumUncachedAssetFiles;
	/**
	 * True if the current TickInternal is synchronous, which may be because !IsSynchronous or because the game thread has
	 * taken over the tick for a synchronous function.
	 */
	bool bSynchronousTick;
	/** 
	* True when a thread is saving an async cache and therefore the cache cannot be deleted.
	* This value resets after the save attempt has completed regardless of whether it was successful or not. 
	*/
	bool bIsSavingAsyncCache;
	/** Packages can be marked for retry up until bInitialPluginsLoaded is set. After it is set, we retry them once. */
	bool bFlushedRetryFiles;
	/** Directories to explicitly wait for all files under it to be gatherered the next time we tick. */
	TOptional<UE::AssetDataGather::Private::FWaitBatchDirectorySet> WaitBatchDirectories;
	/** Used to manage the async task for reading gathered files. */
	TUniquePtr<UE::AssetDataGather::Private::FFileReadScheduler> FileReadScheduler;
};


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////

inline bool FAssetDataGatherer::IsCacheSaveRequested(double* OutSaveRequestTime) const
{
	double LocalSaveAsyncCacheStartTime = DelayedSaveAsyncCacheStartTime.load(std::memory_order_relaxed);
	if (OutSaveRequestTime)
	{
		*OutSaveRequestTime = LocalSaveAsyncCacheStartTime;
	}
	return LocalSaveAsyncCacheStartTime != 0.0;
}

inline void FAssetDataGatherer::RequestAsyncCacheSave()
{
	DelayedSaveAsyncCacheStartTime.store(-UE_DOUBLE_BIG_NUMBER, std::memory_order_relaxed);
}

inline void FAssetDataGatherer::RequestDelayedAsyncCacheSave()
{
	if (DelayedSaveAsyncCacheStartTime.load(std::memory_order_relaxed) == 0.0)
	{
		DelayedSaveAsyncCacheStartTime.store(FPlatformTime::Seconds(), std::memory_order_relaxed);
	}
}

inline void FAssetDataGatherer::ClearRequestAsyncCacheSave()
{
	DelayedSaveAsyncCacheStartTime.store(0.0, std::memory_order_relaxed);
}

inline FGatheredResultsEvent& FAssetDataGatherer::GetGatheredResultsEvent()
{
	return GatheredResultsEvent;
}

inline void FAssetDataGatherer::SetGatherOnGameThreadOnly(bool bValue)
{
	bGatherOnGameThreadOnly.store(bValue, std::memory_order_relaxed);
}

inline bool FAssetDataGatherer::IsGatherOnGameThreadOnly() const
{
	return bGatherOnGameThreadOnly.load(std::memory_order_relaxed);
}

inline void FAssetDataGatherer::SetIsAdditionalMountSearchInProgress(bool bIsInProgress)
{
	bIsAdditionalMountSearchInProgress.store(bIsInProgress, std::memory_order_relaxed);
}