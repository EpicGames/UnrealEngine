// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDataGathererDiscoveryCache.h"

#include "AssetDataGathererPrivate.h"
#include "AssetRegistryPrivate.h"
#include "AssetRegistry/AssetRegistryTelemetry.h"
#include "Compression/CompressedBuffer.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Logging/LogMacros.h"
#include "Memory/SharedBuffer.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Serialization/Archive.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "TelemetryRouter.h"
#include "Templates/UniquePtr.h"

FArchive& operator<<(FArchive& Ar, FFileJournalFileHandle& H)
{
	Ar.Serialize(H.Bytes, sizeof(H.Bytes));
	return Ar;
}

namespace AssetDataGathererConstants
{
const FGuid DiscoveryCacheVersion(0x18A13604, 0x1BC44658, 0x97F7E0F0, 0x47A84CA2);
}

namespace UE::AssetDataGather::Private
{

FString FAssetDataDiscoveryCache::GetCacheFileName() const
{
	return FPaths::Combine(GGatherSettings.GetAssetRegistryCacheRootFolder(),
		TEXT("CachedAssetRegistryDiscovery.bin"));
}

void FAssetDataDiscoveryCache::LoadAndUpdateCache()
{
	if (bInitialized)
	{
		return;
	}
	TRACE_CPUPROFILER_EVENT_SCOPE(AssetDataGatherLoadDiscoveryCache);
	bInitialized = true;

	WriteEnabled = GGatherSettings.IsDiscoveryCacheWriteEnabled();
	CachedVolumes.Empty();
	if (!GGatherSettings.IsDiscoveryCacheReadEnabled() && WriteEnabled == EFeatureEnabled::Never)
	{
		return;
	}

	FString TestError;
	FFileJournalId TestJournalId;
	FFileJournalEntryHandle TestLatestJournalEntry;
	FString ProjectDir = FPaths::ProjectDir();
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	FString TestVolumeName = PlatformFile.FileJournalGetVolumeName(ProjectDir);
	EFileJournalResult TestResult = PlatformFile.FileJournalGetLatestEntry(*TestVolumeName,	TestJournalId,
		TestLatestJournalEntry, &TestError);
	bool bPlatformSupported = TestResult == EFileJournalResult::Success;

	if (bPlatformSupported)
	{
		// Do a quick test if can grab data from the journal file which does a similar call in IterateDirectory
		// If it fails we are very likely going to fail to iterate using journal files, so lets disable the cache
		FFileJournalData TestData = PlatformFile.FileJournalGetFileData(*FPaths::ConvertRelativePathToFull(ProjectDir), &TestError);
		if (!TestData.bIsValid)
		{
			bPlatformSupported = false;

			UE::Telemetry::AssetRegistry::FFileJournalErrorTelemetry Telemetry;
			Telemetry.Directory = FPaths::ConvertRelativePathToFull(ProjectDir);
			Telemetry.ErrorString = TestError;
			FTelemetryRouter::Get().ProvideTelemetry(Telemetry);
		}
	}

	bool bInvalidateEnabled = GGatherSettings.IsDiscoveryCacheInvalidateEnabled();
	bool bReadEnabled = GGatherSettings.IsDiscoveryCacheReadEnabled() && (bPlatformSupported || !bInvalidateEnabled);
	// Precalculate IfPlatformSupported -> Never if we already know the project doesn't support it
	if (WriteEnabled == EFeatureEnabled::IfPlatformSupported && !bPlatformSupported)
	{
		WriteEnabled = EFeatureEnabled::Never;
	}

	if ((!bReadEnabled && GGatherSettings.IsDiscoveryCacheReadEnabled()) ||
		(WriteEnabled == EFeatureEnabled::Never && GGatherSettings.IsDiscoveryCacheWriteEnabled() != EFeatureEnabled::Never))
	{
		const TCHAR* MissingOperation = (!bReadEnabled && WriteEnabled == EFeatureEnabled::Never) ? TEXT("read or written") :
			(!bReadEnabled ? TEXT("read") : TEXT("written"));
		UE_LOGF(LogAssetRegistry, Display,
			"PlatformFileJournal is not available on volume '%ls' of project directory '%ls', so AssetDiscovery cache will not be %ls. Unavailability reason:\n\t%ls",
			*TestVolumeName, *FPaths::ConvertRelativePathToFull(ProjectDir), MissingOperation, *TestError);

		UE::Telemetry::AssetRegistry::FFileJournalErrorTelemetry Telemetry;
		Telemetry.Directory = FPaths::ConvertRelativePathToFull(ProjectDir);
		Telemetry.ErrorString = TestError;
		FTelemetryRouter::Get().ProvideTelemetry(Telemetry);
	}

	if (!bReadEnabled)
	{
		return;
	}

	bool bReadSucceeded = TryReadCacheFile();
	if (!bReadSucceeded)
	{
		return;
	}

	if (!bInvalidateEnabled)
	{
		return;
	}

	bool bReportCacheErrorDiagnostics = false;
	for (TPair<FString, FCachedVolumeInfo>& VolumePair : CachedVolumes)
	{
		FString& VolumeName = VolumePair.Key;
		FCachedVolumeInfo& VolumeInfo = VolumePair.Value;

		TMap<FFileJournalFileHandle, FString> KnownDirectories;
		TSet<FString> ModifiedDirectories;
		for (TMap<FString, FCachedDirScanDir>::TIterator DirIter(VolumeInfo.Dirs); DirIter; ++DirIter)
		{
			FString& DirName = DirIter->Key;
			FCachedDirScanDir& DirData = DirIter->Value;
			// If not valid, we cannot remove the DirData yet because we will need it to get the list of
			// its child directories that exist in the tree, at the point when we rescan it
			if (DirData.bCacheValid)
			{
				if (DirData.JournalHandle != FileJournalFileHandleInvalid)
				{
					KnownDirectories.Add(DirData.JournalHandle, DirName);
				}
				else
				{
					// We have serialized DirData that doesn't have a valid handle and thus we can't check
					// if the directory has been modified. Mark it as invalid so it can be rescanned (same 
					// effect as appending to ModifiedDirectories but avoids the overhead)
					DirData.bCacheValid = false;
				}
			}
		}

		EFileJournalResult ReadModifiedResult = EFileJournalResult::Success;
		bool bReadModifiedSucceeded = false;
		if (!VolumeInfo.bJournalAvailable)
		{
			UE_LOGF(LogAssetRegistry, Warning,
				"PlatformFileJournal is not available on volume '%ls'. AssetRegistry discovery of files on this volume will be uncached. Unavailability reason:\n\t%ls",
				*VolumeName, *VolumeInfo.LastError);

			UE::Telemetry::AssetRegistry::FFileJournalErrorTelemetry MissingJournalTelemetry;
			MissingJournalTelemetry.Directory = VolumeName;
			MissingJournalTelemetry.ErrorString = VolumeInfo.LastError;
			FTelemetryRouter::Get().ProvideTelemetry(MissingJournalTelemetry);
		}
		else
		{
			EFileJournalResult Result = PlatformFile.FileJournalReadModified(*VolumeInfo.VolumeName,
				VolumeInfo.JournalId, VolumeInfo.NextJournalEntryToScan, KnownDirectories, ModifiedDirectories,
				VolumeInfo.NextJournalEntryToScan, &VolumeInfo.LastError);
			switch (Result)
			{
			case EFileJournalResult::Success:
				bReadModifiedSucceeded = true;
				break;
			case EFileJournalResult::JournalWrapped:
			{
				UE_LOGF(LogAssetRegistry, Display,
					"PlatformFileJournal journal has wrapped for volume '%ls'. AssetRegistry discovery of files on this volume will be uncached. Notes on wrapping:" "\r\n%ls",
					*VolumeName, *VolumeInfo.LastError);

				UE::Telemetry::AssetRegistry::FFileJournalWrappedTelemetry WrappedJournalTelemetry;
				WrappedJournalTelemetry.VolumeName = VolumeName;
				WrappedJournalTelemetry.JournalMaximumSize = VolumeInfo.JournalMaximumSize;
				FTelemetryRouter::Get().ProvideTelemetry(WrappedJournalTelemetry);
				break;
			}
			default:
			{
				UE_LOGF(LogAssetRegistry, Warning,
					"PlatformFileJournal is not available for volume '%ls'. AssetRegistry discovery of files on this volume will be uncached. Unavailability reason:" "\n\t%ls",
					*VolumeName, *VolumeInfo.LastError);
				bReportCacheErrorDiagnostics = true;

				UE::Telemetry::AssetRegistry::FFileJournalErrorTelemetry MissingJournalTelemetry;
				MissingJournalTelemetry.Directory = VolumeName;
				MissingJournalTelemetry.ErrorString = VolumeInfo.LastError;
				FTelemetryRouter::Get().ProvideTelemetry(MissingJournalTelemetry);
				break;
			}
			}
		}

		if (!bReadModifiedSucceeded)
		{
			VolumeInfo.JournalId = VolumeInfo.JournalIdOnDisk;
			VolumeInfo.NextJournalEntryToScan = VolumeInfo.NextJournalEntryOnDisk;
			VolumeInfo.Dirs.Empty();
		}
		else
		{
			for (const FString& ModifiedDirectory : ModifiedDirectories)
			{
				FCachedDirScanDir* DirData = VolumeInfo.Dirs.Find(ModifiedDirectory);
				if (DirData)
				{
					// We cannot remove the DirData yet because we will need it to get the list of its child
					// directories that exist in the tree, at the point when we rescan it
					DirData->bCacheValid = false;
					// For modified directories, also clear their JournalHandle; the directory might have been
					// marked modified because its JournalHandle changed, so we need to recalculate it.
					// For directories under the top-level directory we recalculate it automatically when we
					// rescan their parent directory (which we do because parent directories of changed directories
					// are also marked modified) but for top-level directories we never recalculate their JournalHandle
					// unless it has been set to invalid.
					DirData->JournalHandle = FileJournalFileHandleInvalid;
				}
			}
		}
	}

	if (bReportCacheErrorDiagnostics)
	{
#if WITH_EDITOR
		// Rename the discovery cache so it may remain on disk for inspection
		FString CachePath = GetCacheFileName();
		FStringView CacheDir, CacheFileName, CacheExtension;
		FPathViews::Split(CachePath, CacheDir, CacheFileName, CacheExtension);
		FString NewCacheFileName = FString::Printf(TEXT("%.*s-%s.%.*s"), CacheFileName.Len(), CacheFileName.GetData(), *FDateTime::Now().ToString(), CacheExtension.Len(), CacheExtension.GetData());
		FString NewCachePath = FPaths::Combine(CacheDir, NewCacheFileName);
		UE_LOGF(LogAssetRegistry, Warning, "Saving a copy of %ls to %ls for debug diagnostics.", *CachePath, *NewCachePath);
		if (!IFileManager::Get().Move(*NewCachePath, *CachePath, true /*Replace*/, true /*EvenIfReadOnly*/))
		{
			UE_LOGF(LogAssetRegistry, Warning, "Failed creating a copy of %ls at %ls", *CachePath, *NewCachePath);
		}
#endif
	}
}

void FAssetDataDiscoveryCache::SaveCache()
{
	if (WriteEnabled == EFeatureEnabled::Never)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(AssetDataGatherSaveDiscoveryCache);
	for (TPair<FString, FCachedVolumeInfo>& VolumePair : CachedVolumes)
	{
		VolumePair.Value.PreSave();
	}

	FString Filename = GetCacheFileName();

	FLargeMemoryWriter Writer;
	SerializeWriteCacheFile(Writer);
	FCompressedBuffer Compressed = FCompressedBuffer::Compress(FSharedBuffer::MakeView(Writer.GetView()));

	TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*Filename));
	if (!Ar)
	{
		UE_LOGF(LogAssetRegistry, Warning, "Could not write to DiscoveryCacheFile %ls.", *Filename);
		return;
	}
	Compressed.Save(*Ar);
}

bool FAssetDataDiscoveryCache::TryReadCacheFile()
{
	FSharedBuffer RawBuffer;
	bool bCacheValid = true;
	bool bCacheCorrupt = false;
	bool bCacheVersionMismatch = false;
	FString Filename = GetCacheFileName();
	{
		TUniquePtr<FArchive> CompressedFileArchive(IFileManager::Get().CreateFileReader(*Filename));
		if (!CompressedFileArchive || CompressedFileArchive->TotalSize() == 0)
		{
			bCacheValid = false;
		}
		else
		{
			FCompressedBufferReader CompressedBuffer(*CompressedFileArchive);
			if (CompressedBuffer.GetRawSize() == 0)
			{
				bCacheValid = false;
				bCacheCorrupt = true;
			}
			else
			{
				RawBuffer = CompressedBuffer.Decompress();
				if (RawBuffer.GetSize() != CompressedBuffer.GetRawSize())
				{
					bCacheValid = false;
					bCacheCorrupt = true;
				}
			}
		}
	}
	if (bCacheValid)
	{
		FMemoryReaderView Ar(RawBuffer.GetView());
		SerializeReadCacheFile(Ar, bCacheVersionMismatch);
		if (Ar.IsError())
		{
			bCacheValid = false;
			bCacheCorrupt = true;
		}
	}

	if (!bCacheValid)
	{
		if (bCacheVersionMismatch)
		{
			UE_LOGF(LogAssetRegistry, Display,
				"Version mismatch for AssetDiscovery cache %ls. AssetRegistry discovery of files will be uncached.",
				*Filename);
		}
		else if (bCacheCorrupt)
		{
			UE_LOGF(LogAssetRegistry, Warning,
				"Corrupt AssetDiscovery cache %ls. AssetRegistry discovery of files will be uncached.",
				*Filename);
		}
		else
		{
			UE_LOGF(LogAssetRegistry, Display,
				"No AssetDiscovery cache present at %ls. AssetRegistry discovery of files will be uncached.",
				*Filename);
		}
		CachedVolumes.Empty();
		return false;
	}

	for (TPair<FString, FCachedVolumeInfo>& Pair : CachedVolumes)
	{
		Pair.Value.InitializePlatformData();
	}
	return true;
}

void FAssetDataDiscoveryCache::SerializeReadCacheFile(FArchive& Ar, bool& bOutCacheVersionMismatch)
{
	FGuid Version;
	Ar << Version;
	bOutCacheVersionMismatch = Version != AssetDataGathererConstants::DiscoveryCacheVersion;
	if (bOutCacheVersionMismatch)
	{
		Ar.SetError();
		return;
	}

	CachedVolumes.Empty();
	Ar << CachedVolumes;
}

void FAssetDataDiscoveryCache::SerializeWriteCacheFile(FArchive& Ar)
{
	FGuid Version = AssetDataGathererConstants::DiscoveryCacheVersion;
	Ar << Version;
	Ar << CachedVolumes;
}

void FAssetDataDiscoveryCache::Shutdown()
{
	WriteEnabled = EFeatureEnabled::Never;

	CachedVolumes.Empty();
	while (!ScanQueueDirFullDatas.IsEmpty())
	{
		TPair<FString, FCachedDirScanDir> Pair;
		ScanQueueDirFullDatas.Dequeue(Pair);
	}
	while (!ScanQueueDirHandles.IsEmpty())
	{
		FDiscoveredSubDirData SubDirData;
		ScanQueueDirHandles.Dequeue(SubDirData);
	}
}

FCachedVolumeInfo& FAssetDataDiscoveryCache::FindOrAddVolume(FStringView PathOrVolumeName)
{
	FStringView VolumeNameView;
	FStringView Remainder;
	FPathViews::SplitVolumeSpecifier(PathOrVolumeName, VolumeNameView, Remainder);
	FString VolumeName(VolumeNameView);
	if (VolumeName.IsEmpty())
	{
		VolumeName = GEmptyVolumeName;
	}
	FCachedVolumeInfo& Volume = CachedVolumes.FindOrAdd(VolumeName);
	Volume.ConditionalConstruct(VolumeName);
	return Volume;
}

FCachedDirScanDir& FAssetDataDiscoveryCache::FindOrAddDir(FStringView Path, FCachedVolumeInfo** OutVolume)
{
	FCachedVolumeInfo& Volume = FindOrAddVolume(Path);
	if (OutVolume)
	{
		*OutVolume = &Volume;
	}
	return Volume.FindOrAddDir(Path);
}

void FAssetDataDiscoveryCache::RemoveDir(FStringView Path)
{
	FStringView VolumeNameView;
	FStringView Remainder;
	FPathViews::SplitVolumeSpecifier(Path, VolumeNameView, Remainder);
	FString VolumeName(VolumeNameView);
	FCachedVolumeInfo* Info = CachedVolumes.Find(VolumeName);
	if (!Info)
	{
		return;
	}
	Info->RemoveDirs({ FString(Path) });
}

FCachedVolumeInfo* FAssetDataDiscoveryCache::FindVolume(FStringView PathOrVolumeName)
{
	FStringView VolumeNameView;
	FStringView Remainder;
	FPathViews::SplitVolumeSpecifier(PathOrVolumeName, VolumeNameView, Remainder);
	FString VolumeName(VolumeNameView);

	return CachedVolumes.Find(VolumeName);
}

FCachedDirScanDir* FAssetDataDiscoveryCache::FindDir(FStringView Path, FCachedVolumeInfo** OutVolume)
{
	FCachedVolumeInfo* Volume = FindVolume(Path);
	if (OutVolume)
	{
		*OutVolume = Volume;
	}
	if (!Volume)
	{
		return nullptr;
	}
	return Volume->FindDir(Path);
}

bool FindOrAddIsReparsePointRecursive(FStringView DirName, FCachedVolumeInfo& Volume, bool& bOutAdded)
{
	bOutAdded = false;
	if (DirName.IsEmpty())
	{
		return false;
	}

	// We must use a pointer here since re-assigning to Existing later as a value will invoke 
	// copy assignment on a potentially invalid type
	FCachedDirScanDir* Existing = &Volume.FindOrAddDir(DirName, bOutAdded);

	if (!Existing->bIsInsideReparsePoint.IsSet())
	{
		// Usually bIsReparsePoint is already set because we calculated it when we encountered this directory in the directory scan of its parent directory.
		// But for top-level directories or directories that were found to be modified by the journal, we did not already encounter it.
		// Calculate it now manually if it is not already set.
		if (!Existing->bIsReparsePoint.IsSet())
		{
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			Existing->bIsReparsePoint = PlatformFile.IsSymlink(DirName.GetData()) == ESymlinkResult::Symlink;
		}

		// Base case for when we have found a reparse point, end the recursion and set true for all our previous calls
		if (Existing->bIsReparsePoint.GetValue())
		{
			Existing->bIsInsideReparsePoint = true;
			return true;
		}

		bool bAddedDuringRecursion = false;
		bool bIsInsideReparsePoint = FindOrAddIsReparsePointRecursive(FPaths::GetPath(DirName.GetData()), Volume, bAddedDuringRecursion);
		// If we added a new entry, then we may have invalidated Existing and need to fetch it again
		if (bAddedDuringRecursion)
		{
			bOutAdded = true;
			Existing = Volume.FindDir(DirName);
		}
		Existing->bIsInsideReparsePoint = bIsInsideReparsePoint;
	}

	return Existing->bIsInsideReparsePoint.GetValue();
}

bool FAssetDataDiscoveryCache::FindOrAddIsReparsePoint(FStringView DirName, FCachedVolumeInfo& Volume)
{
	bool bOutAdded = false;
	return FindOrAddIsReparsePointRecursive(DirName, Volume, bOutAdded);
}

void FAssetDataDiscoveryCache::QueueConsume()
{
	if (WriteEnabled == EFeatureEnabled::Never)
	{
		return;
	}
	FDiscoveredSubDirData SubDirData;
	while (ScanQueueDirHandles.Dequeue(SubDirData))
	{
		FString& DirName = SubDirData.DirNameAbsPath;
		FCachedVolumeInfo& Volume = FindOrAddVolume(DirName);
		if (!Volume.bJournalAvailable && WriteEnabled != EFeatureEnabled::Always)
		{
			continue;
		}
		FCachedDirScanDir& Existing = Volume.FindOrAddDir(DirName);
		Existing.JournalHandle = MoveTemp(SubDirData.JournalHandle);
		Existing.bIsReparsePoint = SubDirData.bIsReparsePoint;
	}

	TPair<FString, FCachedDirScanDir> ScanDirPair;
	while (ScanQueueDirFullDatas.Dequeue(ScanDirPair))
	{
		FString& DirName = ScanDirPair.Key;
		FCachedDirScanDir& ScanDir = ScanDirPair.Value;
		FCachedVolumeInfo& Volume = FindOrAddVolume(DirName);
		if (FindOrAddIsReparsePoint(DirName, Volume))
		{
			continue;
		}
		if (!Volume.bJournalAvailable && WriteEnabled != EFeatureEnabled::Always)
		{
			continue;
		}

		FCachedDirScanDir& Existing = Volume.FindOrAddDir(DirName);

		// Mark for removal any subpaths in the cache that no longer exist on disk
		if (!Existing.SubDirRelPaths.IsEmpty())
		{
			TSet<FString> StillExisting(ScanDir.SubDirRelPaths);
			for (FString& OldRelPath : Existing.SubDirRelPaths)
			{
				if (!StillExisting.Contains(OldRelPath))
				{
					Volume.DirsToRemove.Add(FPaths::Combine(DirName, OldRelPath));
				}
			}
		}

		// If neither the new entry nor the existing entry have the JournalHandle,
		// initialize it now.
		if (ScanDir.JournalHandle == FileJournalFileHandleInvalid)
		{
			if (Existing.JournalHandle != FileJournalFileHandleInvalid)
			{
				ScanDir.JournalHandle = Existing.JournalHandle;
			}
			else
			{
				FFileJournalData PlatformData =
					FPlatformFileManager::Get().GetPlatformFile().FileJournalGetFileData(*DirName);
				ScanDir.JournalHandle = PlatformData.JournalHandle;
			}
		}

		// These values get set in FindOrAddIsReparsePoint, make sure they dont get lost here
		ScanDir.bIsReparsePoint = Existing.bIsReparsePoint;
		ScanDir.bIsInsideReparsePoint = Existing.bIsInsideReparsePoint;
		// Confirm our handle is valid otherwise mark the cache as invalid
		ScanDir.bCacheValid = ScanDir.JournalHandle != FileJournalFileHandleInvalid;

		Existing = MoveTemp(ScanDir);
	}
}

void FAssetDataDiscoveryCache::QueueAdd(FString DirName, FCachedDirScanDir DirData)
{
	if (WriteEnabled == EFeatureEnabled::Never)
	{
		return;
	}
	ScanQueueDirFullDatas.Enqueue(TPair<FString, FCachedDirScanDir>(MoveTemp(DirName), MoveTemp(DirData)));
}

void FAssetDataDiscoveryCache::QueueAdd(FString DirName, FFileJournalFileHandle JournalHandle, bool bIsReparsePoint)
{
	if (WriteEnabled == EFeatureEnabled::Never)
	{
		return;
	}
	ScanQueueDirHandles.Enqueue(FDiscoveredSubDirData{MoveTemp(DirName), MoveTemp(JournalHandle), bIsReparsePoint});
}

void FCachedVolumeInfo::ConditionalConstruct(const FString& InVolumeName)
{
	if (!VolumeName.IsEmpty())
	{
		return;
	}
	VolumeName = InVolumeName;
	InitializePlatformData();
}

void FCachedVolumeInfo::PreSave()
{
	RemoveDirs(MoveTemp(DirsToRemove));
	RemoveInvalidDirs();
}

void FCachedVolumeInfo::InitializePlatformData()
{
	if (VolumeName.IsEmpty() || VolumeName == GEmptyVolumeName)
	{
		bJournalAvailable = false;
		JournalMaximumSize = 0;
		JournalIdOnDisk = FileJournalIdInvalid;
		NextJournalEntryOnDisk = FileJournalEntryHandleInvalid;
		JournalId = FileJournalIdInvalid;
		NextJournalEntryToScan = FileJournalEntryHandleInvalid;
	}
	else
	{
		EFileJournalResult Result = FPlatformFileManager::Get().GetPlatformFile().FileJournalGetLatestEntry(
			*VolumeName, JournalIdOnDisk, NextJournalEntryOnDisk, &LastError);
		bJournalAvailable = Result == EFileJournalResult::Success;
		JournalMaximumSize = bJournalAvailable ? FPlatformFileManager::Get().GetPlatformFile().FileJournalGetMaximumSize(*VolumeName) : 0;
		if (NextJournalEntryToScan == FileJournalEntryHandleInvalid)
		{
			JournalId = JournalIdOnDisk;
			check(!bJournalAvailable || NextJournalEntryOnDisk != FileJournalEntryHandleInvalid);
			NextJournalEntryToScan = NextJournalEntryOnDisk;
		}
	}
}

FCachedDirScanDir& FCachedVolumeInfo::FindOrAddDir(FStringView InPath)
{
	bool bAdded = false;
	return FindOrAddDir(InPath, bAdded);
}

FCachedDirScanDir& FCachedVolumeInfo::FindOrAddDir(FStringView InPath, bool& bOutAdded)
{
	uint32 PathHash = GetTypeHash(InPath);
	FCachedDirScanDir* ScanDir = Dirs.FindByHash(PathHash, InPath);
	bOutAdded = ScanDir == nullptr;
	if (bOutAdded)
	{
		ScanDir = &Dirs.AddByHash(PathHash, FString(InPath));
	}
	return *ScanDir;
}

void FCachedVolumeInfo::RemoveDirs(TArray<FString>&& InPaths)
{
	// Recursively remove RemoveDir directories; iterate by popping InPaths and pushing child dirs back on
	while (!InPaths.IsEmpty())
	{
		FString RemoveDir = InPaths.Pop(EAllowShrinking::No);
		FCachedDirScanDir DirData;
		if (Dirs.RemoveAndCopyValue(RemoveDir, DirData))
		{
			for (FString& RelPath : DirData.SubDirRelPaths)
			{
				InPaths.Add(FPaths::Combine(RemoveDir, RelPath));
			}
		}
	}
	InPaths.Empty(); // Free allocated memory
}

void FCachedVolumeInfo::RemoveInvalidDirs()
{
	for (TMap<FString, FCachedDirScanDir>::TIterator It = Dirs.CreateIterator(); It; ++It)
	{
		const FCachedDirScanDir& ScanDir = It->Value;
		if (!ScanDir.bCacheValid || ScanDir.JournalHandle == FileJournalFileHandleInvalid)
		{
			It.RemoveCurrent();
		}
	}
}

FCachedDirScanDir* FCachedVolumeInfo::FindDir(FStringView Path)
{
	uint32 PathHash = GetTypeHash(Path);
	return Dirs.FindByHash(PathHash, Path);
}

FArchive& operator<<(FArchive& Ar, FCachedVolumeInfo& Data)
{
	Ar << Data.Dirs;
	Ar << Data.VolumeName;
	Ar << Data.JournalId;
	Ar << Data.NextJournalEntryToScan;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FCachedDirScanDir& Data)
{
	Ar << Data.JournalHandle;
	Ar << Data.SubDirRelPaths;
	Ar << Data.Files;
	Ar << Data.bCacheValid;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FCachedDirScanFile& Data)
{
	Ar << Data.RelPath;
	Ar << Data.ModificationTime;
	Ar << Data.bIsReadOnly;
	return Ar;
}


} // namespace UE::AssetDataGather::Private
