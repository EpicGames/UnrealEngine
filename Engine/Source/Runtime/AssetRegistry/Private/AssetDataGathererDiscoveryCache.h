// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/MpscQueue.h"
#include "Containers/SortedMap.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformFile.h"
#include "Misc/DateTime.h"

class FArchive;

namespace UE::AssetDataGather::Private
{

enum class EFeatureEnabled : uint8
{
	Never,
	IfPlatformSupported,
	Always
};

enum class EFeatureEnabledReadWrite : uint32
{
	NeverRead		= 0x00,
	DefaultRead		= 0x01,
	AlwaysRead		= 0x02,
	ReadMask		= 0x0f,
	NeverWrite		= 0x00,
	DefaultWrite	= 0x10,
	AlwaysWrite		= 0x20,
	WriteMask		= 0xf0,
	Invalid			= 0xffffffff,
	NeverWriteNeverRead		= NeverWrite   | NeverRead,   // aka Never
	NeverWriteDefaultRead	= NeverWrite   | DefaultRead,
	NeverWriteAlwaysRead	= NeverWrite   | AlwaysRead,
	DefaultWriteNeverRead	= DefaultWrite | NeverRead,   
	DefaultWriteDefaultRead = DefaultWrite | DefaultRead, // aka Default
	DefaultWriteAlwaysRead	= DefaultWrite | AlwaysRead,
	AlwaysWriteNeverRead	= AlwaysWrite  | NeverRead,
	AlwaysWriteDefaultRead	= AlwaysWrite  | DefaultRead, // aka AlwaysWrite
	// NOT given the nickname "Always", because skipping invalidation is not what users would expect from "Always"
	AlwaysWriteAlwaysRead	= AlwaysWrite  | AlwaysRead,
};
ENUM_CLASS_FLAGS(EFeatureEnabledReadWrite);
void LexFromString(EFeatureEnabledReadWrite& OutValue, FStringView Text);

/** Data about a file in a cached directory used by FAssetDataDiscoveryCache to avoid the IO cost of rescanning. */
struct FCachedDirScanFile
{
	FString RelPath;
	FDateTime ModificationTime;
	bool bIsReadOnly = false;
};
FArchive& operator<<(FArchive& Ar, FCachedDirScanFile& Data);

/** Data about a directory used by FAssetDataDiscoveryCache to avoid the IO cost of rescanning. */
struct FCachedDirScanDir
{
	TArray<FString> SubDirRelPaths;
	TArray<FCachedDirScanFile> Files;
	FFileJournalFileHandle JournalHandle = FileJournalFileHandleInvalid;
	bool bCacheValid = false;

	/** bIsReparse True when this directory is a reparse point (symlink/junction/etc)
	  *			   It is optionally set to true or false when found during a directory scan
	  *			   If it is never found or set we will have to do an explicit IsSymLink on this directory
	*/
	TOptional<bool> bIsReparsePoint;

	/** bIsInsideReparsePoint True when this directory, or one of its parent directories are a ReparsePoint
	  *						  It is optional set when checking if this directory contains any ReparsePoints, Parent directories included
	*/
	TOptional<bool> bIsInsideReparsePoint;
};
FArchive& operator<<(FArchive& Ar, FCachedDirScanDir& Data);

/** Stores data from QueueAdd for subdirectories discovered during the directory scan, to be read during QueueConsume. */
struct FDiscoveredSubDirData
{
	FString DirNameAbsPath;
	FFileJournalFileHandle JournalHandle;
	TOptional<bool> bIsReparsePoint;
};

/** Data about a volume used by FAssetDataDiscoveryCache to avoid the IO cost of rescanning directories on the volume. */
class FCachedVolumeInfo
{
public:
	void ConditionalConstruct(const FString& InVolumeName);
	void PreSave();
	void InitializePlatformData();
	
	/** Assumes Path has already been normalized. */
	FCachedDirScanDir& FindOrAddDir(FStringView InPath);
	FCachedDirScanDir& FindOrAddDir(FStringView InPath, bool& bOutAdded);
	void RemoveDirs(TArray<FString>&& InPaths);
	void RemoveInvalidDirs();
	/** Assumes Path has already been normalized. */
	FCachedDirScanDir* FindDir(FStringView Path);

public:
	TMap<FString, FCachedDirScanDir> Dirs;
	FString VolumeName;
	FFileJournalId JournalId = FileJournalIdInvalid;
	FFileJournalEntryHandle NextJournalEntryToScan = FileJournalEntryHandleInvalid;

	// Transient
	/** Directories that we marked for recursive removal from CachedVolumes; we consume this during WriteCacheFile. */
	TArray<FString> DirsToRemove;
	FString LastError;
	FFileJournalId JournalIdOnDisk = FileJournalIdInvalid;
	FFileJournalEntryHandle NextJournalEntryOnDisk = FileJournalEntryHandleInvalid;
	uint64 JournalMaximumSize = 0;
	bool bJournalAvailable = false;
};
FArchive& operator<<(FArchive& Ar, FCachedVolumeInfo& Data);

/**
 * Keeps a record of the directories, files, and file timestamps that were found by the directory
 * scan the last time it ran, and invalidates the records that have been reported modified by
 * the PlatformFileJournal.
 * 
 * ThreadSafety: Not threadsafe, with the exception of QueueAdd functions, which can be called
 * at any time from any thread.
 * 
 */
class FAssetDataDiscoveryCache
{
public:
	FString GetCacheFileName() const;

	bool IsInitialized() const { return bInitialized; }
	EFeatureEnabled IsWriteEnabled() const { return WriteEnabled; }
	void Shutdown();

	void LoadAndUpdateCache();
	void SaveCache();

	FCachedVolumeInfo& FindOrAddVolume(FStringView PathOrVolumeName);
	/** Assumes Path has already been normalized. */
	FCachedDirScanDir& FindOrAddDir(FStringView Path, FCachedVolumeInfo** OutVolume = nullptr);
	void RemoveDir(FStringView Path);

	FCachedVolumeInfo* FindVolume(FStringView PathOrVolumeName);
	/** Assumes Path has already been normalized. */
	FCachedDirScanDir* FindDir(FStringView Path, FCachedVolumeInfo** OutVolume = nullptr);

	void QueueConsume();
	void QueueAdd(FString DirName, FCachedDirScanDir DirData);
	void QueueAdd(FString DirName, FFileJournalFileHandle JournalHandle, bool bIsReparsePoint);

private:
	bool TryReadCacheFile();
	void SerializeReadCacheFile(FArchive& Ar, bool& bOutCacheVersionMismatch);
	void SerializeWriteCacheFile(FArchive& Ar);

	bool FindOrAddIsReparsePoint(FStringView DirName, FCachedVolumeInfo& CachedVolume);

private:
	/** Cached information about each volume on disk that has mounted directories. */
	TSortedMap<FString, FCachedVolumeInfo> CachedVolumes;
	/** Childpath information about directories collected from ParallelFor during scanning. */
	TMpscQueue<TPair<FString, FCachedDirScanDir>> ScanQueueDirFullDatas;
	/** JournalHandle information about directories collected during ParallelFor during scanning. */
	TMpscQueue<FDiscoveredSubDirData> ScanQueueDirHandles;

	/** Used to implement initialization on demand. */
	bool bInitialized = false;
	/** Whether writing the cache is enabled for no volumes, all volumes, or ones with journal available. */
	EFeatureEnabled WriteEnabled = EFeatureEnabled::Never;
};

constexpr const TCHAR* GEmptyVolumeName = TEXT("<EmptyVolume>");

} // UE::AssetDataGather::Private