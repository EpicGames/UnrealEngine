// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetDataGatherer.h"

#include "AssetDataGathererDiscoveryCache.h"
#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "Misc/Optional.h"
#include "Misc/StringBuilder.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"

#include <atomic>

namespace UE::AssetDataGather::Private
{

class FMountDir;
struct FScanDirAndParentData;

/** Enum to specify files and directories that should be completed earlier than others */
enum class EPriority : uint8
{
	/** Game thread is blocked until the file/directory is completed */
	Blocking,
	/** Optional information (e.g. use of the ReferenceViewer) is unavailable until the file/directory is completed */
	High,
	/** Nothing has requested the file/directory yet */
	Normal,

	Highest = Blocking,
	Lowest = Normal,
};

/** Fields being set in a call to SetProperty */
struct FSetPathProperties
{
	/** The path (usually a plugin's root content path) was requested for scanning through e.g. ScanPathsSynchronous */
	TOptional<bool> IsOnAllowList;
	/**
	 * Set whether the given directory matches a deny list entry.
	 */
	TOptional<bool> MatchesDenyList;
	/**
	 * Set whether the given directory should ignore the deny list, even if it or its parent matches a deny list entry.
	 */
	TOptional<bool> IgnoreDenyList;
	/**
	 * The directory's list of direct file/subdirectory children has been scanned through a call to
	 * IFileManager::IterateDirectoryStat after process start or the last request to rescan it
	 */
	TOptional<bool> HasScanned;

	/** Used to early-exit from tree traversal when all properties have finished being handled */
	bool IsSet() const
	{
		return IsOnAllowList.IsSet() | HasScanned.IsSet() | MatchesDenyList.IsSet() | //-V792
			IgnoreDenyList.IsSet();
	}
};

/** To distinguish between assets and Verse source code */
enum class EGatherableFileType : uint8
{
	Invalid,
	Directory,
	PackageFile,
	VerseFile,
	VerseModule,
};

bool IsVerseFile(const EGatherableFileType FileType);

/** Information needed about a discovered asset file or path that is needed by the Discoverer */
struct FDiscoveredPathData
{
	/** The absolute path to the file on disk, relative to the directory that issued the scan */
	FString LocalAbsPath;
	/** The LongPackageName of the path (inherited from the MountDir) */
	FString LongPackageName;
	/** The relative path from the path's parent directory */
	FString RelPath;
	/** If the path is a file, the modification timestamp of the package file (that it had when it was discovered). */
	FDateTime PackageTimestamp;
	/** If the file is readonly */
	bool bIsReadOnly;
	/** The type of file that was found */
	EGatherableFileType Type = EGatherableFileType::Invalid;

	FDiscoveredPathData() = default;
	FDiscoveredPathData(FStringView InLocalAbsPath, FStringView InLongPackageName, FStringView InRelPath,
		const FDateTime& InPackageTimestamp, bool InIsReadonly, EGatherableFileType InType);
	FDiscoveredPathData(FStringView InLocalAbsPath, FStringView InLongPackageName, FStringView InRelPath,
		EGatherableFileType InType);
	void Assign(FStringView InLocalAbsPath, FStringView InLongPackageName, FStringView InRelPath,
		const FDateTime& InPackageTimestamp, bool InIsReadOnly, EGatherableFileType InType);
	void Assign(FStringView InLocalAbsPath, FStringView InLongPackageName, FStringView InRelPath,
		EGatherableFileType InType);

	/** Return the total amount of heap memory used by the gatherer (including not-yet-claimed search results). */
	SIZE_T GetAllocatedSize() const;
};

/** Information needed about a discovered asset file or path that is needed by the Gatherer */
struct FGatheredPathData
{
	/** The absolute path to the file on disk, relative to the directory that issued the scan */
	FString LocalAbsPath;
	/** The LongPackageName of the path (inherited from the MountDir) */
	FString LongPackageName;
	/** The modification timestamp of the package file (that it had when it was discovered) */
	FDateTime PackageTimestamp;
	/** If the file is readonly */
	bool bIsReadOnly;
	/** The type of file that was found */
	EGatherableFileType Type = EGatherableFileType::Invalid;

	FGatheredPathData() = default;
	FGatheredPathData(FStringView InLocalAbsPath, FStringView InLongPackageName, const FDateTime& InPackageTimestamp,
		bool InIsReadOnly, EGatherableFileType InType);
	explicit FGatheredPathData(const FDiscoveredPathData& DiscoveredData);
	explicit FGatheredPathData(FDiscoveredPathData&& DiscoveredData);
	void Assign(FStringView InLocalAbsPath, FStringView InLongPackageName, const FDateTime& InPackageTimestamp,
		bool InIsReadOnly, EGatherableFileType InType);
	void Assign(const FDiscoveredPathData& DiscoveredData);

	/**
	 * Return the total amount of heap memory used by the gatherer (including not-yet-claimed search results).
	 * Used for performance metrics.
	 */
	SIZE_T GetAllocatedSize() const;
};

/** Holds scheduling context reading assets during gathering defined in a ReadContext*/
struct FDirectoryReadTaskData
{
	FDirectoryReadTaskData()
		: UnblockRead(TEXT("UnblockingDirectoryReadTaskEvent"))
		, LastScheduledReadTask()
	{
	}

	~FDirectoryReadTaskData()
	{
		// We can verify that we properly signaled our task, but we cannot verify that the ReadTask is complete,
		// because it is possible that we insert the last outstanding read from the task and then process and remove
		// our DirectoryReadTaskData before the task can complete. However we can verify there is no outstanding work.
		check(UnblockRead.IsCompleted());
		check(OutstandingReads == 0);
	}

	UE::Tasks::FTaskEvent UnblockRead;
	UE::Tasks::FTask LastScheduledReadTask;
	int32 OutstandingReads = 0;
	/** Indicates a read for this directory was cancelled, implying that the directory will be rescheduled in a later gather. */
	bool bHasCanceledRead = false;
};

/** Holds data gathered from a package file on disk that is ingested into the cache and the AssetRegistry. */
struct FReadContext
{
	FName PackageName;
	FName Extension;
	FGatheredPathData AssetFileData;
	TArray<FAssetData*> AssetDataFromFile;
	FPackageDependencyData DependencyData;
	TArray<FString> CookedPackageNamesWithoutAssetData;
	TSharedPtr<FDirectoryReadTaskData> DirectoryReadTaskData;
	UE::AssetRegistry::Private::EOptionalBool HasMarkOfTheWeb = UE::AssetRegistry::Private::EOptionalBool::Unset;
	bool bCanAttemptAssetRetry = false;
	bool bResult = false;
	bool bCanceled = false;

	FReadContext(FName InPackageName, FName InExtension, FGatheredPathData&& InAssetFileData,
		TSharedPtr<FDirectoryReadTaskData> InDirectoryReadTaskData)
		: PackageName(InPackageName)
		, Extension(InExtension)
		, AssetFileData(MoveTemp(InAssetFileData))
		, DirectoryReadTaskData(MoveTemp(InDirectoryReadTaskData))
	{
		check(!AssetFileData.LocalAbsPath.IsEmpty());
		check(DirectoryReadTaskData.IsValid());
		check(DirectoryReadTaskData->OutstandingReads >= 0);
		DirectoryReadTaskData->OutstandingReads++;
	}

	~FReadContext()
	{
		check(DirectoryReadTaskData.IsValid());
		check(DirectoryReadTaskData->OutstandingReads > 0);
		DirectoryReadTaskData->OutstandingReads--;
	}
};

/** Manages the scheduling of FReadContexts */
class FFileReadScheduler
{
public:
	FFileReadScheduler(int32 InNumReadThreads, bool InGatherAssetPackageData, bool InGatherDependsData);
	~FFileReadScheduler();

	/** Returns true if AssetPackageData should be gathered. Constant during threading. */
	bool IsAssetPackageDataGatherEnabled() const;
	/** Returns true if dependency data should be gathered. Constant during threading. */
	bool IsDependsDataGatherEnabled() const;
	/** Add file to be scheduled for reading asynchronously. */
	void AddFile(const FName PackageName, FGatheredPathData&& AssetData);
	/** 
	* Schedule immediately and waits for all reads for the directories listed in WaitBatchDirectories. The 
	* list will be updated with all child directories waited on if the passed in directory was listed as recursive.
	*/
	void WaitOnDirectoriesAndUpdateWaitList(FWaitBatchDirectorySet& WaitBatchDirectories);
	/** 
	* Schedules all added files since the last call to Schedule to be picked up by the task system such that up to NumReadTaskThreads 
	* normal priority tasks may be in-flight at once. Existing or newly added directories in the given list of PriorityDirectories 
	* will be scheduled immediately and not bound by the schedulers NumReadTaskThreads limit.
	*/
	void Schedule(const TArray<FString>& PriorityDirectories);
	/** 
	* Populates the provided array with all completed reads (completed means read context is no longer
	* being processed. One must still check for success/failure/cancellation).
	*/
	void CollectCompletedReads(TArray<TUniquePtr<FReadContext>>& OutReadyContexts);
	/** 
	* Removes a scheduled directory from the scheduler. When all added files for a directory have been scheduled and completed, the 
	* directory should be removed to avoid overhead should a wait batch directory attempt to wait on the empty, still scheduled, directory.
	*/
	void RemoveScheduledDirectory(const FString& Directory);
	/**
	* Reads and stores gather results for asset files in the provided array. If time slicing is used
	* this method may exit before all read contexts are processed, and if so, will mark unprocessed
	* read contexts as cancelled so they may be rescheduled in another gatherer tick.
	*/
	void ProcessReadContexts(TArray<TUniquePtr<UE::AssetDataGather::Private::FReadContext>>&& ReadContexts, bool bIsTimeSliced);
	/**
	 * Releases all allocated read task loaders created during gathering.
	 * Must be called while gathering is not in progress.
	 */
	void FreeReadTaskLoaders();
	/** Frees unnecessary allocations the scheduler may have accrued. */
	void Shrink();
private:
	/** 
	* Schedules the specified directory and read contexts for reading inside of the provided Task Limiter. 
	* If bUnblockImmediately is true, the limiter is not used.
	*/
	void ScheduleDirectory(UE::Tasks::FTaskConcurrencyLimiter& TaskLimiter, const FString& DirectoryPath, 
		TArray<TUniquePtr<FReadContext>>&& ReadContexts, bool bUnblockImmediately);

	/** Map of directories to the files within that directory (non-recursive) needing scheduling. */
	TMap<FString, TArray<TUniquePtr<FReadContext>>> DirectoryReadsToSchedule;
	/** List of FReadContexts that have been populated (or cancelled) from worker threads requiring processing. */
	TMpscQueue<TUniquePtr<UE::AssetDataGather::Private::FReadContext>> ReadyReadContexts;
	/** Freelist of loaders created on task threads during gathering.*/
	TLockFreePointerListFIFO<FNonBufferingReadOnlyArchive, 0> ReadTaskLoaderFreeList;
	/** Used to control how parallel background asset file reading can be. */
	UE::Tasks::FTaskConcurrencyLimiter ReadConcurrencyLimiter;
	/**
	* Tree of directories being gathered asynchronously. The FDirectoryReadTaskData is used to trigger and track the
	* FTask performing file reads for that directory. We scheduled file reads by the directory they reside in to avoid
	* contention caused by the OS filesystem.
	*/
	TUniquePtr<TDirectoryTree<TSharedPtr<UE::AssetDataGather::Private::FDirectoryReadTaskData>>> ScheduledDirectories;
	/**
	* Number of parallel read tasks allowed while gathering (wait batches are allowed to run in parallel to these tasks
	* allowing for > NumReadTaskThreads to possibly be in flight at once.)
	*/
	int32 NumReadTaskThreads;
	/** True if AssetPackageData should be gathered. Constant during threading. */
	bool bGatherAssetPackageData;
	/** True if dependency data should be gathered. Constant during threading. */
	bool bGatherDependsData;
};

/** A container to efficiently receive files from discovery and search the container to raise priority of a path */
class FFilesToSearch
{
public:
	/** Add the given FilePath as a single file with blocking priority. */
	void AddPriorityFile(FGatheredPathData&& FilePath);
	/** Add the given FilePaths with normal priority. */
	void AddFiles(TArray<FGatheredPathData>&& FilePaths);
	/** Re-add a file from PopFront that we didn't have time to process. */
	void AddFileAgainAfterTimeout(FGatheredPathData&& FilePath);
	/**
	 * Re-add a file from PopFront that uses a not-yet-loaded class for later retry.
	 * Later retry files are not returned from PopFront until RetryLaterRetryFiles is called.
	 */
	void AddFileForLaterRetry(FGatheredPathData&& FilePath);
	/** Move all of the LaterRetry files into the main list so that they are once again visible to PopFront. */
	void RetryLaterRetryFiles();

	/** In coarse priority order, pop the given number of files out of the container. */
	template <typename AllocatorType>
	void PopFront(TArray<FGatheredPathData, AllocatorType>& Out, int32 NumToPop);

	/**
	 * Search the container for all files at or under the given path and set their priority.
	 * Path might be a directory name or a file BaseNameWithPath, or a file BaseNameWithPath.Extension.
	*/
	void PrioritizePath(FPathExistence& QueryPath, EPriority Priority);

	void PopPriorityDirectories(TArray<FString>& OutPriorityDirectories);

	/** How many files with blocking priority are in the container. */
	int32 NumBlockingFiles() const;

	/** Reduce memory used in buffers. */
	void Shrink();
	/** How many files are in the container, including unavailable files. */
	int32 Num() const;
	/** How many files are returnable from PopFront. */
	int32 GetNumAvailable() const;
	/** How much memory is used by the container, not counting sizeof(*this). */
	SIZE_T GetAllocatedSize() const;
private:
	TRingBuffer<FString> PriorityDirectories;
	TRingBuffer<UE::AssetDataGather::Private::FGatheredPathData> NormalPriorityFiles;
	TRingBuffer<UE::AssetDataGather::Private::FGatheredPathData> BlockingFiles;
	TRingBuffer<UE::AssetDataGather::Private::FGatheredPathData> LaterRetryFiles;
	int32 AvailableFilesNum = 0;
};


/** Stores a LocalAbsPath and existence information. A file system query is issued the first time the data is needed. */
struct FPathExistence
{
	/** What kind of thing we found at the given path */
	enum EType
	{
		Directory,
		File,
		MissingButDirExists,
		MissingParentDir,
	};

	explicit FPathExistence(FStringView InLocalAbsPath);
	const FString& GetLocalAbsPath() const;
	EType GetType();
	FDateTime GetModificationTime();
	bool IsReadOnly();

	/** If the path is a directory or file, return the path. Otherwise return the parent directory, if it exists. */
	FStringView GetLowestExistingPath();

	/** Issue the file system query if not already done. */
	void LoadExistenceData();
	bool HasExistenceData() const;
	void SetConfirmedExists(bool bValue);

private:
	FString LocalAbsPath;
	FDateTime ModificationTime;
	EType PathType = EType::MissingParentDir;
	bool bIsReadOnly = false;
	bool bHasExistenceData = false;
};

/**
 * Tree data node representing a directory in the scan; direct subdirectories are stored as other FScanDir instances
 * referenced from the SubDirs array. Directories are removed from the tree once their scans are finished to save memory.
 * Queries take into account that deleted nodes have been completed.
 *
 * This class is not ThreadSafe; The FAssetDataDiscovery reads/writes its data only while holding TreeLock.
 */
class FScanDir : public FRefCountedObject
{
public:
	/** Holder for fields on FScanDir that are inherited from parents */
	struct FInherited
	{
		FInherited() = default;

		/** Whether the ScanDir is in the allow list. */
		bool bIsOnAllowList = false;
		/** Whether the ScanDir is under one of the folders on the deny list. */
		bool bMatchesDenyList = false;
		/** Whether the ScanDir has been set to ignore the deny list. */
		bool bIgnoreDenyList = false;

		bool IsMonitored() const;
		bool IsOnDenyList() const;
		bool IsOnAllowList() const;
		bool HasSetting() const;
		FInherited(const FInherited& Parent, const FInherited& Child);
	};

	FScanDir(FMountDir& InMountDir, FScanDir* InParent, FStringView RelPath);
	~FScanDir();

	/**
	 * Marks that this ScanDir is no longer in use and clear its data. The ScanDir will remain allocated until all threads
	 * have dropped their reference to it.
	 */
	void Shutdown();

	/** Check whether this ScanDir is alive; it may have been marked for destruction and cleared on another thread. */
	bool IsValid() const;

	FMountDir* GetMountDir() const;

	/** Get this ScanDir's RelPath from its Parent */
	FStringView GetRelPath() const;

	/** Calculate this ScanDir's full absolute path by accumulating RelPaths from parents and append it. */
	void AppendLocalAbsPath(FStringBuilderBase& OutFullPath) const;
	/** Calculate this ScanDir's full absolute path by accumulating RelPaths from parents and return it as a string. */
	FString GetLocalAbsPath() const;
	/** Calculate the relative path from the MountPoint to this ScanDir and append it to OutRelPath.  */
	void AppendMountRelPath(FStringBuilderBase& OutRelPath) const;
	/** Calculate the relative path from the MountPoint to this ScanDir and return it as a string.  */
	FString GetMountRelPath() const;

	/**
	 * Report whether the given RelPath is allow listed and not deny listed, based on parent data and
	 * on the discovered settings on Stem ScanDirs between *this and the leaf path.
	 */
	void GetMonitorData(FStringView InRelPath, const FInherited& ParentData, FInherited& OutData) const;
	/** Return whether this scandir is allow listed and not deny listed and hence needs to be or has been scanned. */
	bool IsMonitored(const FInherited& ParentData) const;

	/** Report whether this ScanDir will be scanned in the current or future Tick. */
	bool ShouldScan(const FInherited& ParentData) const;
	/** Report whether this ScanDir has been scanned. */
	bool HasScanned() const;
	/** Report whether this ScanDir is complete: has scanned or should not scan, and all subdirs have completed. */
	bool IsComplete() const;

	/** Return the memory used by the tree under this ScanDir; excludes sizeof(*this). */
	SIZE_T GetAllocatedSize() const;
	/**
	 * Find the Direct parent of InRelPath, or a fallback. Will return null only if !bIsDirectory and InRelPath is empty.
	 * The fallback is returned if InRelPath has already completed and been deleted, or if InRelPath is not monitored.
	 * The fallback is the lowest existing parent directory of InRelPath.
	 */
	FScanDir* GetControllingDir(FStringView InRelPath, bool bIsDirectory, const FInherited& ParentData,
		FInherited& OutData, FString& OutRelPath);

	/**
	 * Set values of fields on the given directory indicated by InRelPath for all properties existing on InProperties.
	 * @param bInOutMadeChanges Set to true if any changes are made. Caller is responsible for initializing to false.
	 */
	void TrySetDirectoryProperties(FPathExistence& QueryPath, FStringView InRelPath, FInherited& ParentData,
		const FSetPathProperties& InProperties, FScanDirAndParentData& OutControllingDir,
		FStringView& OutControllingDirRelPath, bool& bInOutMadeChanges);
	/**
	 * Mark that the given file has already been scanned, so that it will not be double reporting in the upcoming
	 * directory scan, if one is upcoming.
	 */
	void MarkFileAlreadyScanned(FStringView BaseName);

	/**
	 * Called from the Tick; handle the list of subdirs and files that were found from IterateDirectoryStat called
	 * on this ScanDir, reporting discovered files and updating status variables.
	 */
	void SetScanResults(FStringView LocalAbsPath, const FInherited& ParentData,
		TArrayView<FDiscoveredPathData>& InOutSubDirs, TArrayView<FDiscoveredPathData>& InOutFiles);
	/**
	 * Update the completion state of this ScanDir and all ScanDirs under it based on each dir's scan status and its
	 * child dirs' completion state. Add any ScanDirs that need to be scanned to OutScanRequests.
	 */
	void Update(TArray<FScanDirAndParentData>& OutScanRequests, const FScanDir::FInherited& ParentData);

	FScanDir* GetFirstIncompleteScanDir();

	/** Thread-synchronization helper - return true if the Tick thread is in the middle of scanning this directory. */
	bool IsScanInFlight() const;
	/** Thread-synchronization helper - set that the Tick thread is starting/done with the scan of this directory. */
	void SetScanInFlight(bool bInScanInFlight);
	/**
	 * Thread-synchronization helper - report whether a non tick thread has marked that this directory is changed or
	 * invalidated and the scan should be thrown out.
	 */
	bool IsScanInFlightInvalidated() const;
	/**
	 * Thread-synchronization helper - set that the current ongoing scan is invalidated, or clear the marker from
	 * the tick thread once it has been consumed.
	 */
	void SetScanInFlightInvalidated(bool bInValidated);

	/** Set completion flags on this and its parents (and optionally its descendents) so that it will be updated again. */
	void MarkDirty(bool bMarkDescendents);

	/** Minimize data in internal buffers. */
	void Shrink();

protected:

	/**
	 * Setting to complete marks that this directory has been scanned, and all of its subdirectories have scanned
	 * as well, so it can be skipped when it or its parent is waited on. Setting back to incomplete can occur when a
	 * AssetDataGatherer client wants to rescan the directory.
	 */
	void SetComplete(bool bInIsComplete);
	/** 
	 * Return whether this ScanDir has direct settings (allow list, deny list, etc) that we need to preserve on it
	 * even if has finished scanning.
	 */
	bool HasPersistentSettings() const;

	/** Find the ScanDir subdirectory for the given basename, or return nullptr. */
	FScanDir* FindSubDir(FStringView SubDirBaseName);
	const FScanDir* FindSubDir(FStringView SubDirBaseName) const;
	/** Find the ScanDir subdirectory for the given basename, and add it if it does not exist. */
	FScanDir& FindOrAddSubDir(FStringView SubDirBaseName);
	/**
	 * Find the ScanDir subdirectory for the given basename, and if it exists, Shutdown and remove it from SubDirs,
	 * which will eventually delete it.
	 */
	void RemoveSubDir(FStringView SubDirBaseName);

	/** Find the index of the subdir with the given Relative path. */
	int32 FindLowerBoundSubDir(FStringView SubDirBaseName);

	/** Call the given lambda void(FScanDir&) on each existing SubDir. */
	template <typename CallbackType> void ForEachSubDir(const CallbackType& Callback);
	/**
	 * Call the given lambda void(FScanDir&) on each present-in-memory descedent ScanDir of this.
	 * Does not look for directories on disk, only the ones that have already been created in memory.
	 * Depth-first-search traversal, called on parents before children.
	 */
	template <typename CallbackType> void ForEachDescendent(const CallbackType& Callback);

	TArray<TRefCountPtr<FScanDir>> SubDirs; // Sorted
	TArray<FString> AlreadyScannedFiles; // Unsorted
	FMountDir* MountDir = nullptr;
	FScanDir* Parent = nullptr;
	FString RelPath;
	/** Whether each piece of the inherited data has been set directly on this directory */
	FInherited DirectData;
	bool bHasScanned = false;
	bool bScanInFlight = false;
	bool bScanInFlightInvalidated = false;
	bool bIsComplete = false;
};

/**
 *  A refcounted pointer to a ScanDir, along with the ParentData found when tracing down from the root
 * path to the ScanDir. Used to return list of directories needing scanning from Update.
 */
struct FScanDirAndParentData
{
	TRefCountPtr<FScanDir> ScanDir;
	FScanDir::FInherited ParentData;
};

/**
 * Gather data about a MountPoint that has been registered with FPackageName
 * The FMountDir holds a FScanTree with information about each directory (that is pruned when not in use).
 * It also holds data that is needed only per MountPoint, such as the packagename.
 * It also holds data per subdirectory that is more performant to hold in a map rather than to require the FScanTrees to
 * be kept.
 *
 * This class is not ThreadSafe; The FAssetDataDiscovery reads/writes its data only while holding TreeLock.
 */
class FMountDir
{
public:
	FMountDir(FAssetDataDiscovery& InDiscovery, FStringView LocalAbsPath, FStringView PackagePath);
	~FMountDir();

	/** The local path from FPackageName, absolute d:\root\Engine\Content rather relative ../../../Engine/Content. */
	FStringView GetLocalAbsPath() const;
	/** The package path from FPackageName. */
	FStringView GetLongPackageName() const;

	/** Return the FAssetDataDiscovery that owns this FMountDir. */
	FAssetDataDiscovery& GetDiscovery() const;

	/** Find the direct parent of InRelPath, or the lowest fallback. See FScanDir::GetControllingDir. */
	FScanDir* GetControllingDir(FStringView LocalAbsPath, bool bIsDirectory, FScanDir::FInherited& OutParentData,
		FString& OutRelPath);
	/** Return the memory used by the tree under this MountDir, except that sizeof(*this) is excluded. */
	SIZE_T GetAllocatedSize() const;

	/** Report whether this MountDir is complete: all ScanDirs under it either have scanned or should not scan. */
	bool IsComplete() const;

	/** Report the collapsed data for the scandir - allow list, deny list, etc. Returns false data for non child paths. */
	void GetMonitorData(FStringView InLocalAbsPath, FScanDir::FInherited& OutData) const;
	/**
	 * Return whether the given path is a child path of *this and is allow listed and is not deny listed, which means
	 * it will be or has been scanned.
	 */
	bool IsMonitored(FStringView InLocalAbsPath) const;

	/*
	 * Set values of fields on the given directory for all of the properties existing on InProperties.
	 */
	void TrySetDirectoryProperties(FPathExistence& QueryPath, const FSetPathProperties& InProperties,
		FScanDirAndParentData* OutControllingDir = nullptr,
		FStringView* OutControllingDirRelPath = nullptr,
		bool* bOutMadeChanges = nullptr);
	/** Update all incomplete ScanDirs under this MountDir and add any that need to be scanned to OutScanRequests. */
	void Update(TArray<FScanDirAndParentData>& OutScanRequests);

	FScanDir* GetFirstIncompleteScanDir();

	/** Record a directory under the MountDir has been scanned, used to detect if configuration occurs after scanning. */
	void SetHasStartedScanning();
	/** Minimize data in internal buffers. */
	void Shrink();

	/** Record that the MountDir is rooted at a childpath of this. The childpath will not be scanned by this. */
	void AddChildMount(FMountDir* ChildMount);
	/** Mark that a childpath MountDir is being deleted and the childpath should be scanned again by this. */
	void RemoveChildMount(FMountDir* ChildMount);
	/**
	 * Remove all childmounts. Does not handle properly updating the MountDir to reown those paths;
	 * this is used during destruction all MountDirs.
	 */
	void OnDestroyClearChildMounts();
	/**
	 * Record the backpointer to the parent mountdir that this mountdir's path is a child path of,
	 * or null if the parent no longer exists.
	 */
	void SetParentMount(FMountDir* ParentMount);
	/** Return the parent MountDir. */
	FMountDir* GetParentMount() const;
	/** Return the MountDirs that have been recorded as ChildMounts. */
	TArray<FMountDir*> GetChildMounts() const;

protected:
	/** Inspect the Discovery's DenyLists and add the ones applicable to this into this MountDir's set of DenyLists. */
	void UpdateDenyList();
	/** Mark that given path needs to be reconsidered by Update. */
	void MarkDirty(FStringView MountRelPath);

	/** Add a ChildMountPath, which indicates that another MountPoint owns that path and this Mount should not scan it. */
	void AddChildMountPath(FStringView MountRelPath);
	/** Remove a ChildMountPath, return whether there was a match to be removed. */
	bool RemoveChildMountPath(FStringView MountRelPath);
	/** Report whether the given Path is equal to or a child of an existing ChildMountPath. */
	bool IsChildMountPath(FStringView MountRelPath) const;

	/** Child mount paths; these directories should not be scanned by this MountDir. */
	TArray<FString> ChildMountPaths;
	/**
	 * Set of relative path from the MountDir paths that should not be scanned, because they were requested
	 * deny listed by clients or because a childmount owns them.
	 */
	TSet<FString> RelPathsDenyList;
	/** Absolute path to the root of the MountDir in the local file system. */
	FString LocalAbsPath;
	/** LongPackageName that was assigned to the MountDir in FPackageName. */
	FString LongPackageName;
	/**
	 * ScanDir for the root directory of this MountDir; child paths to scan will be created (and destroyed after use)
	 * as children of the ScanDir (with the exception of childmounts).
	 */
	TRefCountPtr<FScanDir> Root;
	/** Backpointer to the Discovery that owns this MountDir. */
	FAssetDataDiscovery& Discovery;
	/**
	 * If this is a nested MountDir (a nested path was registered with FPackageName) ParentMount is a
	 * pointer to the FMountDir that corresponds to the registered parent directory.
	 */
	FMountDir* ParentMount = nullptr;
	/** Records whether any directory at or under the MountDir's root has been scanned. */
	bool bHasStartedScanning = false;
};

/** Subsystem that discovers the files that FAssetDataGatherer should process.
 */
class FAssetDataDiscovery : public FRunnable
{
public:
	FAssetDataDiscovery();
	virtual ~FAssetDataDiscovery();


	// Controlling Async behavior

	/** Start the async thread, if this Gatherer was created async. Does nothing if not async or already started. */
	void StartAsync();

	// FRunnable implementation
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;

	/** Signals to end the thread and waits for it to close before returning */
	void EnsureCompletion();

	bool IsSynchronous() const;

	// Receiving Results and reading properties (possibly while tick is running)
	/** Gets search results from the file discovery. */
	void GetAndTrimSearchResults(bool& bOutIsComplete, TArray<FString>& OutDiscoveredPaths,
		FFilesToSearch& OutFilesToSearch, int32& OutNumPathsToSearch);
	/** Get diagnostics for telemetry or logging. */
	void GetDiagnostics(float& OutCumulativeDiscoveryTime, int32& OutNumCachedDirectories,
		int32& OutNumUncachedDirectories);
	/** Wait (joining in on the tick) until all currently monitored paths have been scanned. */
	void WaitForIdle(double EndTimeSeconds);
	bool IsIdle() const;
	void OnInitialSearchCompleted();
	void OnAdditionalMountSearchCompleted();

	/** Optionally set some scan properties for the given paths and then wait for their scans to finish. */
	void SetPropertiesAndWait(TArrayView<FPathExistence> QueryPaths, bool bAddToAllowList, bool bForceRescan,
		bool bIgnoreDenyListScanFilters);
	/** Return whether the given path is allowed due to e.g. TrySetDirectoryProperties with IsOnAllowList. */
	bool IsOnAllowList(FStringView LocalAbsPath) const;
	/** Return whether the given path matches the deny list and has not been marked IgnoreDenyList. */
	bool IsOnDenyList(FStringView LocalAbsPath) const;
	/** Return whether the path should or has been scanned because it is on the AllowList and not on the DenyList. */
	bool IsMonitored(FStringView LocalAbsPath) const;
	/** Return whether the discovery cache can be written. */
	bool IsCacheWriteEnabled() const;
	/** Return the memory used by *this. sizeof(*this) is not included. */
	SIZE_T GetAllocatedSize() const;

	// Events and setting of properties (possibly while tick is running)
	/**
	 * Register the given LocalAbsPath/LongPackageName pair that came from FPackageName's list of mount points as
	 * a mountpoint to track. Will not be scanned until allow listed.
	 */
	void AddMountPoint(const FString& LocalAbsPath, FStringView LongPackageName, bool& bOutAlreadyExisted);
	/** Remove the mountpoint because FPackageName has removed it. */
	void RemoveMountPoint(const FString& LocalAbsPath);
	/** Raise the priority until completion of scans of the given path and its subdirs. */
	void PrioritizeSearchPath(const FString& LocalAbsPath, EPriority Priority);
	/** Set properties on the directory, called when files are requested to be on an allow/deny list or rescanned. */
	void TrySetDirectoryProperties(const FString& LocalAbsPath, const FSetPathProperties& Properties,
		bool bConfirmedExists);
	/** Event called from the directory watcher when a directory is created. It will be scanned if IsMonitored. */
	void OnDirectoryCreated(FStringView LocalAbsPath);
	/** Event called from the directory watcher when files are created. Each will be reported if IsMonitored. */
	void OnFilesCreated(TConstArrayView<FString> LocalAbsPaths);

private:
	/** Find the MountDir with a root that contains LocalAbsPath, finding the lowest child if there are multiple. */
	FMountDir* FindContainingMountPoint(FStringView LocalAbsPath);
	const FMountDir* FindContainingMountPoint(FStringView LocalAbsPath) const;
	/** Find the mountpoint with the given root. */
	FMountDir* FindMountPoint(FStringView LocalAbsPath);
	/**
	 * Find the mountpoint with the given root, creating it if it does not already exist.
	 * Handle registration of child mountdirs.
	 */
	FMountDir& FindOrAddMountPoint(FStringView LocalAbsPath, FStringView LongPackageName);
	/** Execute the search on the sorted-by-path mountdirs, returning the first mountdir with root >= the given path. */
	int32 FindLowerBoundMountPoint(FStringView LocalAbsPath) const;

	/** Run the tick, either called from the async Run or called on thread from a thread executing a synchronous wait. */
	void TickInternal(bool bTickAll);
	/** Update all incomplete ScanDirs under all MountDirs and add any that need to be scanned to OutScanRequests. */
	void UpdateAll(TArray<FScanDirAndParentData>& OutScanRequests);
	/** Mark whether this discoverer has finished and is idle. Update properties dependent upon the idle state. */
	void SetIsIdle(bool bInIdle);
	void SetIsIdle(bool bInIdle, double& TickStartTime);

	/** Store the given discovered files and directories in the results. */
	void AddDiscovered(FStringView DirAbsPath, FStringView DirPackagePath, TConstArrayView<FDiscoveredPathData> SubDirs,
		TConstArrayView<FDiscoveredPathData> Files);
	/** Store the given specially reported single file in the results. */
	void AddDiscoveredFile(FDiscoveredPathData&& File);

	/**
	   For a given file path, determine the type it should be gathered as.
	   For Verse files, this does _not_ verify the file matches Verse naming conventions, and you probably want to call
	   `DoesPathContainInvalidCharacters` instead.
	 */
	static EGatherableFileType GetFileType(FStringView FilePath);

	/** Handle the actions necessary for a single created file. */
	void OnFileCreated(const FString& LocalPaths);
	/** Helper for TrySetDirectoryProperties, called from within the critical section. */
	void TrySetDirectoryPropertiesInternal(FPathExistence& QueryPath, const FSetPathProperties& Properties);

	/** Add the given path as a MountPoint and update child registrations. */
	void AddMountPointInternal(const FString& LocalAbsPath, FStringView LongPackageName, bool& bOutAlreadyExisted);
	/** Remove the given path as a MountPoint and update child registrations. */
	void RemoveMountPointInternal(const FString& LocalAbsPath);

	/** Minimize memory usage in the buffers used during gathering. */
	void Shrink();

	/** Scoped guard for pausing the asynchronous tick. */
	struct FScopedPause
	{
		FScopedPause(const FAssetDataDiscovery& InOwner);
		~FScopedPause();
		const FAssetDataDiscovery& Owner;
	};

private:
	/**
	 * Protect access to data in the ScanDir tree which can be read/write from the tick or from SetProperties.
	 * To prevent DeadLocks, TreeLock must not be entered while holding ResultsLock.
	 */
	mutable FGathererCriticalSection TreeLock;
	/**
	 * Prevent simultaneous ticks from two different threads and protect access to Tick-specific data.
	 * Can only be queried or taken while holding TreeLock, but can continue to be held after exiting TreeLock.
	 */
	mutable FThreadOwnerSection TickOwner;
	/**
	 * Protect access to the data written from tick and read/written from GetAndTrimSearchResults.
	 * ResultsLock can be entered while holding TreeLock.
	 */
	mutable FGathererCriticalSection ResultsLock;


	// Variable section for variables that are constant during threading.

	/** Deny list of full absolute paths. Child paths will not be scanned unless requested to ignore DenyLists. */
	TArray<FString> LongPackageNamesDenyList;
	/** DenyList of relative paths in each mount. Child paths will not be scanned unless requested to ignore DenyLists. */
	TArray<FString> MountRelativePathsDenyList;
	/** Thread to run the discovery FRunnable on. Read-only while threading is possible. */
	FRunnableThread* Thread;
	/**
	 * True if async discovery is enabled, false if e.g. singlethreaded or disabled by commandline.
	 * Even when enabled, discovery is still synchronous until StartAsync is called.
	 */
	bool bAsyncEnabled;


	// Variable section for variables that are atomics read/writable from outside critical sections.

	/**
	 * Whether this Discoverer has finished all work (may be still present in the results.)
	 * Readable anywhere. Writable only within TreeLock.
	 */
	std::atomic<bool> bIsIdle;
	/**
	 * Set to true after editing PriorityScanDirs to inform a running Tick that it should abort its
	 * current batch to reduce latency in handling the new priorities.
	 */
	std::atomic<bool> bPriorityDirty;
	/** > 0 if we've been asked to abort work in progress at the next opportunity. */
	std::atomic<uint32> IsStopped;
	/** > 0 if we've been asked to pause the worker thread so a synchronous function can take over the tick. */
	mutable std::atomic<uint32> IsPaused;
	/**
	 * Number of directories that have been discovered and IsMonitored but have not yet been scanned.
	 * Used for progress tracking.
	 */
	FThreadSafeCounter NumDirectoriesToScan;
	/** Triggered after scanning PriorityDirs to inform threads waiting on a PriorityDir to recheck conditions. */
	FEventRef PriorityDataUpdated{ EEventMode::ManualReset };

	// Variable section for variables that are read/writable only within ResultsLock.

	/** Directories found in the scan; may be empty. */
	TArray<FString> DiscoveredDirectories;

	/** Files found found in the scan. */
	struct FDirectoryResult
	{
		FDirectoryResult(FStringView InDirAbsPath, TConstArrayView<FDiscoveredPathData> InFiles);
		FString DirAbsPath;
		TArray<FGatheredPathData> Files;
		SIZE_T GetAllocatedSize() const;
	};
	TArray<FDirectoryResult> DiscoveredFiles;
	TArray<FGatheredPathData> DiscoveredSingleFiles;
	/** Time spent in TickInternal since the last time cook was idle. Used for logging. */
	double CurrentDiscoveryTime = 0.;
	/** Number of files discovered during scanning since start or resumed from idle. Used for logging. */
	int32 NumDiscoveredFiles = 0;
	/** Cumulative total of NumDiscoveredFiles across all non-idle periods. */
	int32 CumulativeDiscoveredFiles = 0;
	/** Cumulative total of time spent in all non-idle periods. */
	float CumulativeDiscoveryTime = 0.f;
	/** The total number of directories in the search results that were read from the cache. */
	int32 NumCachedDirectories = 0;
	/** The total number of directories in the search results that were not in the cache and were read by iterating the disk. */
	int32 NumUncachedDirectories = 0;

	// Variable section for variables that are read/writable only within TreeLock.

	/**
	 * Sorted list of MountDirs, sorted by FPackagePath::Less on the absolute paths.
	 * Each MountDir contains a ScanDir tree and other data that configures the scanning within that MountPoint.
	 * Read/writable only with TreeLock, both the list and all data owned by each MountDir.
	 */
	TArray<TUniquePtr<FMountDir>> MountDirs;

	/** A ScanDir and referencecount for directories that have been prioritized by ScanPathsSynchronous. */
	struct FPriorityScanDirData
	{
		TRefCountPtr<FScanDir> ScanDir;
		FScanDir::FInherited ParentData;
		int32 RequestCount = 0;
		/**
		 * Fire and forget prioritization increments RequestCount without ever decrementing it, and
		 * sets bReleaseWhenComplete to have it be decremented by TickInternal.
		 */
		bool bReleaseWhenComplete = false;
	};
	TArray<FPriorityScanDirData> PriorityScanDirs;

	// Variable section for variables that are read/writable only by the TickOwner thread

	/** Scratch space to store scan results during the tick to avoid allocations. */
	struct FDirToScanData
	{
		TRefCountPtr<FScanDir> ScanDir;
		FScanDir::FInherited ParentData;
		// This type is used in a TArray, and TArray assumes elements can be memmoved. 
		// TStringBuilder<N> cannot be memmoved, so use TStringBuilder<0> without a buffer.
		TStringBuilder<0> DirLocalAbsPath;
		TStringBuilder<0> DirLongPackageName;
		TArray<FDiscoveredPathData> IteratedSubDirs;
		TArray<FDiscoveredPathData> IteratedFiles;
		int32 NumIteratedDirs = 0;
		int32 NumIteratedFiles = 0;
		bool bScanned = false;

		void Reset();
		SIZE_T GetAllocatedSize() const;
	};
	TArray<FDirToScanData> DirToScanDatas;

	/** Scratch space used per helper thread in the forloop to avoid allocations. */
	struct FDirToScanBuffer
	{
		bool bAbort = false;

		void Reset();
	};
	TArray<FDirToScanBuffer> DirToScanBuffers;

	FAssetDataDiscoveryCache Cache;

	friend class FMountDir;
	friend class FScanDir;
};

/**
* How the editor, when run with -game or (under more limited circumstances) -server, performs its asset registry scan
*/
enum class EEditorGameScanMode : uint8
{
	Sync,
	Async,
	None
};

/**
 * Settings about whether to use cache data for the AssetDataGatherer; these settings are shared by
 * FPreloader, FAssetDataGatherer, and FAssetDataDiscovery.
 */
struct FGatherSettings
{
public:
	static constexpr int32 ExpectedMaxBatchSize = 100;
	static constexpr int32 MinSecondsToElapseBeforeCacheWrite = 300;
	static constexpr int32 MinAssetReadsBeforeCacheWrite = 100000;
	static constexpr int32 CacheShardAssetCount = 250000;
	static constexpr uint32 CacheSerializationMagic = 0xC2832E98; // Versioning and integrity checking
	static constexpr uint64 CurrentVersion = FAssetRegistryVersion::LatestVersion | (uint64(CacheSerializationMagic) << 32);
	static inline const FTimespan CachePruneAge = FTimespan(7, 0, 0, 0);

	/** How long we wait on the gatherer thread between asking the discovery thread for no results. */
	static constexpr float PollDiscoveryPeriodSeconds = .005f;

	/**
	 * The maximum number of threads used in the ParallelFor in FAssetDataDiscovery::TickInternal.
	 * Higher values cause more time spent waking up workers. Lower values cause more time spent running the DirectoryEnumeration on each thread.
	 * Current value was chosen based on experiments on an SSD drive with a warm disk cache.
	 */
	static inline int32 GARDiscoverThreads = 15;
	/**
	 * Minimum batch sized used used in the ParallelFor in FAssetDataDiscovery::TickInternal.
	 * In theory higher minimum values could reduce the cost of waking up workers, but in practice it does not have a significant effect.
	 */
	static inline int32 GARDiscoverMinBatchSize = 1;
	/**
	 * The maximum number of threads used in the ParallelFor around ReadAssetFile in FAssetDataGatherer::TickInternal.
	 * Very high numbers of threads does not reduce wall time of the gather because the threads become IO limited.
	 * Capping the number of threads allows those threads to be used in other async operations in Engine startup.
	 * A value of <= 0 will let the system decide how many threads to use.
	 */
	static inline int32 GARGatherThreads = 0;

	/**
	 * The maximum number of the gatherer's cache files that can be loaded in parallel in LoadCacheFiles.
	 * The amount of memory used when reading the cache rises proportional to this value.
	 * Other than the memory cost, it should be set as high as possible to load the cache faster.
	 */
	static inline int32 GARGatherCacheParallelism = 16;

	void Initialize();
	bool IsInitialized() const;
	bool IsAsyncEnabled() const;
	bool IsGatherCacheReadEnabled() const;
	bool IsGatherCacheWriteEnabled() const;
	bool IsDiscoveryCacheReadEnabled() const;
	EFeatureEnabled IsDiscoveryCacheWriteEnabled() const;
	bool IsDiscoveryCacheInvalidateEnabled() const;
	bool IsPreloadGatherCache() const;
	bool IsPreloadDiscoveryCache() const;
	bool IsGatherDependsData() const;
	bool IsForceDependsGathering() const;
	EEditorGameScanMode GetEditorGameScanMode() const;
	FString GetCacheDirectory() const;
	const FString& GetCacheBaseFilename() const;
	const FString& GetLegacyCacheBaseFilename() const;
	const FString& GetAssetRegistryCacheRootFolder() const;
	TArray<FString> FindShardedCacheFiles() const;
	TArray<FString> FindLegacyShardedCacheFiles() const;

private:
	FString CacheBaseFilename;
	FString LegacyCacheBaseFilename;
	FString AssetRegistryCacheRootFolder;
	bool bForceDependsGathering = false;
	bool bGatherDependsData = false;
	bool bGatherCacheReadEnabled = false;
	bool bGatherCacheWriteEnabled = false;
	bool bDiscoveryCacheReadEnabled = false;
	EFeatureEnabled DiscoveryCacheWriteEnabled = EFeatureEnabled::Never;
	bool bDiscoveryCacheInvalidateEnabled = false;
	bool bPreloadGatherCache = false;
	bool bAsyncEnabled = false;
	bool bInitialized = false;
	EEditorGameScanMode EditorGameScanMode = EEditorGameScanMode::None;
};
extern FGatherSettings GGatherSettings;

enum class ECacheCleanupContext
{
	PreLoad,
	PostWrite,
};
void CleanupOrphanedCacheFiles(const FString& CacheBaseFilename, ECacheCleanupContext Context);

/** Checks if the given path, for a given file type, matches the naming conventions for said type. */
bool DoesPathContainInvalidCharacters(const EGatherableFileType FileType, FStringView FilePath);

///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////

inline bool FGatherSettings::IsInitialized() const
{
	return bInitialized;
}

inline bool FGatherSettings::IsAsyncEnabled() const
{
	return bAsyncEnabled;
}

inline bool FGatherSettings::IsGatherCacheReadEnabled() const
{
	return bGatherCacheReadEnabled;
}

inline bool FGatherSettings::IsGatherCacheWriteEnabled() const
{
	return bGatherCacheWriteEnabled;
}

inline bool FGatherSettings::IsDiscoveryCacheReadEnabled() const
{
	return bDiscoveryCacheReadEnabled;
}

inline EFeatureEnabled FGatherSettings::IsDiscoveryCacheWriteEnabled() const
{
	return DiscoveryCacheWriteEnabled;
}

inline bool FGatherSettings::IsDiscoveryCacheInvalidateEnabled() const
{
	return bDiscoveryCacheInvalidateEnabled;
}

inline bool FGatherSettings::IsPreloadGatherCache() const
{
	return bGatherCacheReadEnabled && bPreloadGatherCache;
}

inline bool FGatherSettings::IsPreloadDiscoveryCache() const
{
	return IsDiscoveryCacheReadEnabled() && bPreloadGatherCache;
}

inline bool FGatherSettings::IsGatherDependsData() const
{
	return bGatherDependsData;
}

inline bool FGatherSettings::IsForceDependsGathering() const
{
	return bForceDependsGathering;
}

inline EEditorGameScanMode FGatherSettings::GetEditorGameScanMode() const
{
	return EditorGameScanMode;
}

inline FString FGatherSettings::GetCacheDirectory() const
{
	return GetAssetRegistryCacheRootFolder() / TEXT("CachedAssetRegistry");
}

inline const FString& FGatherSettings::GetCacheBaseFilename() const
{
	return CacheBaseFilename;
}

inline const FString& FGatherSettings::GetLegacyCacheBaseFilename() const
{
	return LegacyCacheBaseFilename;
}

inline const FString& FGatherSettings::GetAssetRegistryCacheRootFolder() const
{
	return AssetRegistryCacheRootFolder;
}
inline bool FFileReadScheduler::IsAssetPackageDataGatherEnabled() const
{
	return bGatherAssetPackageData;
}
inline bool FFileReadScheduler::IsDependsDataGatherEnabled() const
{
	return bGatherDependsData;
}
} // namespace UE::AssetDataGather::Private