// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/AssetDataMap.h"

#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
#include "SetKeyFuncs.h"
#endif

#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS

namespace UE::AssetRegistry::Private
{

struct FAssetObjectNameKeyFuncs
{
	explicit FAssetObjectNameKeyFuncs(const FAssetDataMap& InOwner)
		: Owner(&InOwner)
	{
	}
	FAssetDataPtrIndex GetInvalidElement()
	{
		return AssetDataPtrIndexInvalid;
	}
	bool IsInvalid(FAssetDataPtrIndex Value)
	{
		return Value == AssetDataPtrIndexInvalid;
	}
	uint32 GetTypeHash(FAssetDataPtrIndex Value)
	{
		return Owner->AssetByObjectNameValueToTypeHash(Value);
	}
	uint32 GetTypeHash(const FCachedAssetKey& Key)
	{
		return UE::AssetRegistry::Private::GetTypeHash(Key);
	}
	bool Matches(FAssetDataPtrIndex A, FAssetDataPtrIndex B)
	{
		return A == B;
	}
	bool Matches(FAssetDataPtrIndex Value, const FCachedAssetKey& Key)
	{
		return Owner->AssetByObjectNameValueMatches(Value, Key);
	}

	const FAssetDataMap* Owner;
};

FAssetDataMap::FAssetDataMap()
{
	AssetByObjectName.Reset(new FAssetObjectNameSet(FAssetObjectNameKeyFuncs(*this)));
}

FAssetDataMap::FAssetDataMap(FAssetDataMap&& Other)
{
	// We require that AssetByObjectName is non-null in all FAssetDataMap.
	AssetByObjectName.Reset(new FAssetObjectNameSet(FAssetObjectNameKeyFuncs(*this)));
	*this = MoveTemp(Other);
}

FAssetDataMap& FAssetDataMap::operator=(FAssetDataMap&& Other)
{
	// Do not use operator=(TUniquePtr&&) because we require that AssetByObjectName is non-null in all FAssetDataMap,
	// and so we need to guarantee that AssetByObjectName is swapped rather than moved and cleared;
	// possibly the TUniquePtr class does move and clear rather than swap for operator=&&.
	Swap(AssetByObjectName, Other.AssetByObjectName);
	// Set the Owner pointers inside the KeyFuncs inside the TSetKeyFuncs to point to the correct FAssetDataMap
	AssetByObjectName->SetKeyFuncs(FAssetObjectNameKeyFuncs(*this));
	Other.AssetByObjectName->SetKeyFuncs(FAssetObjectNameKeyFuncs(Other));
	Swap(AssetDatas, Other.AssetDatas);
	Swap(FreeIndex, Other.FreeIndex);
	Swap(NumFree, Other.NumFree);

	return *this;
}

FAssetDataMap::~FAssetDataMap() = default;

void FAssetDataMap::Empty(int32 ReservedSize)
{
	AssetDatas.Empty(ReservedSize);
	AssetByObjectName->Empty(ReservedSize);
	FreeIndex = AssetDataPtrIndexInvalid;
	NumFree = 0;
}

FAssetDataPtrIndex FAssetDataMap::Add(FAssetData* AssetData, bool* bAlreadyInSet)
{
	checkf(IsAligned(AssetData, 4),
		TEXT("Pointers stored in FAssetDataMap must be 4-byte aligned, because we set the low bits to indicate the data in our containers is not an added pointer."));

	FCachedAssetKey Key(AssetData);
	uint32 HashKey = GetTypeHash(Key);
	const FAssetDataPtrIndex* ExistingValue = AssetByObjectName->FindByHash(HashKey, Key);
	if (ExistingValue)
	{
		if (bAlreadyInSet)
		{
			*bAlreadyInSet = true;
		}
		return *ExistingValue;
	}

	FAssetDataPtrIndex AssignedIndex = AssetDataPtrIndexInvalid;
	if (FreeIndex != AssetDataPtrIndexInvalid)
	{
		AssignedIndex = PopFreeIndex();
		AssetDatas[AssignedIndex] = AssetData;
	}
	else
	{
		AssignedIndex = AssetDatas.Add(AssetData);
	}
	AssetByObjectName->AddByHash(HashKey, AssignedIndex);

	if (bAlreadyInSet)
	{
		*bAlreadyInSet = false;
	}
	return AssignedIndex;
}

void FAssetDataMap::AddKeyLookup(FAssetData* AssetData, FAssetDataPtrIndex AssetIndex, bool* bAlreadyInSet)
{
	checkf(IsAligned(AssetData, 4),
		TEXT("Pointers stored in FAssetDataMap must be 4-byte aligned, because we set the low bits to indicate the data in our containers is not an added pointer."));

	FCachedAssetKey Key(AssetData);
	uint32 HashKey = GetTypeHash(Key);
	const FAssetDataPtrIndex* ExistingValue = AssetByObjectName->FindByHash(HashKey, Key);
	if (ExistingValue)
	{
		if (bAlreadyInSet)
		{
			*bAlreadyInSet = true;
		}
		return;
	}

	AssetByObjectName->AddByHash(HashKey, AssetIndex);

	if (bAlreadyInSet)
	{
		*bAlreadyInSet = false;
	}
}

int32 FAssetDataMap::Remove(const FCachedAssetKey& Key)
{
	uint32 HashKey = GetTypeHash(Key);
	const FAssetDataPtrIndex* ExistingValue = AssetByObjectName->FindByHash(HashKey, Key);
	if (!ExistingValue)
	{
		return 0;
	}
	FAssetDataPtrIndex ExistingIndex = *ExistingValue;
	AssetByObjectName->RemoveByHash(HashKey, *ExistingValue);

	AddToFreeList(ExistingIndex);
	return 1;
}

void FAssetDataMap::Shrink()
{
	AssetDatas.Shrink();
}

int32 FAssetDataMap::RemoveOnlyKeyLookup(const FCachedAssetKey& Key)
{
	uint32 HashKey = GetTypeHash(Key);
	const FAssetDataPtrIndex* ExistingValue = AssetByObjectName->FindByHash(HashKey, Key);
	if (!ExistingValue)
	{
		return 0;
	}
	AssetByObjectName->RemoveByHash(HashKey, *ExistingValue);
	return 1;
}

int32 FAssetDataMap::Num() const
{
	return AssetDatas.Num() - NumFree;
}

SIZE_T FAssetDataMap::GetAllocatedSize() const
{
	return AssetDatas.GetAllocatedSize() + sizeof(*AssetByObjectName) + AssetByObjectName->GetAllocatedSize();
}

TArray<FAssetData*> FAssetDataMap::Array() const
{
	TArray<FAssetData*> Result;
	Result.Reserve(Num());
	for (FAssetData* AssetData : *this)
	{
		Result.Add(AssetData);
	}
	return Result;
}

int32 FAssetDataMap::GetMaxIndex() const
{
	return AssetDatas.Num();
}

bool FAssetDataMap::IsValidId(FSetElementId SetId) const
{
	int32 Index = SetId.AsInteger();
	return 0 <= Index && Index < AssetDatas.Num() && IsInUse(AssetDatas[Index]);
}

FAssetData* FAssetDataMap::Get(FSetElementId SetId) const
{
	return AssetDatas[SetId.AsInteger()];
}

bool FAssetDataMap::Contains(const FCachedAssetKey& Key) const
{
	return FindId(Key) != AssetDataPtrIndexInvalid;
}

FAssetData* const* FAssetDataMap::Find(const FCachedAssetKey& Key) const
{
	FAssetDataPtrIndex AssetIndex = FindId(Key);
	return AssetIndex != AssetDataPtrIndexInvalid ? &AssetDatas[AssetIndex] : nullptr;
}

FAssetDataPtrIndex FAssetDataMap::FindId(const FCachedAssetKey& Key) const
{
	const uint32* StoredValue = AssetByObjectName->Find(Key);
	if (StoredValue)
	{
		return static_cast<FAssetDataPtrIndex>(*StoredValue);
	}
	return AssetDataPtrIndexInvalid;
}

FAssetData* FAssetDataMap::operator[](FAssetDataPtrIndex AssetIndex) const
{
	return AssetDatas[static_cast<int32>(AssetIndex)];
}

void FAssetDataMap::Enumerate(
	TFunctionRef<bool(FAssetData& AssetData, FAssetDataPtrIndex AssetIndex)> Callback) const
{
	FAssetData*const* Start = AssetDatas.GetData();
	FAssetData*const* End = AssetDatas.GetData() + AssetDatas.Num();
	for (FAssetData*const* Current = Start; Current < End; ++Current)
	{
		FAssetData* AssetData = *Current;
		if (IsInUse(AssetData))
		{
			FAssetDataPtrIndex AssetIndex = static_cast<FAssetDataPtrIndex>(Current - Start);
			if (!Callback(*AssetData, AssetIndex))
			{
				break;
			}
		}
	}
}

FAssetDataMap::FIterator FAssetDataMap::begin() const
{
	return FIterator(*this, -1);
}

FAssetDataMap::FIterator FAssetDataMap::end() const
{
	return FIterator(*this, AssetDatas.Num());
}

FAssetDataMap::FIterator::FIterator(const FAssetDataMap& InOwner, int32 InIndex)
	: Owner(InOwner)
	, Index(InIndex)
{
	if (Index < 0)
	{
		this->operator++();
	}
}

FAssetData* FAssetDataMap::FIterator::operator*() const
{
	return Owner.AssetDatas[Index];
}

FAssetDataMap::FIterator& FAssetDataMap::FIterator::operator++()
{
	++Index;
	while (Index < Owner.AssetDatas.Num() && !Owner.IsInUse(Owner.AssetDatas[Index]))
	{
		++Index;
	}
	return *this;
}

bool FAssetDataMap::FIterator::operator!=(const FIterator& Other) const
{
	return Index != Other.Index;
}

bool FAssetDataMap::IsInUse(const FAssetData* DataFromAssetDatas)
{
	return (reinterpret_cast<const UPTRINT>(DataFromAssetDatas) & 0x1) == 0;
}

void FAssetDataMap::AddToFreeList(FAssetDataPtrIndex Index)
{
	static_assert(sizeof(UPTRINT) > sizeof(FAssetDataPtrIndex),
		"We assume we can fit the entire FAssetDataPtrIndex, plus one additional bit, into a UPTRINT");
	UPTRINT& IntValue = reinterpret_cast<UPTRINT&>(AssetDatas[Index]);
	IntValue = 0x1 | (static_cast<UPTRINT>(FreeIndex) << 1);
	FreeIndex = Index;
	++NumFree;
}

FAssetDataPtrIndex FAssetDataMap::PopFreeIndex()
{
	FAssetDataPtrIndex Result = FreeIndex;
	const UPTRINT& IntValue = reinterpret_cast<const UPTRINT&>(AssetDatas[FreeIndex]);
	FreeIndex = static_cast<FAssetDataPtrIndex>(IntValue >> 1);
	--NumFree;
	return Result;
}

uint32 FAssetDataMap::AssetByObjectNameValueToTypeHash(FAssetDataPtrIndex Value) const
{
	if (static_cast<uint32>(AssetDatas.Num()) <= Value)
	{
		return 0;
	}
	const FAssetData* AssetData = AssetDatas[static_cast<int32>(Value)];
	if (!IsInUse(AssetData))
	{
		return 0;
	}
	return GetTypeHash(FCachedAssetKey(AssetData));
}

bool FAssetDataMap::AssetByObjectNameValueMatches(FAssetDataPtrIndex Value, const FCachedAssetKey& Key) const
{
	if (static_cast<uint32>(AssetDatas.Num()) <= Value)
	{
		return false;
	}
	FAssetData* AssetData = AssetDatas[static_cast<int32>(Value)];
	if (IsInUse(AssetData))
	{
		FCachedAssetKey ExistingKey(AssetData);
		if (ExistingKey == Key)
		{
			return true;
		}
	}
	return false;
}

void FIndirectAssetDataArrays::AddElement(FAssetDataOrArrayIndex& Array, FAssetDataPtrIndex AssetIndex)
{
	if (Array.IsEmptyList())
	{
		Array = FAssetDataOrArrayIndex::CreateAssetDataPtrIndex(AssetIndex);
	}
	else if (Array.IsAssetDataPtrIndex())
	{
		int32 Index = AllocateArrayIndex();
		check(Arrays[Index].bArray);
		TArray<FAssetDataPtrIndex>& IndirectArray = Arrays[Index].Array;
		IndirectArray.Add(Array.AsAssetDataPtrIndex());
		IndirectArray.Add(AssetIndex);
		Array = FAssetDataOrArrayIndex::CreateArrayIndex(static_cast<FAssetDataArrayIndex>(Index));
	}
	else
	{
		check(Array.IsAssetDataArrayIndex());
		int32 Index = static_cast<int32>(Array.AsAssetDataArrayIndex());
		if (Index < 0 || Arrays.Num() <= Index || !Arrays[Index].bArray)
		{
			ensureMsgf(false, TEXT("Invalid Index %d passed as Array into AddElement. Valid values are [0, %d)."),
				Index, Arrays.Num());
			// Assign a one-elemenet list, stored as a FAssetDataPtrIndex.
			Array = FAssetDataOrArrayIndex::CreateAssetDataPtrIndex(AssetIndex);
		}
		else
		{
			TArray<FAssetDataPtrIndex>& IndirectArray = Arrays[Index].Array;
			IndirectArray.Add(AssetIndex);
		}
	}
}

void FIndirectAssetDataArrays::RemoveElement(FAssetDataOrArrayIndex& Array, FAssetDataPtrIndex AssetIndex)
{
	if (Array.IsEmptyList())
	{
		// Nothing to do, removing from an empty list is a noop
	}
	else if (Array.IsAssetDataPtrIndex())
	{
		FAssetDataPtrIndex ExistingElement = Array.AsAssetDataPtrIndex();
		if (ExistingElement == AssetIndex)
		{
			// Assign an empty list into Array
			Array = FAssetDataOrArrayIndex::CreateEmptyList();
		}
		else
		{
			// Nothing to do, removing an element not in the list is a noop
		}
	}
	else
	{
		check(Array.IsAssetDataArrayIndex());
		int32 Index = static_cast<int32>(Array.AsAssetDataArrayIndex());
		if (Index < 0 || Arrays.Num() <= Index || !Arrays[Index].bArray)
		{
			ensureMsgf(false, TEXT("Invalid Index %d passed as Array into RemoveElement. Valid values are [0, %d)."),
				Index, Arrays.Num());
			// Assign an empty list
			Array = FAssetDataOrArrayIndex::CreateEmptyList();
		}
		else
		{
			TArray<FAssetDataPtrIndex>& IndirectArray = Arrays[Index].Array;
			IndirectArray.RemoveSwap(AssetIndex, EAllowShrinking::No);
			if (IndirectArray.Num() <= 1)
			{
				if (IndirectArray.Num() == 1)
				{
					Array = FAssetDataOrArrayIndex::CreateAssetDataPtrIndex(IndirectArray[0]);
				}
				else
				{
					// This can happen if the same value was present multiple times in the array,
					// and no other values were in the array.
					Array = FAssetDataOrArrayIndex::CreateEmptyList();
				}
				IndirectArray.Empty();
				ReleaseArrayIndex(Index);
			}
			else
			{
				// Array needs to remain as an indirect array, no further action necessary.
			}
		}
	}
}

void FIndirectAssetDataArrays::RemoveAllElements(FAssetDataOrArrayIndex& Array)
{
	if (Array.IsEmptyList())
	{
		// Nothing to do, clearing an empty list is a noop
	}
	else
	{
		if (Array.IsAssetDataArrayIndex())
		{
			int32 Index = static_cast<int32>(Array.AsAssetDataArrayIndex());
			if (Index < 0 || Arrays.Num() <= Index || !Arrays[Index].bArray)
			{
				ensureMsgf(false, TEXT("Invalid Index %d passed as Array into RemoveAllElements. Valid values are [0, %d)."),
					Index, Arrays.Num());
			}
			else
			{
				ReleaseArrayIndex(Index);
			}
		}
		Array = FAssetDataOrArrayIndex::CreateEmptyList();
	}
}

TConstArrayView<FAssetDataPtrIndex> FIndirectAssetDataArrays::Iterate(const FAssetDataOrArrayIndex* ArrayPtr) const
{
	if (!ArrayPtr || ArrayPtr->IsEmptyList())
	{
		return TConstArrayView<FAssetDataPtrIndex>();
	}
	else if (ArrayPtr->IsAssetDataPtrIndex())
	{
		static_assert(FAssetDataOrArrayIndex::AssetDataType == 0,
			"We rely on the converted value for an FAssetDataOrArrayIndex to FAssetDataPtrIndex being the same bits so we can do a reinterpret_cast on the pointer.");
		const FAssetDataPtrIndex* AssetIndexPtr = reinterpret_cast<const FAssetDataPtrIndex*>(ArrayPtr);
		return TConstArrayView<FAssetDataPtrIndex>(AssetIndexPtr, 1);
	}
	else
	{
		check(ArrayPtr->IsAssetDataArrayIndex());
		int32 Index = static_cast<int32>(ArrayPtr->AsAssetDataArrayIndex());
		if (Index < 0 || Arrays.Num() <= Index || !Arrays[Index].bArray)
		{
			return TConstArrayView<FAssetDataPtrIndex>();
		}
		return Arrays[Index].Array;
	}
}

SIZE_T FIndirectAssetDataArrays::GetAllocatedSize() const
{
	SIZE_T Result = Arrays.GetAllocatedSize();
	for (const FArrayOrNextIndex& ArrayOrNextIndex : Arrays)
	{
		if (ArrayOrNextIndex.bArray)
		{
			Result += ArrayOrNextIndex.Array.GetAllocatedSize();
		}
	}
	return Result;
}

void FIndirectAssetDataArrays::Empty()
{
	Arrays.Empty();
}

void FIndirectAssetDataArrays::Shrink()
{
	Arrays.Shrink();
}

int32 FIndirectAssetDataArrays::AllocateArrayIndex()
{
	int32 Index;
	if (FreeList != UnusedIndex)
	{
		Index = static_cast<int32>(FreeList);
		check(0 <= Index && Index < Arrays.Num());
		FreeList = Arrays[Index].NextIndex;
	}
	else
	{
		Index = Arrays.Num();
		Arrays.Emplace();
	}
	FArrayOrNextIndex& ArrayOrNextIndex = Arrays[Index];
	check(!ArrayOrNextIndex.bArray);
	ArrayOrNextIndex.bArray = true;
	new (&ArrayOrNextIndex.Array) TArray<FAssetDataPtrIndex>();

	return Index;
}

void FIndirectAssetDataArrays::ReleaseArrayIndex(int32 Index)
{
	if (Index < 0 || Arrays.Num() <= Index || !Arrays[Index].bArray)
	{
		ensureMsgf(false, TEXT("Invalid Index %d passed. Arrays.Num() == %d. Arrays[Index].bArray == %s"),
			Index, Arrays.Num(),
			0 <= Index && Index < Arrays.Num() ? (Arrays[Index].bArray ? TEXT("true") : TEXT("false")) : TEXT("<Invalid>"));
		return;
	}
	FArrayOrNextIndex& ArrayOrNextIndex = Arrays[Index];
	ArrayOrNextIndex.Array.~TArray<FAssetDataPtrIndex>();
	ArrayOrNextIndex.bArray = false;
	ArrayOrNextIndex.NextIndex = FreeList;
	FreeList = static_cast<uint32>(Index);
}

/**
 * This shunt to GetTypeHash is necessary for FAssetPackageNameKeyFuncs, because c++ does not provide
 * a way to call ::GetTypeHash from a struct function when the struct has a member named GetTypeHash.
 */
FORCEINLINE uint32 AssetRegistryPrivateGetTypeHash(FName Key)
{
	return GetTypeHash(Key);
}

struct FAssetPackageNameKeyFuncs
{
	explicit FAssetPackageNameKeyFuncs(const FAssetPackageNameMap& InOwner)
		: Owner(&InOwner)
	{
	}
	FAssetDataOrArrayIndex GetInvalidElement()
	{
		return FAssetDataOrArrayIndex::CreateEmptyList();
	}
	bool IsInvalid(FAssetDataOrArrayIndex Value)
	{
		return Value.IsEmptyList();
	}
	uint32 GetTypeHash(FAssetDataOrArrayIndex Value)
	{
		return Owner->AssetOrArrayByPackageNameValueToTypeHash(Value);
	}
	uint32 GetTypeHash(FName Key)
	{
		return AssetRegistryPrivateGetTypeHash(Key);
	}
	bool Matches(FAssetDataOrArrayIndex A, FAssetDataOrArrayIndex B)
	{
		return A == B;
	}
	bool Matches(FAssetDataOrArrayIndex Value, FName PackageName)
	{
		return Owner->AssetOrArrayByPackageNameValueMatches(Value, PackageName);
	}

	const FAssetPackageNameMap* Owner;
};

FAssetPackageNameMap::FAssetPackageNameMap(FAssetDataMap& InAssetDataMap, FIndirectAssetDataArrays& InIndirectAssetDataArrays)
	: AssetOrArrayByPackageName(new FAssetPackageNameSet(FAssetPackageNameKeyFuncs(*this)))
	, AssetDataMap(InAssetDataMap)
	, IndirectArrays(InIndirectAssetDataArrays)
{
}

FAssetPackageNameMap& FAssetPackageNameMap::operator=(FAssetPackageNameMap&& Other)
{
	Swap(AssetOrArrayByPackageName, Other.AssetOrArrayByPackageName);

	// Set the Owner pointers inside the KeyFuncs inside the TSetKeyFuncs to point to the correct FAssetPackageNameMap
	AssetOrArrayByPackageName->SetKeyFuncs(FAssetPackageNameKeyFuncs(*this));
	Other.AssetOrArrayByPackageName->SetKeyFuncs(FAssetPackageNameKeyFuncs(Other));

	// Do not move the references we keep to the other structures on FAssetRegistryState.
	// Our contract with our caller is that the references never change, and the caller swaps the data in those other
	// structures during the same operation in which it swaps our data.

	return *this;
}

FAssetPackageNameMap::~FAssetPackageNameMap()
{
	Empty(0);
}

void FAssetPackageNameMap::Empty(int32 ReservedSize)
{
	for (FAssetDataOrArrayIndex DataOrArray : *AssetOrArrayByPackageName)
	{
		IndirectArrays.RemoveAllElements(DataOrArray);
	}
	AssetOrArrayByPackageName->Empty(ReservedSize);
}

void FAssetPackageNameMap::Shrink()
{
	// There is no means to shrink FAssetPackageNameSet.
	// Do nothing as any change without reducing the HashSize will only cause growth
}

void FAssetPackageNameMap::Add(FName PackageName, FAssetDataPtrIndex AssetIndex)
{
	uint32 PackageNameTypeHash = GetTypeHash(PackageName);
	FAssetDataOrArrayIndex OldStoredValue;
	const FAssetDataOrArrayIndex* OldPtr = AssetOrArrayByPackageName->FindByHash(PackageNameTypeHash, PackageName);
	if (OldPtr)
	{
		OldStoredValue = *OldPtr;
	}
	else
	{
		OldStoredValue = FAssetDataOrArrayIndex::CreateEmptyList();
	}

	FAssetDataOrArrayIndex NewStoredValue = OldStoredValue;
	IndirectArrays.AddElement(NewStoredValue, AssetIndex);
	if (NewStoredValue != OldStoredValue)
	{
		if (!OldStoredValue.IsEmptyList())
		{
			AssetOrArrayByPackageName->RemoveByHash(PackageNameTypeHash, OldStoredValue);
		}
		// We are not allowed to store empty lists in AssetOrArrayByPackageName; it should be impossible for
		// the list to be empty, but log a failed ensure and handle it if it is.
		if (ensure(!NewStoredValue.IsEmptyList()))
		{
			AssetOrArrayByPackageName->AddByHash(PackageNameTypeHash, NewStoredValue);
		}
	}
}

int32 FAssetPackageNameMap::Remove(FName PackageName, FAssetDataPtrIndex AssetIndex)
{
	uint32 PackageNameTypeHash = GetTypeHash(PackageName);
	const FAssetDataOrArrayIndex* OldPtr = AssetOrArrayByPackageName->FindByHash(PackageNameTypeHash, PackageName);
	if (OldPtr)
	{
		FAssetDataOrArrayIndex OldStoredValue = *OldPtr;
		FAssetDataOrArrayIndex NewStoredValue = OldStoredValue;
		IndirectArrays.RemoveElement(NewStoredValue, AssetIndex);
		if (NewStoredValue != OldStoredValue)
		{
			AssetOrArrayByPackageName->RemoveByHash(PackageNameTypeHash, OldStoredValue);
			if (!NewStoredValue.IsEmptyList())
			{
				AssetOrArrayByPackageName->AddByHash(PackageNameTypeHash, NewStoredValue);
			}
			return 1;
		}
	}
	return 0;
}

int32 FAssetPackageNameMap::Num() const
{
	return AssetOrArrayByPackageName->Num();
}

SIZE_T FAssetPackageNameMap::GetAllocatedSize() const
{
	return sizeof(*AssetOrArrayByPackageName) + AssetOrArrayByPackageName->GetAllocatedSize();
}

void FAssetPackageNameMap::GenerateKeyArray(TArray<FName>& OutKeys) const
{
	OutKeys.Reserve(OutKeys.Num() + Num());
	for (const FIteratorValue& Pair : *this)
	{
		if (!Pair.Key.IsNone())
		{
			OutKeys.Add(Pair.Key);
		}
	}
}

TOptional<TConstArrayView<FAssetDataPtrIndex>> FAssetPackageNameMap::Find(FName PackageName) const
{
	const FAssetDataOrArrayIndex* DataOrArrayIndex = AssetOrArrayByPackageName->Find(PackageName);
	if (DataOrArrayIndex)
	{
		return TOptional<TConstArrayView<FAssetDataPtrIndex>>(IndirectArrays.Iterate(DataOrArrayIndex));
	}
	return TOptional<TConstArrayView<FAssetDataPtrIndex>>();
}

bool FAssetPackageNameMap::Contains(FName PackageName) const
{
	return Find(PackageName).IsSet();
}

FAssetPackageNameMap::FIterator FAssetPackageNameMap::begin() const
{
	return FIterator(*this);
}

FAssetPackageNameMap::FIterationSentinel FAssetPackageNameMap::end() const
{
	return FIterationSentinel();
}

uint32 FAssetPackageNameMap::AssetOrArrayByPackageNameValueToTypeHash(FAssetDataOrArrayIndex Value) const
{
	// We only need the first AssetData in the list stored in the given Value
	// because all assets in the list have the same packagename
	const FAssetData* AssetData = AssetOrArrayIndexToFirstAssetDataPtr(Value);
	if (!AssetData)
	{
		return 0;
	}
	return GetTypeHash(AssetData->PackageName);
}

bool FAssetPackageNameMap::AssetOrArrayByPackageNameValueMatches(FAssetDataOrArrayIndex Value, FName PackageName) const
{
	TConstArrayView<FAssetDataPtrIndex> AssetIndexArray = IndirectArrays.Iterate(&Value);

	if (!AssetIndexArray.IsEmpty())
	{
		// To check whether the elements in this list match the requested PackageName,
		// We only need the first AssetData in the list stored in the given StoredValue 
		// because all assets in the list have the same PackageName.
		return AssetDataMap[AssetIndexArray[0]]->PackageName == PackageName;
	}
	return false;
}

FAssetData* FAssetPackageNameMap::AssetOrArrayIndexToFirstAssetDataPtr(FAssetDataOrArrayIndex DataOrArrayIndex) const
{
	TConstArrayView<FAssetDataPtrIndex> AssetIndices = IndirectArrays.Iterate(&DataOrArrayIndex);
	if (AssetIndices.Num() > 0)
	{
		return AssetDataMap[AssetIndices[0]];
	}
	return nullptr;
}

/**
 * The converter from FAssetPackageNameSetIteratorBytes to FAssetPackageNameSet::FIterator, this allows us to
* have a TSetKeyFuncs::FIterator inside of our public iterator, with only a foward declare of TSetKeyFuncs.
*/
inline const FAssetPackageNameSet::FIterator& HashIter(const FAssetPackageNameSetIteratorBytes& HashIterBytes)
{
	return reinterpret_cast<const FAssetPackageNameSet::FIterator&>(HashIterBytes);
}

inline FAssetPackageNameSet::FIterator& HashIter(FAssetPackageNameSetIteratorBytes& HashIterBytes)
{
	static_assert(sizeof(FAssetPackageNameSetIteratorBytes) >= sizeof(FAssetPackageNameSet::FIterator) &&
		alignof(FAssetPackageNameSetIteratorBytes) >= alignof(FAssetPackageNameSet::FIterator),
		"FAssetMapHashSetIteratorBytes is a forward declare proxy of FAssetMapHashSetIterator and has to be sized to handle it.");
	return reinterpret_cast<FAssetPackageNameSet::FIterator&>(HashIterBytes);
}

FAssetPackageNameMap::FIterator::FIterator(const FAssetPackageNameMap& InOwner)
	: Owner(InOwner)
{
	new (&HashIter(HashIterBytes)) FAssetPackageNameSet::FIterator(*Owner.AssetOrArrayByPackageName);
}

FAssetPackageNameMap::FIterator::~FIterator()
{
	HashIter(HashIterBytes).~FIterator();
}

FAssetPackageNameMap::FIteratorValue FAssetPackageNameMap::FIterator::operator*() const
{
	FAssetDataOrArrayIndex Value = *HashIter(HashIterBytes);
	// When getting just the key we only need to use the first AssetData in the list stored in the given Value
	// because all assetdatas in the list have the same packagename
	FAssetData* AssetData = Owner.AssetOrArrayIndexToFirstAssetDataPtr(Value);
	FIteratorValue Result;
	Result.Key = AssetData ? AssetData->PackageName : NAME_None;
	return Result;
}

FAssetPackageNameMap::FIterator& FAssetPackageNameMap::FIterator::operator++()
{
	++HashIter(HashIterBytes);
	return *this;
}

bool FAssetPackageNameMap::FIterator::operator!=(FIterationSentinel) const
{
	// TSetKeyFuncs defines FIterationSentinel as an empty structure, just for type-specific definition of !=,
	// so constructing it every time our != is called should have no cost.
	return HashIter(HashIterBytes) != FAssetPackageNameSet::FIterationSentinel();
}

} // namespace UE::AssetRegistry::Private

#endif // UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS