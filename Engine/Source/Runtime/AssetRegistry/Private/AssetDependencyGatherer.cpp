// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/AssetDependencyGatherer.h"
#include "AssetRegistryPrivate.h"

#if WITH_EDITOR

namespace UE::AssetDependencyGatherer::Private
{

static TLinkedList<FRegisteredAssetDependencyGatherer*>* GRegisteredAssetDependencyGathererList = nullptr;
FRegisteredAssetDependencyGatherer::FOnAssetDependencyGathererRegistered FRegisteredAssetDependencyGatherer::OnAssetDependencyGathererRegistered;

FRegisteredAssetDependencyGatherer::FRegisteredAssetDependencyGatherer()
	: GlobalListLink(this)
{
	GlobalListLink.LinkHead(GetRegisteredList());
	OnAssetDependencyGathererRegistered.Broadcast();
}

FRegisteredAssetDependencyGatherer::~FRegisteredAssetDependencyGatherer()
{
	GlobalListLink.Unlink();
}

TLinkedList<FRegisteredAssetDependencyGatherer*>*& FRegisteredAssetDependencyGatherer::GetRegisteredList()
{
	return GRegisteredAssetDependencyGathererList;
}

void FRegisteredAssetDependencyGatherer::ForEach(TFunctionRef<void(FRegisteredAssetDependencyGatherer*)> Func)
{
	for (TLinkedList<FRegisteredAssetDependencyGatherer*>::TIterator It(GetRegisteredList()); It; It.Next())
	{
		Func(*It);
	}
}
	
}

namespace UE::AssetDependencyGatherer::Private
{

void FRegisteredAssetDependencyGatherer::GatherDependenciesForGatherer(IAssetDependencyGatherer* GathererInterface, IAssetDependencyGatherer::FGatherDependenciesContext& Context, const char* InGathererClass) const 
{ 
	// reset marker that indicates if the default implementation in IAssetDependencyGatherer was called
	Context.bInterfaceGatherDependenciesCalled = false;
	GathererInterface->GatherDependencies(Context); 

	if (Context.bInterfaceGatherDependenciesCalled)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS

		// We call the deprecated implementation if we detect a call that directly ended in IAssetDependencyGatherer::GatherDependencies(FGatherDependenciesContext&)
		// This shows the IAssetDependencyGatherer we have is still using the old interface. 
		GathererInterface->GatherDependencies(Context.AssetData, Context.AssetRegistryState, Context.CompileFilterFunc, Context.OutDependencies, Context.OutDependencyDirectories);

		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (!bCalledOnce)
		{
			// Deprecated in 5.8
			UE_LOGF(LogAssetRegistry, Warning, 
				"%s is overriding a deprecated function, GatherDependencies(const FAssetData& , const FAssetRegistryState&, TArray&, TArray&). " "Override GatherDependencies that takes an FGatherDependenciesContext instead. " "Deprecation handling will be removed in a future version and your code will no longer compile.",
				InGathererClass);

			bCalledOnce = true;
		}
	}
}

}


void IAssetDependencyGatherer::GatherDependencies(FGatherDependenciesContext& Context) const
{
	// Flag the context to indicate IAssetDependencyGatherer::GatherDependencies(FGatherDependenciesContext&) was called which will 
	// happen in classes only implementing GatherDependencies(const FAssetData&, const FAssetRegistryState&, TFunctionRef<FARCompiledFilter(const FARFilter&)>, TArray<FGathereredDependency>&, TArray<FString>&)
	Context.bInterfaceGatherDependenciesCalled = true;
}

void IAssetDependencyGatherer::GatherDependencies(const FAssetData & AssetData, const FAssetRegistryState & AssetRegistryState, TFunctionRef<FARCompiledFilter(const FARFilter&)> CompileFilterFunc, TArray<FGathereredDependency>&OutDependencies, TArray<FString>&OutDependencyDirectories) const
{
	checkf(false, TEXT("Implement IAssetDependencyGatherer::GatherDependencies(FGatherDependenciesContext& Context) in your class, this class currently appears to not be implementing any."));
}

#endif