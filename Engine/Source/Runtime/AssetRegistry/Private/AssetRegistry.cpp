// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry.h"

#include "Algo/AnyOf.h"
#include "Algo/HeapSort.h"
#include "Algo/Transform.h"
#include "Algo/Unique.h"
#include "AssetDataGatherer.h"
#include "AssetDataGathererPrivate.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetDependencyGatherer.h"
#include "AssetRegistry/AssetRegistryTelemetry.h"
#include "AssetRegistryConsoleCommands.h"
#include "AssetRegistryImpl.h"
#include "AssetRegistryPrivate.h"
#include "Async/Async.h"
#include "Async/ParallelFor.h"
#include "AutoRTFM.h"
#include "Blueprint/BlueprintSupport.h"
#include "Containers/MpscQueue.h"
#include "DependsNode.h"
#include "GenericPlatform/GenericPlatformChunkInstall.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformMisc.h"
#include "HAL/ThreadHeartBeat.h"
#include "Interfaces/IPluginManager.h"
#include "IPlatformFilePak.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FileHelper.h"
#include "Misc/Optional.h"
#include "Misc/PackageAccessTracking.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/PackageSegment.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/RedirectCollector.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TrackedActivity.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "AssetRegistry/PackageReader.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "String/RemoveFrom.h"
#include "Tasks/Task.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "TelemetryRouter.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/CoreRedirects.h"
#include "UObject/MetaData.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UObjectThreadContext.h"
#include <atomic>

#include UE_INLINE_GENERATED_CPP_BY_NAME(IAssetRegistry)
#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetRegistry)

#if WITH_EDITOR
#include "DirectoryWatcherModule.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformProcess.h"
#include "IDirectoryWatcher.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#endif // WITH_EDITOR

#define UE_ENABLE_DIRECTORYWATCH_ROOTS !UE_IS_COOKED_EDITOR

/**
 * ********** AssetRegistry threading model **********
 * *** Functions and InterfaceLock ***
 * All data(except events and RWLock) in the AssetRegistry is stored on the FAssetRegistryImpl GuardedData object.
 * No data can be read on GuardedData unless the caller has entered the InterfaceLock.
 * All data on FAssetRegistryImpl is private; this allows us to mark threading model with function prototypes.
 * All functions on FAssetRegistryImpl are intended to be called only within a critical section.
 * const functions require a ReadLock critical section; non-const require a WriteLock.
 * The requirement that functions must be called only from within a critical section(and non-const only within a
 * WriteLock) is not enforced technically; change authors need to carefully follow the synchronization model.
 * FAssetRegistryImpl methods that expect an FInterfaceScope[Read|Write]Lock& may release and re-acquire the lock. Functions
 * re-entering the lock are expected to validate possible re-entrancy issues that may arise from doing so.
 * FAssetRegistryImpl methods that expect an FInterfaceScope[Read|Write]Lock* work like above however passing in nullptr
 * indicates that the function should maintain the lock state from the caller and the caller is responsible for ensuring
 * the threading model is being adhered to.

 * *** Events, Callbacks, and Object Virtuals ***
 * The AssetRegistry provides several Events(e.g.AssetAddedEvent) that can be subscribed to from arbitrary engine or
 * licensee code, and some functions(e.g.EnumerateAssets) take a callback, and some functions call arbitrary
 * UObject virtuals(e.g. new FAssetData(UObject*)). Some of this arbitrary code can call AssetRegistry functions of
 * their own, and if they were called from within the lock that reentrancy would cause a deadlock when we tried
 * to acquire the RWLock (RWLocks are not reenterable on the same thread). With some exceptions AssetRegistryImpl code
 * is therefore not allowed to call callbacks, send events, or call UObject virtuals from inside a lock.

 * FEventContext allows deferring events to a point in the top-level interface function outside the lock. The top-level
 * function passes the EventContext in to the GuardedData functions, which add events on to it, and then it broadcasts
 * the events outside the lock. FEventContext also handles deferring events to the Tick function executed from
 * the GameThread, as we have a contract that events are only called from the game thread.

 * Callbacks are handled on a case-by-case basis; each interface function handles queuing up the data for the callback
 * functions and calling it outside the lock. The one exception is the ShouldSetManager function, which we call
 * from inside the lock, since it is relatively well-behaved code as it is only used by UAssetManager and licensee
 * subclasses of UAssetManager.

 * UObject virtuals are handled on a case-by-case basis; the primary example is `new FAssetData(UObject*)`, which
 * ProcessLoadedAssetsToUpdateCache takes care to call outside the lock and only on the game thread.
 *
 * *** Updating Caches - InheritanceContext ***
 * The AssetRegistry has a cache for CodeGeneratorClasses and for an InheritanceMap of classes - native and blueprint.
 * Updating these caches needs to be done within a writelock; for CodeGeneratorClasses we do this normally by marking
 * all functions that need to update it as non-const. For InheritanceMap that would be overly pessimistic as several
 * otherwise-const functions need to occasionally update the caches. For InheritanceMap we therefore have
 * FClassInheritanceContext and FClassInheritanceBuffer. The top-level interface functions check whether the
 * inheritance map will need to be updated during their execution, and if so they enter a write lock with the ability
 * to update the members in the InheritanceContext. Otherwise they enter a readlock and the InheritanceBuffer will not
 * be modified. All functions that use the cached data require the InheritanceContext to give them access, to ensure
 * they are only using correctly updated cache data.
 *
 * *** Returning Internal Data ***
 * All interface functions that return internal data return it by copy, or provide a ReadLockEnumerate function that
 * calls a callback under the readlock, where the author of the callback has to ensure other AssetRegistry functions
 * are not called.
 */

static FAssetRegistryConsoleCommands ConsoleCommands; // Registers its various console commands in the constructor

namespace UE::AssetRegistry
{
	const FName WildcardFName("*");
	const FTopLevelAssetPath WildcardPathName(TEXT("/*"), TEXT("*"));

	const FName Stage_ChunkCountFName("Stage_ChunkCount");
	const FName Stage_ChunkSizeFName("Stage_ChunkSize");
	const FName Stage_ChunkCompressedSizeFName("Stage_ChunkCompressedSize");
	const FName Stage_ChunkInstalledSizeFName("Stage_ChunkInstalledSize");
	const FName Stage_ChunkStreamingSizeFName("Stage_ChunkStreamingSize");
	const FName Stage_ChunkOptionalSizeFName("Stage_ChunkOptionalSize");
	
	FString LexToString(EScanFlags Flags)
	{
		const TCHAR* Names[] = {
			TEXT("ForceRescan"),
			TEXT("IgnoreDenyListScanFilters"),
			TEXT("WaitForInMemoryObjects"),
			TEXT("IgnoreInvalidPathWarning")
		};
		
		if (Flags == EScanFlags::None)
		{
			return TEXT("None");
		}
		
		uint32 AllKnownFlags = (1 << (UE_ARRAY_COUNT(Names)+1)) - 1;
		ensureMsgf(EnumHasAllFlags((EScanFlags)AllKnownFlags, Flags), TEXT("LexToString(UE::AssetRegistry::EScanFlags) is missing some cases"));

		TStringBuilder<256> Builder;
		for (uint32 i=0; i < UE_ARRAY_COUNT(Names); ++i)
		{
			if (EnumHasAllFlags(Flags, (EScanFlags)(1 << i)))
			{
				if (Builder.Len() != 0)
				{
					Builder << TEXT("|");
				}
				Builder << Names[i];	
			}
		}
		
		return Builder.ToString();
	}
}

namespace UE::AssetRegistry::Impl
{
	/** The max time to spend in UAssetRegistryImpl::Tick */
	constexpr float MaxSecondsPerFrameToUseInBlockingInitialLoad = 5.0f;
	float MaxSecondsPerFrame = 0.04f;
	static FAutoConsoleVariableRef CVarAssetRegistryMaxSecondsPerFrame(
		TEXT("AssetRegistry.MaxSecondsPerFrame"),
		UE::AssetRegistry::Impl::MaxSecondsPerFrame,
		TEXT("Maximum amount of time allowed for Asset Registry processing, in seconds"));
	float MaxSecondsPerTickBackgroundThread = -1.f;
	static FAutoConsoleVariableRef CVarAssetRegistryMaxSecondsPerTickBackgroundThread(
		TEXT("AssetRegistry.MaxSecondsPerTickBackgroundThread"),
		UE::AssetRegistry::Impl::MaxSecondsPerTickBackgroundThread,
		TEXT("Maximum amount of time allowed for Asset Registry processing, in seconds, per iteration on the background thread. Negative are treated as no maximum."));

	/** If true, defer sorting of dependencies until loading is complete */
	bool bDeferDependencySort = false;
	static FAutoConsoleVariableRef CVarAssetRegistryDeferDependencySort(
		TEXT("AssetRegistry.DeferDependencySort"),
		UE::AssetRegistry::Impl::bDeferDependencySort,
		TEXT("If true, the dependency lists on dependency nodes will not be sorted until after the initial load is complete"));

	/** If true, defer sorting of referencer data until loading is complete, this is enabled by default because of native packages with many referencers */
	bool bDeferReferencerSort = true;
	static FAutoConsoleVariableRef CVarAssetRegistryDeferReferencerSort(
		TEXT("AssetRegistry.DeferReferencerSort"),
		UE::AssetRegistry::Impl::bDeferReferencerSort,
		TEXT("If true, the referencer list on dependency nodes will not be sorted until after the initial load is complete"));

	bool bDisableDirectoryWatcher = false;
	static FAutoConsoleVariableRef CVarAssetRegistryDisableDirectoryWatcher(
		TEXT("AssetRegistry.DisableDirectoryWatcher"),
		UE::AssetRegistry::Impl::bDisableDirectoryWatcher,
		TEXT("If true, do not listen to mounted directories for file changes"));

	bool IsDirectoryWatcherEnabled()
	{
		// In-game and in commandlets AR doesn't listen for directory changes
		return !bDisableDirectoryWatcher && GIsEditor && !IsRunningCommandlet();
	}

	/** Name of UObjectRedirector property */
	const FName DestinationObjectFName("DestinationObject");
}

/**
 * Implementation of IAssetRegistryInterface; forwards calls from the CoreUObject-accessible IAssetRegistryInterface into the AssetRegistry-accessible IAssetRegistry
 */
class FAssetRegistryInterface : public IAssetRegistryInterface
{
public:
	virtual void GetDependencies(FName InPackageName, TArray<FName>& OutDependencies,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::Package,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) override
	{
		IAssetRegistry::GetChecked().GetDependencies(InPackageName, OutDependencies, Category, Flags);
	}

	virtual UE::AssetRegistry::EExists TryGetAssetByObjectPath(const FSoftObjectPath& ObjectPath, FAssetData& OutAssetData) const override
	{
		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		if (!AssetRegistry)
		{
			return UE::AssetRegistry::EExists::Unknown;
		}
		return AssetRegistry->TryGetAssetByObjectPath(ObjectPath, OutAssetData);
	}

	virtual UE::AssetRegistry::EExists TryGetAssetPackageData(FName PackageName, class FAssetPackageData& OutPackageData, bool bFailIfLockHeld) const override
	{
		FName OutCorrectCasePackageName;
		return TryGetAssetPackageData(PackageName, OutPackageData, OutCorrectCasePackageName, bFailIfLockHeld);
	}
	
	virtual UE::AssetRegistry::EExists TryGetAssetPackageData(FName PackageName, class FAssetPackageData& OutPackageData, FName& OutCorrectCasePackageName, bool bFailIfLockHeld) const override
	{
		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		if (!AssetRegistry)
		{
			return UE::AssetRegistry::EExists::Unknown;
		}
		return AssetRegistry->TryGetAssetPackageData(PackageName, OutPackageData, OutCorrectCasePackageName, bFailIfLockHeld);
	}

	virtual bool EnumerateAssets(const FARFilter& Filter, TFunctionRef<bool(const FAssetData&)> Callback,
		UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags) const override
	{
		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		if (!AssetRegistry)
		{
			return false;
		}
		return AssetRegistry->EnumerateAssets(Filter, Callback, InEnumerateFlags);
	}

	virtual bool RegisterOnAssetsAddedDelegate(const TFunction<void(TConstArrayView<FAssetData>)>& Function, FDelegateHandle& OutHandle) override
	{
		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		if (!AssetRegistry)
		{
			return false;
		}
		OutHandle = AssetRegistry->OnAssetsAdded().AddLambda(Function);
		return true;
	}

	virtual bool UnregisterOnAssetsAddedDelegate(const FDelegateHandle& Handle) override
	{
		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		if (!AssetRegistry)
		{
			return false;
		}
		return AssetRegistry->OnAssetsAdded().Remove(Handle);
	}

	virtual bool RegisterOnAssetsRemovedDelegate(const TFunction<void(TConstArrayView<FAssetData>)>& Function, FDelegateHandle& OutHandle) override
	{
		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		if (!AssetRegistry)
		{
			return false;
		}
		OutHandle = AssetRegistry->OnAssetsRemoved().AddLambda(Function);
		return true;
	}

	virtual bool UnregisterOnAssetsRemovedDelegate(const FDelegateHandle& Handle) override
	{
		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		if (!AssetRegistry)
		{
			return false;
		}
		return AssetRegistry->OnAssetsRemoved().Remove(Handle);
	}
};
FAssetRegistryInterface GAssetRegistryInterface;

// Caching is permanently enabled in editor because memory is not that constrained, disabled by default otherwise
#define ASSETREGISTRY_CACHE_ALWAYS_ENABLED (WITH_EDITOR)

DEFINE_LOG_CATEGORY(LogAssetRegistry);


namespace UE::AssetRegistry
{

namespace DependsNode
{
typedef TMap<FDependsNode*, FDependsNode::FPackageFlagSet> FPackageDependencyMap;

/** Uncompressed version of DependsNode data, used to avoid allocations and
compression cost when constructing batches of DependsNodes in DependencyDataGathered. */
struct FConstructData
{
	FDependsNode* SourceNode;
	FPackageDependencyMap PackageDependencies;
	FDependsNode::FDependsNodeList SearchableNamesNodes;
};

struct FGatherDependencyDataScratchpad
{
	TMap<FName, FConstructData> AssetNameToConstructDataMap;
	TMap<FName, FName> CachedDepToRedirect;
	TSet<FDependsNode*> ModifiedDependNodes;

	void Reset()
	{
		AssetNameToConstructDataMap.Reset();
		CachedDepToRedirect.Reset();
		ModifiedDependNodes.Reset();
	}

	void Shrink()
	{
		AssetNameToConstructDataMap.Shrink();
		CachedDepToRedirect.Shrink();
		ModifiedDependNodes.Shrink();
	}
};

}

void FAssetRegistryImpl::ConditionalLoadPremadeAssetRegistry(UE::AssetRegistry::FInterfaceWriteScopeLock& ScopeLock, Impl::FEventContext& EventContext)
{
	AsyncConsumer.Wait(ScopeLock);
}

void FAssetRegistryImpl::ConsumeOrDeferPreloadedPremade(UAssetRegistryImpl& UARI, Impl::FEventContext& EventContext,
	Impl::FActiveMountEvents& ActiveMountEvents, FInterfaceWriteScopeLock& ProofOfWriteLock)
{
	// Called from inside WriteLock on InterfaceLock
	using namespace UE::AssetRegistry::Premade;
	if (!Premade::IsEnabled())
	{
		// if we aren't doing any preloading, then we can set the initial search is done right away.
		// Otherwise, it is set from LoadPremadeAssetRegistry
		bPreloadingComplete = true;
		return;
	}

	if (Premade::CanLoadAsync())
	{
		FPreloader::FConsumeFunction ConsumeFromAsyncThread = [this, &UARI](Premade::ELoadResult LoadResult,
			UE::AssetRegistry::Impl::FAssetProviderInitData&& StateInitData)
		{
			Impl::FEventContext EventContext;
			Impl::FActiveMountEvents ActiveMountEvents;
			{
				UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(UARI.InterfaceLock);
				AsyncConsumer.Consume(UARI, EventContext, ActiveMountEvents, InterfaceScopeLock, LoadResult,
					MoveTemp(StateInitData));
			}
			UARI.Broadcast(EventContext);
			ActiveMountEvents.Broadcast();
		};
		auto ConsumeOnCurrentThread = [ConsumeFromAsyncThread](Premade::ELoadResult LoadResult,
			UE::AssetRegistry::Impl::FAssetProviderInitData&& StateInitData) mutable
		{
			Async(EAsyncExecution::TaskGraph, [LoadResult, StateInitData=MoveTemp(StateInitData),
				ConsumeFromAsyncThread=MoveTemp(ConsumeFromAsyncThread)]() mutable
			{
				ConsumeFromAsyncThread(LoadResult, MoveTemp(StateInitData));
			});
		};

		AsyncConsumer.PrepareForConsume();
		GPreloader.ConsumeOrDefer(MoveTemp(ConsumeOnCurrentThread), MoveTemp(ConsumeFromAsyncThread));
	}
	else
	{
		GPreloader.Consume(
			[this, &UARI, &EventContext, &ProofOfWriteLock, &ActiveMountEvents](Premade::ELoadResult LoadResult,
				UE::AssetRegistry::Impl::FAssetProviderInitData&& StateInitData)
			{
				LoadPremadeAssetRegistry(UARI, EventContext, ActiveMountEvents, ProofOfWriteLock, LoadResult,
					MoveTemp(StateInitData));
			});
	}
}
}

/** Returns the appropriate ChunkProgressReportingType for the given Asset enum */
EChunkProgressReportingType::Type GetChunkAvailabilityProgressType(EAssetAvailabilityProgressReportingType::Type ReportType)
{
	EChunkProgressReportingType::Type ChunkReportType;
	switch (ReportType)
	{
	case EAssetAvailabilityProgressReportingType::ETA:
		ChunkReportType = EChunkProgressReportingType::ETA;
		break;
	case EAssetAvailabilityProgressReportingType::PercentageComplete:
		ChunkReportType = EChunkProgressReportingType::PercentageComplete;
		break;
	default:
		ChunkReportType = EChunkProgressReportingType::PercentageComplete;
		UE_LOGF(LogAssetRegistry, Error, "Unsupported assetregistry report type: %i", (int)ReportType);
		break;
	}
	return ChunkReportType;
}

const TCHAR* GetDevelopmentAssetRegistryFilename()
{
	return TEXT("DevelopmentAssetRegistry.bin");
}

FAssetData IAssetRegistry::K2_GetAssetByObjectPath(const FSoftObjectPath& ObjectPath, bool bIncludeOnlyOnDiskAssets, bool bSkipARFilteredAssets) const
{
	return GetAssetByObjectPath(ObjectPath, bIncludeOnlyOnDiskAssets, bSkipARFilteredAssets);
}

void IAssetRegistry::InitializeSerializationOptions(FAssetRegistrySerializationOptions& Options,
	const FString& PlatformIniName,
	UE::AssetRegistry::ESerializationTarget Target) const
{
#if WITH_EDITOR
	if (ITargetPlatformManagerModule* TargetPlatformManager = GetTargetPlatformManager())
	{
		for (ITargetPlatform* TargetPlatform : TargetPlatformManager->GetActiveTargetPlatforms())
		{
			if (TargetPlatform->IniPlatformName() == PlatformIniName)
			{
				return InitializeSerializationOptions(Options, TargetPlatform, Target);
			}
		}
	}
#endif

	if (ensureMsgf(PlatformIniName.IsEmpty(), TEXT("Couldn't find TargetPlatform from ini `%s`"), *PlatformIniName)) {
		return InitializeSerializationOptions(Options, nullptr, Target);
	}
}

IAssetRegistry::FLoadPackageRegistryData::FLoadPackageRegistryData(bool bInGetDependencies)
	: bGetDependencies(bInGetDependencies)
{
}

IAssetRegistry::FLoadPackageRegistryData::~FLoadPackageRegistryData() = default;


UAssetRegistry::UAssetRegistry(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

namespace UE::AssetRegistry::Impl
{

struct FInitializeContext
{
	UAssetRegistryImpl& UARI;
	FEventContext Events;
	FActiveMountEvents ActiveMountEvents;
	FClassInheritanceContext InheritanceContext;
	FClassInheritanceBuffer InheritanceBuffer;
	TArray<FString> RootContentPaths;
	bool bRedirectorsNeedSubscribe = false;
	bool bUpdateDiskCacheAfterLoad = false;
	bool bNeedsSearchAllAssetsAtStartSynchronous = false;
};

}

UAssetRegistryImpl::UAssetRegistryImpl(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SCOPED_BOOT_TIMING("UAssetRegistryImpl::UAssetRegistryImpl");

	UE::AssetRegistry::Impl::FInitializeContext Context{ *this };

	if (HasAnyFlags(RF_ClassDefaultObject) && !HasAnyFlags(RF_ImmutableDefaultObject))
	{
		check(UE::AssetRegistry::Private::IAssetRegistrySingleton::Singleton == nullptr && IAssetRegistryInterface::Default == nullptr);
		UE::AssetRegistry::Private::IAssetRegistrySingleton::Singleton = this;
		IAssetRegistryInterface::Default = &GAssetRegistryInterface;
	}

	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		GetInheritanceContextWithRequiredLock(InterfaceScopeLock, Context.InheritanceContext,
			Context.InheritanceBuffer);

		GuardedData.Initialize(InterfaceScopeLock, Context);
		InitializeEvents(Context);
	}
	Broadcast(Context.Events);
	Context.ActiveMountEvents.Broadcast();
}

bool UAssetRegistryImpl::IsPathBeautificationNeeded(const FString& InAssetPath) const
{
	return InAssetPath.Contains(FPackagePath::GetExternalActorsFolderName()) || InAssetPath.Contains(FPackagePath::GetExternalObjectsFolderName());
}

namespace UE::AssetRegistry
{

FAssetRegistryImpl::FAssetRegistryImpl()
{
}

void FAssetRegistryImpl::LoadPremadeAssetRegistry(UAssetRegistryImpl& UARI, Impl::FEventContext& EventContext,
	Impl::FActiveMountEvents& ActiveMountEvents, FInterfaceWriteScopeLock& ProofOfWriteLock,
	Premade::ELoadResult LoadResult, Impl::FAssetProviderInitData&& StateInitData)
{
	SCOPED_BOOT_TIMING("LoadPremadeAssetRegistry");
	UE_SCOPED_ENGINE_ACTIVITY("Loading premade asset registry");

	const bool bEmitAssetEvents = GIsEditor;

	if (SerializationOptions.bSerializeAssetRegistry)
	{
		SCOPED_BOOT_TIMING("LoadPremadeAssetRegistry_Main");
		if (LoadResult == Premade::ELoadResult::Succeeded)
		{
			UARI.ActiveMountsLoader.InitialProviderReport(UARI, ActiveMountEvents, StateInitData);
			
			if (State.GetNumAssets() == 0)
			{
				// ActiveMounts: Handle the possibility that ActiveMountsInitialize has already been called and if so
				// parse the premade's state now.
				UARI.ActiveMountsLoader.InitialProviderConditionalRegister(UARI, ActiveMountEvents, StateInitData.ARState);

				// Add the premade state into the global AssetRegistry
				State = MoveTemp(StateInitData.ARState);
				CachePathsFromState(EventContext, State);
				if (bEmitAssetEvents)
				{
					EventContext.AssetEvents.Reserve(State.GetNumAssets());
					State.EnumerateAllAssets([&EventContext](const FAssetData& AssetData)
					{
						EventContext.AssetEvents.Emplace(AssetData,
							UE::AssetRegistry::Impl::FEventContext::EEvent::Added);
					});
				}
			}
			else
			{
				// This is unexpected, we should not have called AppendState or in any other way added Assets to the
				// AssetRegistry before loading the premade AssetRegistry. However, we can handle it with only minor
				// performance issues.
				// ActiveMounts usually parses the premade's ARState during ActiveMountsInitialize by reading the
				// global AssetRegistry's GuardedData.State, but that state already contains some other data, so we
				// need to instead parse the premade's state directly.
				UARI.ActiveMountsLoader.InitialProviderOnReadyForRegister(UARI, ActiveMountEvents,
					Impl::EInitialProviderRegisterReason::EarlierState);
				UARI.ActiveMountsLoader.InitialProviderConditionalRegister(UARI, ActiveMountEvents, StateInitData.ARState);

				// Add the premade state into the global AssetRegistry
				if (State.GetNumAssets() < StateInitData.ARState.GetNumAssets())
				{
					FAssetRegistryState ExistingState = MoveTemp(State);
					State = MoveTemp(StateInitData.ARState);
					CachePathsFromState(EventContext, State);
					if (bEmitAssetEvents)
					{
						EventContext.AssetEvents.Reserve(State.GetNumAssets());
						State.EnumerateAllAssets([&EventContext](const FAssetData& AssetData)
							{
								EventContext.AssetEvents.Emplace(AssetData,
									UE::AssetRegistry::Impl::FEventContext::EEvent::Added);
							});
					}
					AppendState(EventContext, ExistingState);
				}
				else
				{
					AppendState(EventContext, StateInitData.ARState,
						FAssetRegistryState::EInitializationMode::OnlyUpdateNew, bEmitAssetEvents);
				}
			}
			UpdatePersistentMountPoints();
			State.bCookedGlobalAssetRegistryState = true;
		}
		else
		{
			UE_CLOGF(FPlatformProperties::RequiresCookedData() && (IsRunningGame() || IsRunningDedicatedServer()),
				LogAssetRegistry, Error, "Failed to load premade asset registry. LoadResult == %d.", static_cast<int32>(LoadResult));
		}
	}

	{
		SCOPED_BOOT_TIMING("LoadPremadeAssetRegistry_Plugins");
		FPakPlatformFile* PakPlatformFile = (FPakPlatformFile*)(FPlatformFileManager::Get().FindPlatformFile(FPakPlatformFile::GetTypeName()));
		TArray<TSharedRef<IPlugin>> ContentPlugins = IPluginManager::Get().GetEnabledPluginsWithContent();
		for (const TSharedRef<IPlugin>& ContentPlugin : ContentPlugins)
		{
			if (ContentPlugin->CanContainContent())
			{
				FAssetRegistryState PluginState;
				FString PluginAssetRegistry = ContentPlugin->GetBaseDir() / TEXT("AssetRegistry.bin");

				if (UARI.ActiveMountsLoader.TryLoadStateAndRegisterAssetProvider(UARI, ActiveMountEvents,
					ProofOfWriteLock, PakPlatformFile, PluginAssetRegistry, PluginState))
				{
#if WITH_EDITOR
					/*
					 * Only update the new assets when using a premade asset registry in editor.
					 * The main state will often already include the DLC/plugin assets and is often in a development mode where the plugin state will not be.
					 * If we update the existing assets in those cases it will be causing a lost of tags and values that are needed for the editor systems.
					 */
					AppendState(EventContext, PluginState, FAssetRegistryState::EInitializationMode::OnlyUpdateNew, bEmitAssetEvents);
#else
					AppendState(EventContext, PluginState, FAssetRegistryState::EInitializationMode::Append, bEmitAssetEvents);
#endif
				}
			}
		}
	}

	// let Tick know that it can finalize the initial search
	bPreloadingComplete = true;
}

void FAssetRegistryImpl::Initialize(UE::AssetRegistry::FInterfaceWriteScopeLock& ScopeLock, Impl::FInitializeContext& Context)
{
	if (bInitializationComplete)
	{
		return;
	}

	const double StartupStartTime = FPlatformTime::Seconds();

	bInitialSearchCompleted.store(true, std::memory_order_relaxed);
	bAdditionalMountSearchInProgress.store(false, std::memory_order_relaxed);
#if WITH_EDITOR
	SetGameThreadTakeOverGatherEachTick(false);
#endif

	UpdateMaxSecondsPerFrame();
	GatherStatus = Impl::EGatherStatus::TickActiveGatherActive;
	PerformanceMode = Impl::EPerformanceMode::MostlyStatic;

	bSearchAllAssets = false;
#if NO_LOGGING
	bVerboseLogging = false;
#else
	bVerboseLogging = LogAssetRegistry.GetVerbosity() >= ELogVerbosity::Verbose;
#endif
	StoreGatherResultsTimeSeconds = 0.f;

	// By default update the disk cache once on asset load, to incorporate changes made in PostLoad. This only happens in editor builds
#if !WITH_EDITOR
	Context.bUpdateDiskCacheAfterLoad = false;
#else
	if (IsRunningCookCommandlet())
	{
		Context.bUpdateDiskCacheAfterLoad = false;
	}
	else
	{
		Context.bUpdateDiskCacheAfterLoad = true;
		if (GConfig)
		{
			GConfig->GetBool(TEXT("AssetRegistry"), TEXT("bUpdateDiskCacheAfterLoad"), Context.bUpdateDiskCacheAfterLoad, GEngineIni);
		}
	}
#endif

	bIsTempCachingAlwaysEnabled = ASSETREGISTRY_CACHE_ALWAYS_ENABLED;
	bIsTempCachingEnabled = bIsTempCachingAlwaysEnabled;
	TempCachedInheritanceBuffer.bDirty = true;

	SavedGeneratorClassesVersionNumber = MAX_uint64;
	SavedAllClassesVersionNumber = MAX_uint64;

	// By default do not double check mount points are still valid when gathering new assets
	bVerifyMountPointAfterGather = false;

#if WITH_EDITOR
	if (GIsEditor)
	{
		// Double check mount point is still valid because it could have been unmounted
		bVerifyMountPointAfterGather = true;
	}
#endif // WITH_EDITOR

	// Collect all code generator classes (currently BlueprintCore-derived ones)
	CollectCodeGeneratorClasses();
#if WITH_ENGINE && WITH_EDITOR
	{
		UE::TWriteScopeLock WriteLock(SkipClassesLock);
		Utils::PopulateSkipClasses(SkipUncookedClasses, SkipCookedClasses);
	}
#endif

	// Read default serialization options
	const bool bSuppressWarnings = true;
	Utils::InitializeSerializationOptionsFromIni(SerializationOptions, &CachedParsedFilterRules, nullptr, UE::AssetRegistry::ESerializationTarget::ForGame, bSuppressWarnings);
	Utils::InitializeSerializationOptionsFromIni(DevelopmentSerializationOptions, nullptr, nullptr, UE::AssetRegistry::ESerializationTarget::ForDevelopment, bSuppressWarnings);

	bool bStartedAsyncGather = false;
	if (ShouldSearchAllAssetsAtStart())
	{
		verify(TryConstructGathererIfNeeded());

		if (GlobalGatherer->IsAsyncEnabled())
		{
			SearchAllAssetsInitialAsync(Context.Events, Context.InheritanceContext);
			bStartedAsyncGather = true;
		}
		else
		{
			// For the Editor and editor game we need to take responsibility for the synchronous search;
			// Commandlets and cooked game will handle it themselves.
#if WITH_EDITOR
			Context.bNeedsSearchAllAssetsAtStartSynchronous = !IsRunningCommandlet();
#else
			Context.bNeedsSearchAllAssetsAtStartSynchronous = false;
#endif 
		}
	}

	Context.UARI.ActiveMountsLoader.InitializeConfig();

	ConsumeOrDeferPreloadedPremade(Context.UARI, Context.Events, Context.ActiveMountEvents, ScopeLock);

	// Report startup time. This does not include DirectoryWatcher startup time.
	double StartupDuration = FPlatformTime::Seconds() - StartupStartTime;
	UE_LOGF(LogAssetRegistry, Log, "FAssetRegistry took %0.4f seconds to start up", StartupDuration);
	
	FTelemetryRouter::Get().ProvideTelemetry<UE::Telemetry::AssetRegistry::FStartupTelemetry>({
		StartupDuration,
		bStartedAsyncGather
	});

	// Content roots always exist; add them as paths
	FPackageName::QueryRootContentPaths(Context.RootContentPaths, false, false, true);
	for (const FString& AssetPath : Context.RootContentPaths)
	{
		AddPath(Context.Events, AssetPath);
	}

	InitRedirectors(Context.Events, Context.InheritanceContext, Context.bRedirectorsNeedSubscribe);

#if WITH_EDITOR
	// Make sure first call to LoadCalculatedDependencies builds the Gatherer list. At that point Classes should be loaded.
	bRegisteredDependencyGathererClassesDirty = true;
#endif

	bInitializationComplete = true;
}

}

IAssetRegistry::FAssetCollisionEvent& UE::AssetRegistry::FAssetRegistryImpl::OnAssetCollision_Private()
{
	return AssetCollisionEvent;
}

namespace UE::AssetRegistry
{

#if WITH_EDITOR
void FAssetRegistryImpl::RebuildAssetDependencyGathererMapIfNeeded()
{
	if (!bRegisteredDependencyGathererClassesDirty)
	{
		return;
	}

	FWriteScopeLock ScopeLock(RegisteredDependencyGathererClassesLock);

	RegisteredDependencyGathererClasses.Reset();

	TArray<UObject*> Classes;
	GetObjectsOfClass(UClass::StaticClass(), Classes);

	/** Per Class dependency gatherers */
	UE::AssetDependencyGatherer::Private::FRegisteredAssetDependencyGatherer::ForEach([&](UE::AssetDependencyGatherer::Private::FRegisteredAssetDependencyGatherer* RegisteredAssetDependencyGatherer)
	{
		UClass* AssetClass = RegisteredAssetDependencyGatherer->GetAssetClass();
		for (UObject* ClassObject : Classes)
		{
			if (UClass* Class = Cast<UClass>(ClassObject); Class && Class->IsChildOf(AssetClass) && !Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
			{
				RegisteredDependencyGathererClasses.Add(FTopLevelAssetPath(Class), RegisteredAssetDependencyGatherer);
			}
		}
	});

	bRegisteredDependencyGathererClassesDirty = false;
}

void FAssetRegistryImpl::ResolveRedirectionsInCachedBPInheritanceMap()
{
	for (TPair<FTopLevelAssetPath, FTopLevelAssetPath>& Pair : CachedBPInheritanceMap)
	{
		FSoftObjectPath BaseClassPath(Pair.Value);
		FSoftObjectPath TargetBaseClassPath = GRedirectCollector.GetAssetPathRedirection(BaseClassPath);
		if (TargetBaseClassPath.IsAsset())
		{
			Pair.Value = TargetBaseClassPath.GetAssetPath();
			TempCachedInheritanceBuffer.bDirty = true;
		}
	}
}

#endif

}

void UAssetRegistryImpl::InitializeEvents(UE::AssetRegistry::Impl::FInitializeContext& Context)
{
	if (Context.bRedirectorsNeedSubscribe)
	{
		TDelegate<bool(const FString&, FString&)> PackageResolveDelegate;
		PackageResolveDelegate.BindUObject(this, &UAssetRegistryImpl::OnResolveRedirect);
		FCoreDelegates::PackageNameResolvers.Add(PackageResolveDelegate);
	}

#if WITH_EDITOR
	if (UE::AssetRegistry::Impl::IsDirectoryWatcherEnabled())
	{
		FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
		IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get();

		if (DirectoryWatcher)
		{
			// The vast majority of directories we are watching are below the Plugin directories. The memory cost per watch
			// is sufficiently high to want to avoid setting up many granular watches when we can also setup two coarse ones.

			// Don't add any roots in configurations where the feature is disabled; their existence can cause performance
			// problems when there are too many disk changes in a short amount of time and the directory watcher's buffer
			// overflows and it issues a FCA_RescanRequired; in that case with one large root we rescan many unrelated
			// directories.

#if UE_ENABLE_DIRECTORYWATCH_ROOTS
			const FString ProjectPluginDir = UE::AssetRegistry::CreateStandardFilename(FPaths::ProjectPluginsDir());
			if (IPlatformFile::GetPlatformPhysical().DirectoryExists(*ProjectPluginDir))
			{
				DirectoryWatchRoots.Add(ProjectPluginDir);
			}
			const FString EnginePluginDir = UE::AssetRegistry::CreateStandardFilename(FPaths::EnginePluginsDir());
			if (IPlatformFile::GetPlatformPhysical().DirectoryExists(*EnginePluginDir))
			{
				DirectoryWatchRoots.Add(EnginePluginDir);
			}

			for (FString& WatchRoot : DirectoryWatchRoots)
			{
				FDelegateHandle NewHandle;
				DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(
					WatchRoot,
					IDirectoryWatcher::FDirectoryChanged::CreateUObject(this, &UAssetRegistryImpl::OnDirectoryChanged),
					NewHandle,
					IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges);

				OnDirectoryChangedDelegateHandles.Add(WatchRoot, NewHandle);
			}
#endif //UE_ENABLE_DIRECTORYWATCH_ROOTS

			FString ContentFolder;
			for (TArray<FString>::TConstIterator RootPathIt(Context.RootContentPaths); RootPathIt; ++RootPathIt)
			{
				const FString& RootPath = *RootPathIt;
				ContentFolder = UE::AssetRegistry::CreateStandardFilename(FPackageName::LongPackageNameToFilename(RootPath));
				if (IsDirAlreadyWatchedByRootWatchers(ContentFolder))
				{
					continue;
				}

				// A missing directory here could be due to a plugin that specifies it contains content, yet has no content yet.
				// PluginManager mounts these folders anyway which results in them being returned from QueryRootContentPaths.
				// Make sure the directory exits on disk so that the OS level DirectoryWatcher can be used to monitor it.
				IPlatformFile::GetPlatformPhysical().CreateDirectoryTree(*ContentFolder);
				FDelegateHandle NewHandle;
				DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(
						ContentFolder,
						IDirectoryWatcher::FDirectoryChanged::CreateUObject(this, &UAssetRegistryImpl::OnDirectoryChanged),
						NewHandle,
						IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges);

				OnDirectoryChangedDelegateHandles.Add(RootPath, NewHandle);
			}
		}
	}

	bUpdateDiskCacheAfterLoad = Context.bUpdateDiskCacheAfterLoad;
	if (bUpdateDiskCacheAfterLoad)
	{
		FCoreUObjectDelegates::OnAssetLoaded.AddUObject(this, &UAssetRegistryImpl::OnAssetLoaded);
	}

	if (bAddMetaDataTagsToOnGetExtraObjectTags)
	{
		UObject::FAssetRegistryTag::OnGetExtraObjectTagsWithContext.AddUObject(this, &UAssetRegistryImpl::OnGetExtraObjectTags);
	}
	if (Context.bNeedsSearchAllAssetsAtStartSynchronous)
	{
		FCoreDelegates::OnFEngineLoopInitComplete.AddUObject(this, &UAssetRegistryImpl::OnFEngineLoopInitCompleteSearchAllAssets);
	}

	UE::AssetDependencyGatherer::Private::FRegisteredAssetDependencyGatherer::OnAssetDependencyGathererRegistered.AddUObject(this, &UAssetRegistryImpl::OnAssetDependencyGathererRegistered);
#endif // WITH_EDITOR

	// We use OnPreExit and not OnEnginePreExit because OnPreExit will be called if there's an error in engine init and 
	// we never get through OnPostEngineInit.
	FCoreDelegates::OnPreExit.AddUObject(this, &UAssetRegistryImpl::OnPreExit);

	// Listen for new content paths being added or removed at runtime.  These are usually plugin-specific asset paths that
	// will be loaded a bit later on.
	FPackageName::OnMountPointMounted().AddUObject(this, &UAssetRegistryImpl::OnMountPointMounted);
	FPackageName::OnMountPointDismounted().AddUObject(this, &UAssetRegistryImpl::OnMountPointDismounted);

	// If we were called before engine has fully initialized, refresh classes on initialize. If not this won't do anything as it already happened
	FCoreDelegates::GetOnPostEngineInit().AddUObject(this, &UAssetRegistryImpl::OnPostEngineInit);

	IPluginManager& PluginManager = IPluginManager::Get();
	if (!IsEngineStartupModuleLoadingComplete())
	{
		FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddUObject(this, &UAssetRegistryImpl::OnInitialPluginLoadingComplete);
	}
	else
	{
		OnInitialPluginLoadingComplete();
	}
}

UAssetRegistryImpl::UAssetRegistryImpl(FVTableHelper& Helper)
	: Super(Helper)
{
}

bool UAssetRegistryImpl::OnResolveRedirect(const FString& InPackageName, FString& OutPackageName)
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	return GuardedData.ResolveRedirect(InPackageName, OutPackageName);
}

namespace UE::AssetRegistry
{

bool FAssetRegistryImpl::ResolveRedirect(const FString& InPackageName, FString& OutPackageName) const
{
	int32 DotIndex = InPackageName.Find(TEXT("."), ESearchCase::CaseSensitive);

	FString ContainerPackageName; 
	const FString* PackageNamePtr = &InPackageName; // don't return this
	if (DotIndex != INDEX_NONE)
	{
		ContainerPackageName = InPackageName.Left(DotIndex);
		PackageNamePtr = &ContainerPackageName;
	}
	const FString& PackageName = *PackageNamePtr;

	for (const FAssetRegistryPackageRedirect& PackageRedirect : PackageRedirects)
	{
		if (PackageName.Compare(PackageRedirect.SourcePackageName) == 0)
		{
			OutPackageName = InPackageName.Replace(*PackageRedirect.SourcePackageName, *PackageRedirect.DestPackageName);
			return true;
		}
	}
	return false;
}

void FAssetRegistryImpl::InitRedirectors(Impl::FEventContext& EventContext,
	Impl::FClassInheritanceContext& InheritanceContext, bool& bOutRedirectorsNeedSubscribe)
{
	bOutRedirectorsNeedSubscribe = false;

	// plugins can't initialize redirectors in the editor, it will mess up the saving of content.
	if ( GIsEditor )
	{
		return;
	}

	TArray<TSharedRef<IPlugin>> EnabledPlugins = IPluginManager::Get().GetEnabledPlugins();
	for (const TSharedRef<IPlugin>& Plugin : EnabledPlugins)
	{
		FString PluginConfigFilename = FConfigCacheIni::NormalizeConfigIniPath(FString::Printf(TEXT("%s%s/%s.ini"), *FPaths::GeneratedConfigDir(), ANSI_TO_TCHAR(FPlatformProperties::PlatformName()), *Plugin->GetName() ));
		
		bool bShouldRemap = false;
		
		if ( !GConfig->GetBool(TEXT("PluginSettings"), TEXT("RemapPluginContentToGame"), bShouldRemap, PluginConfigFilename) )
		{
			continue;
		}

		if (!bShouldRemap)
		{
			continue;
		}

		// if we are -game or -server in editor build we might need to initialize the asset registry manually for this plugin
		if (!FPlatformProperties::RequiresCookedData() && (IsRunningGame() || IsRunningDedicatedServer()))
		{
			TArray<FString> PathsToSearch;
			
			FString RootPackageName = FString::Printf(TEXT("/%s/"), *Plugin->GetName());
			PathsToSearch.Add(RootPackageName);

			Impl::FScanPathContext Context(EventContext, InheritanceContext, PathsToSearch, TArray<FString>());
			ScanPathsSynchronous(nullptr /*ScopeLock*/, Context);
		}

		FName PluginPackageName = FName(*FString::Printf(TEXT("/%s/"), *Plugin->GetName()));
		EnumerateAssetsByPathNoTags(PluginPackageName,
			[&Plugin, this](const FAssetData& PartialAssetData)
			{
				FString NewPackageNameString = PartialAssetData.PackageName.ToString();
				FString RootPackageName = FString::Printf(TEXT("/%s/"), *Plugin->GetName());
				FString OriginalPackageNameString = NewPackageNameString.Replace(*RootPackageName, TEXT("/Game/"));

				PackageRedirects.Add(FAssetRegistryPackageRedirect(OriginalPackageNameString, NewPackageNameString));
				return true;
			}, true, false);

		bOutRedirectorsNeedSubscribe = true;
	}
}

}

void UAssetRegistryImpl::OnInitialPluginLoadingComplete()
{
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		const bool bSuppressWarnings = !UE::AssetRegistry::Utils::HasEngineModuleLoaded();
		UE::AssetRegistry::Utils::UpdateSerializationOptions(GuardedData.SerializationOptions, GuardedData.CachedParsedFilterRules, bSuppressWarnings);
		GuardedData.OnPluginLoadingComplete(true);
	}

	FCoreDelegates::OnAllModuleLoadingPhasesComplete.RemoveAll(this);
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::OnPluginLoadingComplete(bool bPhaseSuccessful)
{
	// If we have constructed the GlobalGatherer then we need to readscriptpackages,
	// otherwise we will read them when constructing the gatherer.
	if (GlobalGatherer.IsValid())
	{
		ReadScriptPackages();
	}

	// Reparse the skip classes the next time ShouldSkipAsset is called, since available classes
	// for the search over all classes may have changed
#if WITH_ENGINE && WITH_EDITOR
	{
		UE::TWriteScopeLock WriteLock(SkipClassesLock);
		Utils::PopulateSkipClasses(SkipUncookedClasses, SkipCookedClasses);
	}
#endif
}

void FAssetRegistryImpl::ReadScriptPackages()
{
	GlobalGatherer->SetInitialPluginsLoaded();
	if (GlobalGatherer->IsGatheringDependencies())
	{
		// Now that all scripts have been loaded, we need to create AssetPackageDatas for every script
		// This is also done whenever scripts are referenced in our gather of existing packages,
		// but we need to complete it for all scripts that were referenced but not yet loaded for packages
		// that we already gathered
		for (TObjectIterator<UPackage> It; It; ++It)
		{
			UPackage* Package = *It;
			if (Package)
			{
				if (Package && FPackageName::IsScriptPackage(Package->GetName()))
				{
					FAssetPackageData* ScriptPackageData = State.CreateOrGetAssetPackageData(Package->GetFName());
#if WITH_EDITORONLY_DATA
					// Get the hash off the script package, it is updated when script is changed so we need to refresh it every run
					ScriptPackageData->SetPackageSavedHash(Package->GetSavedHash());
#endif
				}
			}
		}
	}
}

}

void UAssetRegistryImpl::InitializeSerializationOptions(FAssetRegistrySerializationOptions& Options, const ITargetPlatform* TargetPlatform, UE::AssetRegistry::ESerializationTarget Target) const
{
	if (!TargetPlatform)
	{
		UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
		// Use options we already loaded, the first pass for this happens at object creation time so this is always valid when queried externally
		GuardedData.CopySerializationOptions(Options, Target);
	}
	else
	{
		// Temp container to parse and resolve FilterTags
		UE::AssetRegistry::Impl::FFilterTagRules TempParsedFilterRules;
		const bool bSuppressWarnings = !UE::AssetRegistry::Utils::HasEngineModuleLoaded();
		UE::AssetRegistry::Utils::InitializeSerializationOptionsFromIni(Options, &TempParsedFilterRules, TargetPlatform, Target, bSuppressWarnings);
	}
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::CopySerializationOptions(FAssetRegistrySerializationOptions& OutOptions, ESerializationTarget Target) const
{
	if (Target == UE::AssetRegistry::ESerializationTarget::ForGame)
	{
		OutOptions = SerializationOptions;
	}
	else
	{
		OutOptions = DevelopmentSerializationOptions;
	}
}

namespace Utils
{

static TSet<FName> MakeNameSet(const TArray<FString>& Strings)
{
	TSet<FName> Out;
	Out.Reserve(Strings.Num());
	for (const FString& String : Strings)
	{
		Out.Add(FName(*String));
	}

	return Out;
}

void InitializeSerializationOptionsFromIni(
	FAssetRegistrySerializationOptions& Options,
	Impl::FFilterTagRules* OutRules,
	const ITargetPlatform* TargetPlatform,
	UE::AssetRegistry::ESerializationTarget Target,
	bool bSuppressWarnings)
{
	FString PlatformIniName;
	EBuildTargetType TargetType = EBuildTargetType::Game;
	if (TargetPlatform)
	{
#if WITH_EDITOR
		PlatformIniName = TargetPlatform->IniPlatformName();
		TargetType = TargetPlatform->GetRuntimePlatformType();
#endif
	}
	else
	{
		PlatformIniName = ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName());
		TargetType = FPlatformProperties::IsClientOnly() ? EBuildTargetType::Client
			: FPlatformProperties::IsServerOnly() ? EBuildTargetType::Server
			: EBuildTargetType::Game;
	}
	// Use passed in platform, or current platform if empty
	FConfigFile LocalEngineIni;
	FConfigFile* EngineIni = FConfigCacheIni::FindOrLoadPlatformConfig(LocalEngineIni, TEXT("Engine"), *PlatformIniName);

	Options = FAssetRegistrySerializationOptions(Target);
	// For DevelopmentAssetRegistry, all non-tag options are overridden in the constructor
	const bool bForDevelopment = Target == UE::AssetRegistry::ESerializationTarget::ForDevelopment;
	if (!bForDevelopment)
	{
		EngineIni->GetBool(TEXT("AssetRegistry"), TEXT("bSerializeAssetRegistry"), Options.bSerializeAssetRegistry);
		EngineIni->GetBool(TEXT("AssetRegistry"), TEXT("bSerializeDependencies"), Options.bSerializeDependencies);
		EngineIni->GetBool(TEXT("AssetRegistry"), TEXT("bSerializeNameDependencies"), Options.bSerializeSearchableNameDependencies);
		EngineIni->GetBool(TEXT("AssetRegistry"), TEXT("bSerializeManageDependencies"), Options.bSerializeManageDependencies);
		EngineIni->GetBool(TEXT("AssetRegistry"), TEXT("bSerializePackageData"), Options.bSerializePackageData);
		EngineIni->GetBool(TEXT("AssetRegistry"), TEXT("bFilterAssetDataWithNoTags"), Options.bFilterAssetDataWithNoTags);
		EngineIni->GetBool(TEXT("AssetRegistry"), TEXT("bFilterDependenciesWithNoTags"), Options.bFilterDependenciesWithNoTags);
		EngineIni->GetBool(TEXT("AssetRegistry"), TEXT("bFilterSearchableNames"), Options.bFilterSearchableNames);

		{
			TArray<FString> AssetBundlesDenyList;
			EngineIni->GetArray(TEXT("AssetRegistry"), TEXT("AssetBundlesDenyList"), AssetBundlesDenyList);
			Options.AssetBundlesDenyList.Append(AssetBundlesDenyList);
		}
		if (TargetType == EBuildTargetType::Server)
		{
			TArray<FString> AssetBundlesDenyList;
			EngineIni->GetArray(TEXT("AssetRegistry"), TEXT("ServerAssetBundlesDenyList"), AssetBundlesDenyList);
			Options.AssetBundlesDenyList.Append(AssetBundlesDenyList);
		}
		if (TargetType == EBuildTargetType::Client)
		{
			TArray<FString> AssetBundlesDenyList;
			EngineIni->GetArray(TEXT("AssetRegistry"), TEXT("ClientAssetBundlesDenyList"), AssetBundlesDenyList);
			Options.AssetBundlesDenyList.Append(AssetBundlesDenyList);
		}
	}

	if (OutRules)
	{
		EngineIni->GetBool(TEXT("AssetRegistry"), TEXT("bUseAssetRegistryTagsWhitelistInsteadOfBlacklist"), Options.bUseAssetRegistryTagsAllowListInsteadOfDenyList);
		TArray<FString> FilterListItems;
		if (Options.bUseAssetRegistryTagsAllowListInsteadOfDenyList)
		{
			EngineIni->GetArray(TEXT("AssetRegistry"), TEXT("CookedTagsWhitelist"), FilterListItems);
		}
		else
		{
			EngineIni->GetArray(TEXT("AssetRegistry"), TEXT("CookedTagsBlacklist"), FilterListItems);
		}

		const bool bKeepDevelopmentTags = bForDevelopment || FParse::Param(FCommandLine::Get(), TEXT("ARKeepDevTags"));
		const bool bLoadClasses = true;
		const bool bIsFilteringDevelopmentAR = bKeepDevelopmentTags;
		OutRules->Rules = ParseFilterTags(FilterListItems, bKeepDevelopmentTags, Options.bUseAssetRegistryTagsAllowListInsteadOfDenyList);
		UpdateSerializationOptions(Options, *OutRules, bSuppressWarnings);
	}
	else
	{
		// By default all tags are included
		Options.bUseAssetRegistryTagsAllowListInsteadOfDenyList = false;
	}

	{
		// this only needs to be done once, and only on builds using USE_COMPACT_ASSET_REGISTRY
		TArray<FString> AsFName;
		EngineIni->GetArray(TEXT("AssetRegistry"), TEXT("CookedTagsAsFName"), AsFName);
		Options.CookTagsAsName = MakeNameSet(AsFName);

		TArray<FString> AsPathName;
		EngineIni->GetArray(TEXT("AssetRegistry"), TEXT("CookedTagsAsPathName"), AsPathName);
		Options.CookTagsAsPath = MakeNameSet(AsPathName);
	}
}

void UpdateSerializationOptions(
	FAssetRegistrySerializationOptions& InOutOptions,
	Impl::FFilterTagRules& InOutFilterRules,
	bool bSuppressWarnings)
{
	if (InOutFilterRules.bContainsAnyUnresolvedRule || (InOutFilterRules.ClassesFingerprint != GetRegisteredNativeClassesVersionNumber()))
	{
		InOutOptions.CookFilterlistTagsByClass = ResolveFilterTags(InOutFilterRules, bSuppressWarnings);
	}
}

static int32 CalculateInheritanceDepth(const UClass* Class)
{
	int32 InheritanceDepth = 0;
	if (Class)
	{
		for (auto Iter = Class->GetSuperStructIterator(); Iter; ++Iter)
		{
			++InheritanceDepth;
		}
	}
	return InheritanceDepth;
}

TMap<FName, Impl::FParsedFilterTagRules> ParseFilterTags(const TArray<FString>& FilterListItems, bool bIsFilteringDevelopmentAR , bool bUseAllowListInsteadOfDenyList)
{
	TMap<FName, Impl::FParsedFilterTagRules> TagsOrder;

	// Takes on the pattern "(Class=SomeClass,Tag=SomeTag)"
	// Optional key KeepInDevOnly for tweaking a DevelopmentAssetRegistry (additive if allow list, subtractive if deny list)
	for (const FString& FilterEntry : FilterListItems)
	{
		FString TrimmedEntry = FilterEntry;
		TrimmedEntry.TrimStartAndEndInline();
		if (TrimmedEntry.Left(1) == TEXT("("))
		{
			TrimmedEntry.RightChopInline(1, EAllowShrinking::No);
		}
		if (TrimmedEntry.Right(1) == TEXT(")"))
		{
			TrimmedEntry.LeftChopInline(1, EAllowShrinking::No);
		}

		TArray<FString> Tokens;
		TrimmedEntry.ParseIntoArray(Tokens, TEXT(","));
		FString ClassName;
		FString TagName;
		bool bKeepInDevOnly = false;

		for (const FString& Token : Tokens)
		{
			FString KeyString;
			FString ValueString;
			if (Token.Split(TEXT("="), &KeyString, &ValueString))
			{
				KeyString.TrimStartAndEndInline();
				ValueString.TrimStartAndEndInline();
				if (KeyString.Equals(TEXT("Class"), ESearchCase::IgnoreCase))
				{
					ClassName = ValueString;
				}
				else if (KeyString.Equals(TEXT("Tag"), ESearchCase::IgnoreCase))
				{
					TagName = ValueString;
				}
			}
			else
			{
				KeyString = Token.TrimStartAndEnd();
				if (KeyString.Equals(TEXT("KeepInDevOnly"), ESearchCase::IgnoreCase))
				{
					bKeepInDevOnly = true;
				}
			}
		}

		bool bAddToList;
		if (bUseAllowListInsteadOfDenyList)
		{
			bAddToList = !bKeepInDevOnly || bIsFilteringDevelopmentAR;
		}
		else
		{
			// "KeepInDevOnly" means "Keep the tag only when creating the DevelopmentAssetRegistry"
			// That is complicated meaning when we are creating a DenyList, wherein adding an element to the list removes
			// it from the filtered collection of tags.
			// We add it to the list if either it applies to all ARs because KeepInDevOnly was not set, or if we are creating
			// the GameAssetRegistry.
			bAddToList = !bKeepInDevOnly || !bIsFilteringDevelopmentAR;
		}

		if (!ClassName.IsEmpty() && !TagName.IsEmpty())
		{
			Impl::FParsedFilterTagRules& Rules = TagsOrder.FindOrAdd(FName(TagName.Len(), *TagName));
			TArray<Impl::FParsedFilterTagRules::FFilterTagsRule>& OrderedList = Rules.OrderedList;
			const UClass* FilterlistClass = nullptr;
			// Skip known wildcard
			if (ClassName != TEXTVIEW("*"))
			{
				FilterlistClass = Cast<UClass>(StaticFindObject(UClass::StaticClass(), nullptr, *ClassName));
			}

			if (FilterlistClass)
			{
				const int32 InheritanceDepth = CalculateInheritanceDepth(FilterlistClass);
				// Essentially I want to do OrderedList.HeapPush(FFilterTagsRule{...}, &GetInheritanceDepth);
				// but there is no TArray::HeapPush which takes in an Item and a projection functor.
				// Also there is no equivalent Emplace semantics to construct object inplace without extra copies/moves.
				// So here is an inline implementation of this:
				OrderedList.Emplace(FilterlistClass->GetClassPathName(), FilterlistClass, InheritanceDepth, bAddToList);
				if (!Rules.bContainsAnyUnresolvedRule)
				{
					AlgoImpl::HeapSiftUp(
						OrderedList.GetData(),
						0,
						OrderedList.Num() - 1,
						[](const Impl::FParsedFilterTagRules::FFilterTagsRule& Info)
							{
								return Info.InheritanceDepth; 
							},
						TLess<int32>());
				}
			}
			else
			{
				FTopLevelAssetPath ClassPathName;
				if (ClassName == TEXTVIEW("*"))
				{
					ClassPathName = UE::AssetRegistry::WildcardPathName;
				}
				else if (FPackageName::IsShortPackageName(ClassName))
				{
					ClassPathName = UClass::TryConvertShortTypeNameToPathName<UClass>(ClassName, ELogVerbosity::Warning, TEXT("Parsing [AssetRegistry] CookedTagsWhitelist or CookedTagsBlacklist"));
					UE_CLOGF(ClassPathName.IsNull(), LogAssetRegistry, Warning, "Failed to convert short class name \"%ls\" when parsing ini [AssetRegistry] CookedTagsWhitelist or CookedTagsBlacklist", *ClassName);
				}
				else
				{
					ClassPathName = FTopLevelAssetPath(ClassName);
				}

				if (!ClassPathName.IsValid())
				{
					UE_LOGF(LogAssetRegistry, Warning, 
						"There is a rule to %ls tag \"%ls\" for class \"%ls\", which is not a valid class path. Ignoring.",
						bAddToList ? TEXT("included") : TEXT("excluded"),
						*TagName,
						*ClassName
					);
					continue;
				}

				Rules.bContainsAnyUnresolvedRule = true;
				OrderedList.Emplace(ClassPathName, nullptr, -1, bAddToList);
			}
		}
	}

	return TagsOrder;
}

bool HasEngineModuleLoaded()
{
	static bool bIsEngineModuleAvaliable = false;
	if (bIsEngineModuleAvaliable)
	{
		return true;
	}
	bIsEngineModuleAvaliable = FModuleManager::Get().IsModuleLoaded(TEXT("Engine"));
	return bIsEngineModuleAvaliable;
}

static void ResolveRules(
	FStringView TagName,
	TArray<Impl::FParsedFilterTagRules::FFilterTagsRule>& OrderedList,
	bool bSuppressWarnings,
	bool* bAreRulesResolved)
{
	auto RuleIt = OrderedList.CreateIterator();
	if (RuleIt->ClassPath == UE::AssetRegistry::WildcardPathName)
	{
		// Move cursor to next element and delete the rest
		for (++RuleIt; RuleIt; ++RuleIt)
		{
			const bool bIsDuplicate = (OrderedList[0].bInclude == RuleIt->bInclude);
			UE_CLOGF(!bSuppressWarnings && !bIsDuplicate, LogAssetRegistry, Warning, 
				"There is a wildcard rule to %ls tag \"%.*ls\", " "but there is another rule to %ls that tag for class \"%ls\". " "Only wildcard rule is accepted. Ignoring another rule.",
				OrderedList[0].bInclude ? TEXT("include") : TEXT("exclude"),
				TagName.Len(), TagName.GetData(),
				RuleIt->bInclude ? TEXT("include") : TEXT("exclude"),
				*RuleIt->ClassPath.ToString()
			);
			RuleIt.RemoveCurrentSwap();
		}
		// Wildcard is considered resolved rule
		if (bAreRulesResolved)
		{
			*bAreRulesResolved = true;
		}
		return;
	}

	bool bContainAnyUnresolvedRules = false;
	for (; RuleIt; ++RuleIt)
	{
		// Currently there is no case when we don't want to cause sync load of the class we try to resolve.
		// Expose as a function parameter if that ever will need to change.
		constexpr bool bLoadClasses = true;
		RuleIt->ClassPtr = bLoadClasses
			? StaticLoadClass(UObject::StaticClass(), nullptr, RuleIt->ClassPath.ToString(), {}, LOAD_Quiet)
			: Cast<UClass>(StaticFindObject(UClass::StaticClass(), RuleIt->ClassPath));
		if (RuleIt->ClassPtr)
		{
			RuleIt->InheritanceDepth = CalculateInheritanceDepth(RuleIt->ClassPtr);
		}
		else
		{
			// Silently ignore same unresolved rules
			const bool bIsDuplicate = (RuleIt.GetIndex() != 0) && (OrderedList[RuleIt.GetIndex()-1].bInclude == RuleIt->bInclude);
			UE_CLOGF(!bSuppressWarnings && !bIsDuplicate, LogAssetRegistry, Warning,
				"There is a rule to %ls tag \"%.*ls\" for class \"%ls\". Class is unknown. Ignoring.",
				RuleIt->bInclude ? TEXT("include") : TEXT("exclude"),
				TagName.Len(), TagName.GetData(),
				*RuleIt->ClassPath.ToString()
			);
			// Setting unresolved rule to max_int causes resolved rules to be applied anyway and any unresolved to be skipped.
			RuleIt->InheritanceDepth = MAX_int32;
			bContainAnyUnresolvedRules = true;
		}
	}

	Algo::HeapSortBy(OrderedList, [](const Impl::FParsedFilterTagRules::FFilterTagsRule& Info) { return Info.InheritanceDepth; }, TLess<int32>());

	if (bAreRulesResolved)
	{
		*bAreRulesResolved = !bContainAnyUnresolvedRules;
	}
}

TMap<FTopLevelAssetPath, TSet<FName>> ResolveFilterTags(Impl::FFilterTagRules& InOutRules, bool bSuppressWarnings)
{
	TMap<FTopLevelAssetPath, TSet<FName>> Result;
	// keep temp allocation around while we traverse the ordered list
	TSet<const UClass*> ClassesToAdd;
	InOutRules.bContainsAnyUnresolvedRule = false;
	// Populate CookFilterlistTagsByClass by walking the ordered list
	for (TTuple<FName, Impl::FParsedFilterTagRules>& Pair : InOutRules.Rules)
	{
		const FName Tag = Pair.Key;
		TStringBuilder<512> TagNameBuilder;
		Pair.Key.ToString(TagNameBuilder);
		const FStringView TagName = TagNameBuilder.ToView();
		Impl::FParsedFilterTagRules& TagRules = Pair.Value;
		TArray<Impl::FParsedFilterTagRules::FFilterTagsRule>& OrderedList = TagRules.OrderedList;
		if (TagRules.bContainsAnyUnresolvedRule)
		{
			bool bAreRulesResolved = false;
			ResolveRules(TagName, OrderedList, bSuppressWarnings, &bAreRulesResolved);
			TagRules.bContainsAnyUnresolvedRule = !bAreRulesResolved;
		}
		if (TagRules.bContainsAnyUnresolvedRule)
		{
			InOutRules.bContainsAnyUnresolvedRule = true;
		}

		ClassesToAdd.Reset();
		// The list is sorted in heap order of Inheritance depth,
		// so we can just walk through it and add/remove as needed
		TArray<UClass*> Temp;
		for (const Impl::FParsedFilterTagRules::FFilterTagsRule& Info : OrderedList)
		{
			if (!Info.ClassPtr)
			{
				if (Info.bInclude)
				{
					Result.FindOrAdd(Info.ClassPath).Add(Tag);
				}
				break;
			}

			Temp.Reset();
			GetDerivedClasses(Info.ClassPtr, Temp, true);
			if (Info.bInclude)
			{
				ClassesToAdd.Reserve(ClassesToAdd.Num() + Temp.Num() + 1);
				ClassesToAdd.Add(Info.ClassPtr);
				ClassesToAdd.Append(Temp);
			}
			else
			{
				for (UClass* Derived : Temp)
				{
					ClassesToAdd.Remove(Derived);
				}
				ClassesToAdd.Remove(Info.ClassPtr);
			}
		}

		// After the final list of classes is known - add them to the CookFilterlistTagsByClass
		for (const UClass* Class : ClassesToAdd)
		{
			Result.FindOrAdd(Class->GetClassPathName()).Add(Tag);
		}
	}
	InOutRules.ClassesFingerprint = GetRegisteredNativeClassesVersionNumber();

	return Result;
}

}

uint64 FAssetRegistryImpl::GetCurrentGeneratorClassesVersionNumber()
{
	// Generator classes can only be native, so we can use the less-frequently-updated
	// RegisteredNativeClassesVersionNumber. In monolithic configurations, this will only be
	// updated at program start and when enabling DLC modules.
	return GetRegisteredNativeClassesVersionNumber();
}

uint64 FAssetRegistryImpl::GetCurrentAllClassesVersionNumber()
{
	return GetRegisteredClassesVersionNumber();
}

void FAssetRegistryImpl::CollectCodeGeneratorClasses()
{
	LLM_SCOPE(ELLMTag::AssetRegistry); // Tagged here instead of a higher level because it can occur even when reading
	// Only refresh the list if our registered classes have changed
	uint64 CurrentGeneratorClassesVersionNumber = GetCurrentGeneratorClassesVersionNumber();
	if (SavedGeneratorClassesVersionNumber == CurrentGeneratorClassesVersionNumber)
	{
		return;
	}
	SavedGeneratorClassesVersionNumber = CurrentGeneratorClassesVersionNumber;

	TArray<UClass*> BlueprintCoreDerivedClasses;
	FTopLevelAssetPath BlueprintCorePathName(GetClassPathBlueprintCore());
	UClass* BlueprintCoreClass = nullptr;

	{
		// FindObject and GetDerivedClasses are not legal during GarbageCollection. Note that we might be called from
		// an async thread, in which case we might lock this thread until GC completes. This could cause a deadlock if
		// there aren't enough async threads. But CollectCodeGeneratorClasses is not called on runtime or cooked
		// editor because they are monolithic, and so this lock should only occur on uncooked editor platforms, which
		// should have a high enough number of threads to not block garbage collection.
		FGCScopeGuard NoGCScopeGuard;

		// Work around the fact we don't reference Engine module directly
		BlueprintCoreClass = FindObject<UClass>(BlueprintCorePathName);
		if (!BlueprintCoreClass)
		{
			return;
		}
		GetDerivedClasses(BlueprintCoreClass, BlueprintCoreDerivedClasses);
	}

	ClassGeneratorNames.Add(BlueprintCoreClass->GetClassPathName());
	for (UClass* BPCoreClass : BlueprintCoreDerivedClasses)
	{
		bool bAlreadyRecorded;
		FTopLevelAssetPath BPCoreClassName = BPCoreClass->GetClassPathName();
		ClassGeneratorNames.Add(BPCoreClassName, &bAlreadyRecorded);
		if (bAlreadyRecorded)
		{
			continue;
		}

		// For new generator classes, add all instances of them to CachedBPInheritanceMap. This is usually done
		// when AddAssetData is called for those instances, but when we add a new generator class we have to recheck all
		// instances of the class since they would have failed to detect they were Blueprint classes before.
		// This can happen if blueprints in plugin B are scanned before their blueprint class from plugin A is scanned.
		State.EnumerateAssetsByClassPathName(BPCoreClassName, [this](const FAssetData* AssetData)
		{
			const FString GeneratedClass = AssetData->GetTagValueRef<FString>(FBlueprintTags::GeneratedClassPath);
			const FString ParentClass = AssetData->GetTagValueRef<FString>(FBlueprintTags::ParentClassPath);
			if (!GeneratedClass.IsEmpty() && !ParentClass.IsEmpty())
			{
				const FTopLevelAssetPath GeneratedClassPathName(FPackageName::ExportTextPathToObjectPath(GeneratedClass));
				const FTopLevelAssetPath ParentClassPathName(FPackageName::ExportTextPathToObjectPath(ParentClass));

				if (!CachedBPInheritanceMap.Contains(GeneratedClassPathName))
				{
					AddCachedBPClassParent(GeneratedClassPathName, ParentClassPathName);

					// Invalidate caching because CachedBPInheritanceMap got modified
					TempCachedInheritanceBuffer.bDirty = true;
				}
			}
			return true; // Keep iterating the assets for the class
		});
	}
}

}

void UAssetRegistryImpl::OnPostEngineInit()
{
	LLM_SCOPE(ELLMTag::AssetRegistry);
	UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
	GuardedData.RefreshNativeClasses();
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::RefreshNativeClasses()
{
	// Native classes have changed so reinitialize code generator, class inheritance maps,
	// and serialization options
	CollectCodeGeneratorClasses();
	TempCachedInheritanceBuffer.bDirty = true;
}

}

#if WITH_EDITOR
void UAssetRegistryImpl::OnFEngineLoopInitCompleteSearchAllAssets()
{
	SearchAllAssets(true);
}

void UAssetRegistryImpl::OnAssetDependencyGathererRegistered()
{
	LLM_SCOPE(ELLMTag::AssetRegistry);
	UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
	GuardedData.OnAssetDependencyGathererRegistered();
}
#endif

void UAssetRegistryImpl::OnPreExit()
{
	LLM_SCOPE(ELLMTag::AssetRegistry);

	TUniquePtr<FAssetDataGatherer> GlobalGatherer;
	{
		FScopeLock GatheredDataGuard(&GatheredDataProcessingLock);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		GlobalGatherer = MoveTemp(GuardedData.AccessGlobalGatherer());
		if (GlobalGatherer.IsValid())
		{
			GlobalGatherer->Stop();
		}
	}
	// Now that we are no longer holding the lock, we can destroy the gatherer
	GlobalGatherer.Reset();

	UE::AssetRegistry::Impl::FActiveMountEvents ActiveMountEvents;
	ActiveMountsLoader.Shutdown(*this, ActiveMountEvents);
	ActiveMountEvents.Broadcast();
}

void UAssetRegistryImpl::FinishDestroy()
{
	LLM_SCOPE(ELLMTag::AssetRegistry);

	{
		FScopeLock DeferredEventsLock(&DeferredEventsCriticalSection);
		ClearRequestTick();
	}
	{
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);

		// Stop listening for content mount point events
		FPackageName::OnMountPointMounted().RemoveAll(this);
		FPackageName::OnMountPointDismounted().RemoveAll(this);
		FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
		FCoreDelegates::OnPreExit.RemoveAll(this);
		IPluginManager::Get().OnLoadingPhaseComplete().RemoveAll(this);

#if WITH_EDITOR
		if (UE::AssetRegistry::Impl::IsDirectoryWatcherEnabled())
		{
			// If the directory module is still loaded, unregister any delegates
			if (FModuleManager::Get().IsModuleLoaded("DirectoryWatcher"))
			{
				FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::GetModuleChecked<FDirectoryWatcherModule>("DirectoryWatcher");
				IDirectoryWatcher* DirectoryWatcher = DirectoryWatcherModule.Get();

				if (DirectoryWatcher)
				{
					TArray<FString> RootContentPaths;
					FPackageName::QueryRootContentPaths(RootContentPaths);
					for (TArray<FString>::TConstIterator RootPathIt(RootContentPaths); RootPathIt; ++RootPathIt)
					{
						const FString& RootPath = *RootPathIt;
						const FString ContentFolder = UE::AssetRegistry::CreateStandardFilename(FPackageName::LongPackageNameToFilename(RootPath));
						if (!IsDirAlreadyWatchedByRootWatchers(ContentFolder))
						{
							DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(ContentFolder, OnDirectoryChangedDelegateHandles.FindRef(RootPath));
						}
					}

					for (TArray<FString>::TConstIterator RootPathIt(DirectoryWatchRoots); RootPathIt; ++RootPathIt)
					{
						const FString& RootPath = *RootPathIt;
						DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(RootPath, OnDirectoryChangedDelegateHandles.FindRef(RootPath));
					}
					DirectoryWatchRoots.Empty();
				}
			}
		}

		if (bUpdateDiskCacheAfterLoad)
		{
			FCoreUObjectDelegates::OnAssetLoaded.RemoveAll(this);
		}

		if (bAddMetaDataTagsToOnGetExtraObjectTags)
		{
			UObject::FAssetRegistryTag::OnGetExtraObjectTagsWithContext.RemoveAll(this);
		}
		FCoreDelegates::OnFEngineLoopInitComplete.RemoveAll(this);

		UE::AssetDependencyGatherer::Private::FRegisteredAssetDependencyGatherer::OnAssetDependencyGathererRegistered.RemoveAll(this);
#endif // WITH_EDITOR

		if (HasAnyFlags(RF_ClassDefaultObject) && !HasAnyFlags(RF_ImmutableDefaultObject))
		{
			check(UE::AssetRegistry::Private::IAssetRegistrySingleton::Singleton == this && IAssetRegistryInterface::Default == &GAssetRegistryInterface);
			UE::AssetRegistry::Private::IAssetRegistrySingleton::Singleton = nullptr;
			IAssetRegistryInterface::Default = nullptr;
		}

		// Clear all listeners
		PathAddedEvent.Clear();
		PathRemovedEvent.Clear();
		AssetAddedEvent.Clear();
		AssetRemovedEvent.Clear();
		AssetRenamedEvent.Clear();
		AssetUpdatedEvent.Clear();
		AssetUpdatedOnDiskEvent.Clear();
		for (FAssetsEvent& Event : BatchedAssetEvents)
		{
			Event.Clear();
		}
		InMemoryAssetCreatedEvent.Clear();
		InMemoryAssetDeletedEvent.Clear();
		FileLoadedEvent.Clear();
		FileLoadProgressUpdatedEvent.Clear();
	}

	Super::FinishDestroy();
}

UAssetRegistryImpl::~UAssetRegistryImpl()
{
}

UAssetRegistryImpl& UAssetRegistryImpl::Get()
{
	check(UE::AssetRegistry::Private::IAssetRegistrySingleton::Singleton);
	return static_cast<UAssetRegistryImpl&>(*UE::AssetRegistry::Private::IAssetRegistrySingleton::Singleton);
}

namespace UE::AssetRegistry
{

bool FAssetRegistryImpl::TryConstructGathererIfNeeded()
{
	if (GlobalGatherer.IsValid())
	{
		return true;
	}
	else if (IsEngineExitRequested())
	{
		return false;
	}

	GlobalGatherer = MakeUnique<FAssetDataGatherer>(UAssetRegistryImpl::Get());
	GatherDependencyDataScratchPad.Reset(new UE::AssetRegistry::DependsNode::FGatherDependencyDataScratchpad());

	UpdateMaxSecondsPerFrame();

	// Read script packages if all initial plugins have been loaded, otherwise do nothing; we wait for the callback.
	ELoadingPhase::Type LoadingPhase = IPluginManager::Get().GetLastCompletedLoadingPhase();
	if (LoadingPhase != ELoadingPhase::None && LoadingPhase >= ELoadingPhase::PostEngineInit)
	{
		ReadScriptPackages();
	}
	return true;
}

void FAssetRegistryImpl::SearchAllAssetsInitialAsync(Impl::FEventContext& EventContext,
	Impl::FClassInheritanceContext& InheritanceContext)
{
	SetPerformanceMode(Impl::EPerformanceMode::BulkLoading);
	SearchAllAssets(EventContext, InheritanceContext, false /* bSynchronousSearch */);
}

void FAssetRegistryImpl::SetPerformanceMode(Impl::EPerformanceMode NewMode)
{
	if (PerformanceMode != NewMode)
	{
		const bool bWereDependenciesSorted = ShouldSortDependencies();
		const bool bWereReferencersSorted = ShouldSortReferencers();

		PerformanceMode = NewMode;

		const bool bShouldSortDependencies = ShouldSortDependencies();
		const bool bShouldSortReferencers = ShouldSortReferencers();

		if ((bWereDependenciesSorted != bShouldSortDependencies) || (bWereReferencersSorted != bShouldSortReferencers))
		{
			State.SetDependencyNodeSorting(bShouldSortDependencies, bShouldSortReferencers);
		}		
	}
}

bool FAssetRegistryImpl::ShouldSortDependencies() const
{
	// Always sort in static, sometimes sort during loading
	return (PerformanceMode == Impl::MostlyStatic || (PerformanceMode == Impl::BulkLoading && !Impl::bDeferDependencySort));
}

bool FAssetRegistryImpl::ShouldSortReferencers() const
{
	// Always sort in static, sometimes sort during loading
	return (PerformanceMode == Impl::MostlyStatic || (PerformanceMode == Impl::BulkLoading && !Impl::bDeferReferencerSort));
}

}

void UAssetRegistryImpl::SearchAllAssets(bool bSynchronousSearch)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("UAssetRegistryImpl::SearchAllAssets");
	using namespace UE::AssetRegistry::Impl;

	if (bSynchronousSearch)
	{
		// Ensure any ongoing async scan finishes fully first
		WaitForCompletion();
	}

	FEventContext EventContext;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		FClassInheritanceContext InheritanceContext;
		FClassInheritanceBuffer InheritanceBuffer;
		GetInheritanceContextWithRequiredLock(InterfaceScopeLock, InheritanceContext, InheritanceBuffer);
		if (bSynchronousSearch)
		{
			// make sure any outstanding async preload is complete
			GuardedData.ConditionalLoadPremadeAssetRegistry(InterfaceScopeLock, EventContext);
		}
		GuardedData.SearchAllAssets(EventContext, InheritanceContext, bSynchronousSearch);
	}
	Broadcast(EventContext);

	if (bSynchronousSearch)
	{
		// Continue calling TickGatherer until completion is signaled, and call ProcessLoadedAssetsToUpdateCache
		WaitForCompletion();
	}
}

bool UAssetRegistryImpl::IsSearchAllAssets() const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	return GuardedData.IsSearchAllAssets();
}

bool UAssetRegistryImpl::IsSearchAsync() const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	return GuardedData.IsInitialSearchStarted();
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::SearchAllAssets(Impl::FEventContext& EventContext,
	Impl::FClassInheritanceContext& InheritanceContext, bool bSynchronousSearch)
{
	EventContext.bScanStartedEventBroadcast = true;

	if (!TryConstructGathererIfNeeded())
	{
		return;
	}

	if (!bInitialSearchStarted)
	{
		CSV_TRACE_REGION_BEGIN(TEXT("Scan"), TEXT("AssetRegistry"));
		InitialSearchStartTime = FPlatformTime::Seconds();
		bInitialSearchStarted = true;
		bInitialSearchCompleted.store(false, std::memory_order_relaxed);
#if WITH_EDITOR
		bGatherNeedsApplyObjectRedirects = true;
#endif
		UpdateMaxSecondsPerFrame(); // MaxSecondsPerFrame only depends on initial search
	}

	FAssetDataGatherer& Gatherer = *GlobalGatherer;
	if (!Gatherer.IsAsyncEnabled())
	{
		UE_CLOGF(!bSynchronousSearch, LogAssetRegistry, Warning, "SearchAllAssets: Gatherer is in synchronous mode; forcing bSynchronousSearch=true.");
		bSynchronousSearch = true;
	}

	// Add all existing mountpoints to the GlobalGatherer
	// This will include Engine content, Game content, but also may include mounted content directories for one or more plugins.
	TArray<FString> PackagePathsToSearch;
	FPackageName::QueryRootContentPaths(PackagePathsToSearch);
	for (const FString& PackagePath : PackagePathsToSearch)
	{
		const FString& MountLocalPath = FPackageName::LongPackageNameToFilename(PackagePath);
		Gatherer.AddMountPoint(MountLocalPath, PackagePath);
		Gatherer.SetIsOnAllowList(MountLocalPath, true);
	}
	bSearchAllAssets = true; // Mark that future mounts and directories should be scanned

	if (bSynchronousSearch)
	{
		Gatherer.WaitForIdle();
		Impl::FTickContext TickContext(*this, EventContext, InheritanceContext);
		TickContext.bHandleDeferred = true;
		TickContext.bHandleCompletion = false; // Our caller will call WaitForCompletion which will handle this
		Impl::EGatherStatus UnusedStatus = TickGatherer(TickContext);
	}
	else
	{
		Gatherer.StartAsync();
	}
}

}

void UAssetRegistryImpl::WaitForCompletion()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAssetRegistryImpl::WaitForCompletion);

	using namespace UE::AssetRegistry::Impl;

	bool bInitialSearchStarted = false;
	bool bInitialSearchCompleted = false;
	bool bAsyncGathering = false;

	// Try taking over the gather thread for a short time in case it is mostly done.
	// But if it has more than a small amount of work to do, let the gather thread do that work
	// while we consume the results in parallel.
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		// We don't need to take the GatheredDataProcessingLock here because we actually *do*
		// want to block until we can proceed
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		FClassInheritanceContext InheritanceContext;
		FClassInheritanceBuffer InheritanceBuffer;
		GetInheritanceContextWithRequiredLock(InterfaceScopeLock, InheritanceContext, InheritanceBuffer);
		constexpr float TimeToJoinSeconds = 0.100f;
		GuardedData.WaitForGathererIdle(TimeToJoinSeconds);
		bInitialSearchStarted = GuardedData.IsInitialSearchStarted();
		bInitialSearchCompleted = GuardedData.IsInitialSearchCompleted();
		bAsyncGathering = GuardedData.GlobalGatherer && GuardedData.GlobalGatherer->IsAsyncEnabled();
	}

#if WITH_EDITOR
	if (IsGathering())
	{
		// If we do need to wait, then tick the DirectoryWatcher so we have the most up to date information.
		// This is also important because we ignore rescan events from the directory watcher if they are sent
		// during startup, so if there is a rescan event pending we want to trigger it now and ignore it
		if (UE::AssetRegistry::Impl::IsDirectoryWatcherEnabled())
		{
			FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
			DirectoryWatcherModule.Get()->Tick(-1.f);
		}
	}
#endif

	bool bLocalHasSentFileLoadedEventBroadcast = bInitialSearchCompleted;
	for (;;)
	{
		FEventContext EventContext;
		EGatherStatus Status;
		{
			// Keep the LLM scope limited so it does not surround the broadcast which calls external code
			LLM_SCOPE(ELLMTag::AssetRegistry);
			UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
			FClassInheritanceContext InheritanceContext;
			FClassInheritanceBuffer InheritanceBuffer;
			GetInheritanceContextWithRequiredLock(InterfaceScopeLock, InheritanceContext, InheritanceBuffer);
			if (IsInGameThread())
			{
				// Process any deferred events. Required since deferred events would block sending the FileLoadedEvent
				FScopeLock DeferredEventsLock(&DeferredEventsCriticalSection);
				EventContext = MoveTemp(DeferredEvents);
				DeferredEvents.Clear();
			}

			GuardedData.WaitForGathererIdleIfSynchronous();

			UE::AssetRegistry::Impl::FTickContext TickContext(GuardedData, EventContext, InheritanceContext);
			TickContext.bHandleCompletion = true;
			TickContext.bHandleDeferred = true;
			Status = GuardedData.TickGatherer(TickContext);
		}
#if WITH_EDITOR
		UE::AssetRegistry::Impl::FInterruptionContext InterruptionContext;
		ProcessLoadedAssetsToUpdateCache(EventContext, Status, InterruptionContext);
#endif
		Broadcast(EventContext, true /* bAllowFileLoadedEvent */);
		bLocalHasSentFileLoadedEventBroadcast |= EventContext.bHasSentFileLoadedEventBroadcast;
		if (!IsTickActive(Status) && Status != EGatherStatus::WaitingForEvents)
		{
			if (Status == EGatherStatus::UnableToProgress)
			{
				UE_LOGF(LogAssetRegistry, Display,
					"UAssetRegistryImpl::WaitForCompletion exiting without completing because TickGatherer returned UnableToProgress. "
					"IsInGameThread() == %ls; IsEngineStartupModuleLoadingComplete() == %ls",
					IsInGameThread() ? TEXT("TRUE") : TEXT("FALSE"),
					IsEngineStartupModuleLoadingComplete() ? TEXT("TRUE") : TEXT("FALSE"));
			}
			else if (Status == EGatherStatus::Complete && bInitialSearchStarted)
			{
				// We only perform this validation if we are in a context where we expect the initial search to occur at all
				// In some commandlets, e.g., we do not expect to run the initial search at all
				UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
				if (!GuardedData.IsInitialSearchCompleted())
				{
					UE_LOGF(LogAssetRegistry, Error,
						"Exiting from UAssetRegistryImpl::WaitForCompletion but IsInitialSearchCompleted is still false."
							"EventContext.bHasSentFileLoadedEventBroadcast == %ls; IsInGameThread() == %ls",
						EventContext.bHasSentFileLoadedEventBroadcast ? TEXT("TRUE") : TEXT("FALSE"),
						IsInGameThread() ? TEXT("TRUE") : TEXT("FALSE"));
				}
				else 
				{
					// If we are the main thread and we are exiting this function, one of two things should be true:
					// a) The search was completed before we enter this function (i.e., bInitialSearchCompleted == true); or
					// b) The search has completed during this function and, as the game thread, we have broadcast the FileLoadedEvent 
					//    (i.e., EventContext.bHasSentFileLoadedEventBroadcast == true)
					// Otherwise, something has gone wrong
					ensureMsgf(bLocalHasSentFileLoadedEventBroadcast || bInitialSearchCompleted || !IsInGameThread(),
						TEXT("Exiting from UAssetRegistryImpl::WaitForCompletion in an inconsistent state. "
							 "bLocalHasSentFileLoadedEventBroadcast == %s; EventContext.bHasSentFileLoadedEventBroadcast == %s; bInitialSearchCompleted == %s; IsInGameThread() == %s"),
						bLocalHasSentFileLoadedEventBroadcast ? TEXT("TRUE") : TEXT("FALSE"),
						EventContext.bHasSentFileLoadedEventBroadcast ? TEXT("TRUE") : TEXT("FALSE"),
						bInitialSearchCompleted ? TEXT("TRUE") : TEXT("FALSE"),
						IsInGameThread() ? TEXT("TRUE") : TEXT("FALSE"));
				}
			}
			break;
		}
		else if (Status == EGatherStatus::WaitingForEvents)
		{
			// If we are able to Broadcast events, then we will be able to do the broadcast and move past this status
			// to completion. But if we cannot broadcast events, then we cannot complete the gather and have to exit.
			// We only guarantee completion of the gather when our caller is in the right state to be able to broadcast.
			// Some callers might not expect this, so issue a warning explaining the problem.
			if (!CanBroadcastEvents())
			{
				const TCHAR* Reason = nullptr;
				if (!IsInGameThread())
				{
					Reason = TEXT("caller is !IsInGameThread()");
				}
				else if (FUObjectThreadContext::Get().IsRoutingPostLoad)
				{
					Reason = TEXT("caller is IsRoutingPostLoad");
				}
				else
				{
					Reason = TEXT("<unknownreason>");
				}

				UE_LOGF(LogAssetRegistry, Warning,
					"Exiting from UAssetRegistryImpl::WaitForCompletion with gather almost completed " "but missing the final broadcast of events, so IsInitialSearchCompleted is still false. " "We cannot execute the broadcast because %ls.", Reason);
				break;
			} 
		}

		FThreadHeartBeat::Get().HeartBeat();
		if (Status == EGatherStatus::TickActiveGatherActive && bAsyncGathering)
		{
			// Sleep long enough to avoid causing contention on the CriticalSection in GetAndTrimSearchResults
			constexpr float SleepTimeSeconds = 0.010f;
			FPlatformProcess::SleepNoStats(SleepTimeSeconds);
		}
	}
}

void UAssetRegistryImpl::WaitForPremadeAssetRegistry()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAssetRegistryImpl::WaitForPremadeAssetRegistry);
	using namespace UE::AssetRegistry::Impl;

	FEventContext EventContext;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		FClassInheritanceContext InheritanceContext;
		FClassInheritanceBuffer InheritanceBuffer;
		GetInheritanceContextWithRequiredLock(InterfaceScopeLock, InheritanceContext, InheritanceBuffer);
		GuardedData.ConditionalLoadPremadeAssetRegistry(InterfaceScopeLock, EventContext);
	}
	Broadcast(EventContext);
}

void UAssetRegistryImpl::ClearGathererCache()
{
	LLM_SCOPE(ELLMTag::AssetRegistry);
	UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
	GuardedData.ClearGathererCache();
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::ClearGathererCache()
{
	if (GlobalGatherer)
	{
		GlobalGatherer->ClearCache();
	}
}

}

void UAssetRegistryImpl::WaitForPackage(const FString& PackageName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAssetRegistryImpl::WaitForPackage);

	if (!GuardedData.IsGathering())
	{
		// IsGathering uses relaxed memory order, so it is possible that another thread has just finished
		// marking IsGathering=false and has not finished writing the data it gathered. But that's not
		// a problem, because to read that data, the caller will need to enter the critical section which was
		// held by the other thread that wrote the data, and entering a critical section waits on a memory fence.
		return;
	}
	FString LocalPath;
	if (!FPackageName::TryConvertLongPackageNameToFilename(PackageName, LocalPath))
	{
		return;
	}

	UE::AssetRegistry::Impl::FEventContext EventContext;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		GuardedData.TickGatherPackage(InterfaceScopeLock, EventContext, PackageName, LocalPath);
	}
	Broadcast(EventContext);
}

bool UAssetRegistryImpl::HasAssets(const FName PackagePath, const bool bRecursive) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	return GuardedData.HasAssets(PackagePath, bRecursive);
}

namespace UE::AssetRegistry
{

bool FAssetRegistryImpl::HasAssets(const FName PackagePath, const bool bRecursive) const
{
	bool bHasAssets = State.HasAssets(PackagePath, true /*bARFiltering*/);

	if (!bHasAssets && bRecursive)
	{
		CachedPathTree.EnumerateSubPaths(PackagePath, [this, &bHasAssets](FName SubPath)
		{
			bHasAssets = State.HasAssets(SubPath, true /*bARFiltering*/);
			return !bHasAssets;
		});
	}

	return bHasAssets;
}

}

bool UAssetRegistryImpl::GetAssetsByPackageName(FName PackageName, TArray<FAssetData>& OutAssetData, bool bIncludeOnlyOnDiskAssets, bool bSkipARFilteredAssets) const
{
	FARFilter Filter;
	Filter.PackageNames.Add(PackageName);
	Filter.bIncludeOnlyOnDiskAssets = bIncludeOnlyOnDiskAssets;
	return GetAssets(Filter, OutAssetData, bSkipARFilteredAssets);
}

bool UAssetRegistryImpl::GetAssetsByPath(FName PackagePath, TArray<FAssetData>& OutAssetData, bool bRecursive, bool bIncludeOnlyOnDiskAssets) const
{
	FARFilter Filter;
	Filter.bRecursivePaths = bRecursive;
	Filter.PackagePaths.Add(PackagePath);
	Filter.bIncludeOnlyOnDiskAssets = bIncludeOnlyOnDiskAssets;
	return GetAssets(Filter, OutAssetData);
}

bool UAssetRegistryImpl::GetAssetsByPaths(TArray<FName> PackagePaths, TArray<FAssetData>& OutAssetData, bool bRecursive, bool bIncludeOnlyOnDiskAssets) const
{
	FARFilter Filter;
	Filter.bRecursivePaths = bRecursive;
	Filter.PackagePaths = MoveTemp(PackagePaths);
	Filter.bIncludeOnlyOnDiskAssets = bIncludeOnlyOnDiskAssets;
	return GetAssets(Filter, OutAssetData);
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::EnumerateAssetsByPathNoTags(FName PackagePath,
	TFunctionRef<bool(const FAssetData&)> Callback, bool bRecursive, bool bIncludeOnlyOnDiskAssets) const
{
	if (PackagePath.IsNone())
	{
		return;
	}
	FARFilter Filter;
	Filter.bRecursivePaths = bRecursive;
	Filter.PackagePaths.Add(PackagePath);
	Filter.bIncludeOnlyOnDiskAssets = bIncludeOnlyOnDiskAssets;

	// CompileFilter takes an inheritance context, but only to handle filters with recursive classes, which we are not using here
	UE::AssetRegistry::Impl::FClassInheritanceContext EmptyInheritanceContext;
	FARCompiledFilter CompiledFilter;
	CompileFilter(EmptyInheritanceContext, Filter, CompiledFilter);

	TSet<FName> PackagesToSkip;
	if (!bIncludeOnlyOnDiskAssets)
	{
		bool bStopIteration;
		Utils::EnumerateMemoryAssetsHelper(CompiledFilter, PackagesToSkip, bStopIteration,
			[&Callback](const UObject* Object, FAssetData&& PartialAssetData)
			{
				return Callback(PartialAssetData);
			}, true /* bSkipARFilteredAssets */);
		if (bStopIteration)
		{
			return;
		}
	}
	EnumerateDiskAssets(CompiledFilter, PackagesToSkip, Callback, UE::AssetRegistry::EEnumerateAssetsFlags::None);
}

}

static FTopLevelAssetPath TryConvertShortTypeNameToPathName(FName ClassName)
{
	FTopLevelAssetPath ClassPathName;
	if (ClassName != NAME_None)
	{
		FString ShortClassName = ClassName.ToString();
		ClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(*ShortClassName, ELogVerbosity::Warning, TEXT("AssetRegistry using deprecated function"));
		UE_CLOGF(ClassPathName.IsNull(), LogClass, Error, "Failed to convert short class name %ls to class path name.", *ShortClassName);
	}
	return ClassPathName;
}

bool UAssetRegistryImpl::GetAssetsByClass(FTopLevelAssetPath ClassPathName, TArray<FAssetData>& OutAssetData, bool bSearchSubClasses) const
{
	FARFilter Filter;
	Filter.ClassPaths.Add(ClassPathName);
	Filter.bRecursiveClasses = bSearchSubClasses;
	return GetAssets(Filter, OutAssetData);
}

bool UAssetRegistryImpl::GetAssetsByTags(const TArray<FName>& AssetTags, TArray<FAssetData>& OutAssetData) const
{
	FARFilter Filter;
	for (const FName& AssetTag : AssetTags)
	{
		Filter.TagsAndValues.Add(AssetTag);
	}
	return GetAssets(Filter, OutAssetData);
}

bool UAssetRegistryImpl::GetAssetsByTagValues(const TMultiMap<FName, FString>& AssetTagsAndValues, TArray<FAssetData>& OutAssetData) const
{
	FARFilter Filter;
	for (const auto& AssetTagsAndValue : AssetTagsAndValues)
	{
		Filter.TagsAndValues.Add(AssetTagsAndValue.Key, AssetTagsAndValue.Value);
	}
	return GetAssets(Filter, OutAssetData);
}

bool UAssetRegistryImpl::GetAssets(const FARFilter& InFilter, TArray<FAssetData>& OutAssetData,
	bool bSkipARFilteredAssets) const
{
	using namespace UE::AssetRegistry::Utils;

	FARCompiledFilter CompiledFilter;
	CompileFilter(InFilter, CompiledFilter);
	if (CompiledFilter.IsEmpty() || !IsFilterValid(CompiledFilter))
	{
		return false;
	}
	return GetAssets(CompiledFilter, OutAssetData, bSkipARFilteredAssets);
}

bool UAssetRegistryImpl::GetAssets(const FARCompiledFilter& CompiledFilter, TArray<FAssetData>& OutAssetData,
	bool bSkipARFilteredAssets) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAssetRegistryImpl::GetAssets);
	using namespace UE::AssetRegistry::Utils;

	const UE::AssetRegistry::EEnumerateAssetsFlags Flags = bSkipARFilteredAssets ? UE::AssetRegistry::EEnumerateAssetsFlags::None : UE::AssetRegistry::EEnumerateAssetsFlags::AllowUnfilteredArAssets;
	EnumerateAssetsEvent.Broadcast(CompiledFilter, Flags);

	TSet<FName> PackagesToSkip;
	if (!CompiledFilter.bIncludeOnlyOnDiskAssets)
	{
		bool bStopIterationUnused;
		EnumerateMemoryAssets(CompiledFilter, PackagesToSkip, bStopIterationUnused,
			InterfaceLock, GuardedData.GetState(),
			[&OutAssetData](FAssetData&& AssetData)
			{
				OutAssetData.Add(MoveTemp(AssetData));
				return true;
			}, bSkipARFilteredAssets);
	}

	{
		UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
		GuardedData.EnumerateDiskAssets(CompiledFilter, PackagesToSkip, [&OutAssetData](const FAssetData& AssetData)
			{
				OutAssetData.Emplace(AssetData);
				return true;
			}, Flags);
	}
	return true;
}

bool UAssetRegistryImpl::GetInMemoryAssets(const FARFilter& InFilter, TArray<FAssetData>& OutAssetData, bool bSkipARFilteredAssets) const 
{
	using namespace UE::AssetRegistry::Utils;

	FARCompiledFilter CompiledFilter;
	CompileFilter(InFilter, CompiledFilter);
	if (CompiledFilter.IsEmpty() || !IsFilterValid(CompiledFilter))
	{
		return false;
	}
	return GetInMemoryAssets(CompiledFilter, OutAssetData, bSkipARFilteredAssets);
}

bool UAssetRegistryImpl::GetInMemoryAssets(const FARCompiledFilter& CompiledFilter, TArray<FAssetData>& OutAssetData, bool bSkipARFilteredAssets) const 
{
	using namespace UE::AssetRegistry::Utils;

	TSet<FName> PackagesToSkipUnused;
	bool bStopIterationUnused;
	EnumerateMemoryAssets(CompiledFilter, PackagesToSkipUnused, bStopIterationUnused,
		InterfaceLock, GuardedData.GetState(),
		[&OutAssetData](FAssetData&& AssetData)
		{
			OutAssetData.Add(MoveTemp(AssetData));
			return true;
		}, bSkipARFilteredAssets);
	return true;
}

bool UAssetRegistryImpl::EnumerateAssets(const FARFilter& InFilter, TFunctionRef<bool(const FAssetData&)> Callback,
	UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags) const
{
	FARCompiledFilter CompiledFilter;
	CompileFilter(InFilter, CompiledFilter);
	return EnumerateAssets(CompiledFilter, Callback, InEnumerateFlags);
}

bool UAssetRegistryImpl::EnumerateAssets(const FARCompiledFilter& InFilter, TFunctionRef<bool(const FAssetData&)> Callback,
	UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags) const
{
	using namespace UE::AssetRegistry::Utils;

	// Verify filter input. If all assets are needed, use EnumerateAllAssets() instead.
	if (InFilter.IsEmpty() || !IsFilterValid(InFilter))
	{
		return false;
	}

	EnumerateAssetsEvent.Broadcast(InFilter, InEnumerateFlags);

	TSet<FName> PackagesToSkip;
	if (!InFilter.bIncludeOnlyOnDiskAssets)
	{
		bool bStopIteration;
		EnumerateMemoryAssets(InFilter, PackagesToSkip, bStopIteration,
			InterfaceLock, GuardedData.GetState(),
			[&Callback](FAssetData&& AssetData)
			{
				return Callback(AssetData);
			}, !EnumHasAnyFlags(InEnumerateFlags, UE::AssetRegistry::EEnumerateAssetsFlags::AllowUnfilteredArAssets));
		if (bStopIteration)
		{
			return true;
		}
	}

	TArray<FAssetData, TInlineAllocator<128>> FoundAssets;
	{
		UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
		GuardedData.EnumerateDiskAssets(InFilter, PackagesToSkip, [&FoundAssets](const FAssetData& AssetData)
			{
				FoundAssets.Emplace(AssetData);
				return true;
			}, InEnumerateFlags);
	}
	for (const FAssetData& AssetData : FoundAssets)
	{
		if (!Callback(AssetData))
		{
			break;
		}
	}
	return true;
}

namespace UE::AssetRegistry
{

namespace Utils
{

TOptional<FAssetDataTagMap> AddNonOverlappingTags(FAssetData& ExistingAssetData, const FAssetData& NewAssetData)
{
	TOptional<FAssetDataTagMap> ModifiedTags;
	NewAssetData.TagsAndValues.ForEach([&ExistingAssetData, &ModifiedTags](const TPair<FName, FAssetTagValueRef>& TagPair)
		{
			if (ModifiedTags)
			{
				if (!ModifiedTags->Contains(TagPair.Key))
				{
					ModifiedTags->Add(TagPair.Key, TagPair.Value.GetStorageString());
				}
			}
			else
			{
				if (!ExistingAssetData.TagsAndValues.Contains(TagPair.Key))
				{
					ModifiedTags.Emplace(ExistingAssetData.TagsAndValues.CopyMap());
					ModifiedTags->Add(TagPair.Key, TagPair.Value.GetStorageString());
				}
			}
		});
	return ModifiedTags;
}

void EnumerateMemoryAssetsHelper(const FARCompiledFilter& InFilter, TSet<FName>& OutPackageNamesWithAssets,
	bool& bOutStopIteration, TFunctionRef<bool(const UObject* Object, FAssetData&& PartialAssetData)> Callback,
	bool bSkipARFilteredAssets)
{
	checkf(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread(), TEXT("Enumerating in-memory assets can only be done on the game thread or in the loader, there are too many GetAssetRegistryTags() still not thread-safe."));
	bOutStopIteration = false;
	UE_TRACK_REFERENCING_OPNAME_SCOPED(PackageAccessTrackingOps::NAME_ResetContext);

	// Skip assets that were loaded for diffing
	const uint32 FilterWithoutPackageFlags = InFilter.WithoutPackageFlags | PKG_ForDiffing;
	const uint32 FilterWithPackageFlags = InFilter.WithPackageFlags;

	struct FFilterData
	{
		const UObject* Object;
		const UPackage* Package;
		FString PackageNameStr;
		FSoftObjectPath ObjectPath;
	};

	/**
	 * The portions of the filter that are safe to execute even in the UObject global hash lock in FThreadSafeObjectIterator
	 * Returns true if the object passes the filter and should be copied into an array for calling the rest of the filter
	 * outside the lock.
	 */
	auto PassesLockSafeFilter =
		[&InFilter, bSkipARFilteredAssets, FilterWithoutPackageFlags, FilterWithPackageFlags]
	(const UObject* Obj, FFilterData& FilterData)
	{
		if (!Obj->IsAsset())
		{
			return false;
		}

		// Skip assets that are currently loading
		if (Obj->HasAnyFlags(RF_NeedLoad))
		{
			return false;
		}

		check(!Obj->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor));
		check(!Obj->GetOutermostObject()->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor));

		FilterData.Package = Obj->GetOutermost();

		// Skip assets with any of the specified 'without' package flags 
		if (FilterData.Package->HasAnyPackageFlags(FilterWithoutPackageFlags))
		{
			return false;
		}

		// Skip assets without any the specified 'with' packages flags
		if (!FilterData.Package->HasAllPackagesFlags(FilterWithPackageFlags))
		{
			return false;
		}

		// Skip classes that report themselves as assets but that the editor AssetRegistry is currently not counting as assets
		if (bSkipARFilteredAssets && UE::AssetRegistry::FFiltering::ShouldSkipAsset(Obj))
		{
			return false;
		}

		// Package name
		const FName PackageName = FilterData.Package->GetFName();

		if (InFilter.PackageNames.Num() && !InFilter.PackageNames.Contains(PackageName))
		{
			return false;
		}

		// Asset Path
		FilterData.ObjectPath = FSoftObjectPath::ConstructFromObject(Obj);
		if (InFilter.SoftObjectPaths.Num() > 0)
		{
			if (!InFilter.SoftObjectPaths.Contains(FilterData.ObjectPath))
			{
				return false;
			}
		}

		// Package path
		PackageName.ToString(FilterData.PackageNameStr);
		if (InFilter.PackagePaths.Num() > 0)
		{
			const FName PackagePath = FName(*FPackageName::GetLongPackagePath(FilterData.PackageNameStr));
			if (!InFilter.PackagePaths.Contains(PackagePath))
			{
				return false;
			}
		}

		FilterData.Object = Obj;
		return true;
	};

	auto RunUnsafeFilterAndCallback =
		[&Callback, &OutPackageNamesWithAssets]
	(FFilterData& FilterData, bool& bOutContinue)
	{
		// We mark the package found for this passing asset, so that any followup search for assets on disk will not
		// add a duplicate of this Asset. We do this here for convenience; it would be more correct to call it only for assets that
		// pass the callers remaining filters inside of Callback
		OutPackageNamesWithAssets.Add(FilterData.Package->GetFName());

		// Could perhaps save some FName -> String conversions by creating this a bit earlier using the UObject constructor
		// to get package name and path.
		FAssetData PartialAssetData(MoveTemp(FilterData.PackageNameStr), FilterData.ObjectPath.ToString(),
			FilterData.Object->GetClass()->GetClassPathName(), FAssetDataTagMap(),
			FilterData.Package->GetChunkIDs(), FilterData.Package->GetPackageFlags());

		// All filters passed, except for AssetRegistry filter; caller must check that one
		bOutContinue = Callback(FilterData.Object, MoveTemp(PartialAssetData));
	};

	// Iterate over all in-memory assets to find the ones that pass the filter components
	if (InFilter.ClassPaths.Num() > 0 || InFilter.PackageNames.Num() > 0)
	{
		TArray<UObject*, TInlineAllocator<10>> InMemoryObjects;
		if (InFilter.ClassPaths.Num())
		{
			for (FTopLevelAssetPath ClassName : InFilter.ClassPaths)
			{
				UClass* Class = FindObject<UClass>(ClassName);
				if (Class != nullptr)
				{
					ForEachObjectOfClass(Class, [&InMemoryObjects](UObject* Object)
						{
							InMemoryObjects.Add(Object);
						}, false /* bIncludeDerivedClasses */, RF_NoFlags);
				}
			}
		}
		else
		{
			for (FName PackageName : InFilter.PackageNames)
			{
				UPackage* Package = FindObjectFast<UPackage>(nullptr, PackageName);
				if (Package != nullptr)
				{
					// Store objects in an intermediate rather than calling FilterInMemoryObjectLambda on them directly
					// because the callback is arbitrary code and might create UObjects, which is disallowed in
					// ForEachObjectWithPackage
					ForEachObjectWithPackage(Package, [&InMemoryObjects](UObject* Object)
						{
							// Avoid adding an element to InMemoryObjects for every UObject
							// There could be many UObjects (thousands) but only a single Asset
							if (Object->IsAsset())
							{
								InMemoryObjects.Add(Object);
							}
							return true;
						});
				}
			}
		}

		FFilterData ScratchFilterData;
		for (const UObject* Object : InMemoryObjects)
		{
			if (PassesLockSafeFilter(Object, ScratchFilterData))
			{
				bool bContinue = true;
				RunUnsafeFilterAndCallback(ScratchFilterData, bContinue);
				if (!bContinue)
				{
					bOutStopIteration = true;
					return;
				}
			}
		}
	}
	else
	{
		TArray<FFilterData> FirstPassFilterResults;
		FFilterData ScratchFilterData;
		for (FThreadSafeObjectIterator ObjIt; ObjIt; ++ObjIt)
		{
			if (PassesLockSafeFilter(*ObjIt, ScratchFilterData))
			{
				FirstPassFilterResults.Add(MoveTemp(ScratchFilterData));
			}
		}

		for (FFilterData& FilterData : FirstPassFilterResults)
		{
			bool bContinue = true;
			RunUnsafeFilterAndCallback(FilterData, bContinue);
			if (!bContinue)
			{
				bOutStopIteration = true;
				return;
			}

			FPlatformMisc::PumpEssentialAppMessages();
		}
	}
}

void EnumerateMemoryAssets(const FARCompiledFilter& InFilter, TSet<FName>& OutPackageNamesWithAssets,
	bool& bOutStopIteration, UE::AssetRegistry::Private::FInterfaceRWLock& InterfaceLock, const FAssetRegistryState& GuardedDataState,
	TFunctionRef<bool(FAssetData&&)> Callback, bool bSkipARFilteredAssets)
{
	check(!InFilter.IsEmpty() && Utils::IsFilterValid(InFilter));

	// Avoid contending with the background thread every time we take the interface lock below.
	IAssetRegistry::FPauseBackgroundProcessingScope PauseProcessingScopeGuard;

	EnumerateMemoryAssetsHelper(InFilter, OutPackageNamesWithAssets, bOutStopIteration,
		[&InFilter, &Callback, &InterfaceLock, &GuardedDataState](const UObject* Object, FAssetData&& PartialAssetData)
		{
			FAssetRegistryTagsContextData Context(Object, EAssetRegistryTagsCaller::AssetRegistryQuery);
			Object->GetAssetRegistryTags(Context, PartialAssetData);
			{
				// GetAssetRegistryTags with EAssetRegistryTagsCaller::AssetRegistryQuery does not add some tags that
				// are too expensive to regularly compute but that exist in the on-disk Asset from SavePackage.
				// Our contract for on-disk versus in-memory tags is that in-memory tags override on-disk tags, but we
				// keep any on-disk tags that do not exist in the in-memory tags because they may be extended tags.
				UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
				const FAssetData* OnDiskAssetData = GuardedDataState.GetAssetByObjectPath(FSoftObjectPath::ConstructFromObject(Object));
				if (OnDiskAssetData)
				{
					TOptional<FAssetDataTagMap> ModifiedTags = Utils::AddNonOverlappingTags(PartialAssetData, *OnDiskAssetData);
					if (ModifiedTags)
					{
						PartialAssetData.TagsAndValues = FAssetDataTagMapSharedView(MoveTemp(*ModifiedTags));
					}
#if !WITH_EDITORONLY_DATA
					// In non-editor builds, UObject::GetChunkIds returns an empty set.
					// Like our contract for tags, when the information is missing from the UObject, our contract
					// for that information in AssetRegistry queries is that we return the on-disk version of the data.
					// The on-disk version of the data for GetChunkIds is the data that was stored in the generated
					// AssetRegistry by calling AddChunkId for each chunkID that the cooker found the Asset to be in.
					PartialAssetData.SetChunkIDs(OnDiskAssetData->GetChunkIDs());
#endif
				}
			}
			// After adding tags, PartialAssetData is now a full AssetData

			// Tags and values
			if (InFilter.TagsAndValues.Num() > 0)
			{
				bool bMatch = false;
				for (const TPair<FName, TOptional<FString>>& FilterPair : InFilter.TagsAndValues)
				{
					FAssetTagValueRef RegistryValue = PartialAssetData.TagsAndValues.FindTag(FilterPair.Key);

					if (RegistryValue.IsSet() && (!FilterPair.Value.IsSet() || RegistryValue == FilterPair.Value.GetValue()))
					{
						bMatch = true;
						break;
					}
				}

				if (!bMatch)
				{
					return true;
				}
			}

			// All filters passed
			return Callback(MoveTemp(PartialAssetData));
		}, bSkipARFilteredAssets);
}

}

void FAssetRegistryImpl::EnumerateDiskAssets(const FARCompiledFilter& InFilter, TSet<FName>& PackagesToSkip,
	TFunctionRef<bool(const FAssetData&)> Callback, UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags) const
{
	check(!InFilter.IsEmpty() && Utils::IsFilterValid(InFilter));
	PackagesToSkip.Append(CachedEmptyPackages);
	State.EnumerateAssets(InFilter, PackagesToSkip, Callback, InEnumerateFlags);
}

}

FAssetData UAssetRegistryImpl::GetAssetByObjectPath(const FSoftObjectPath& ObjectPath, bool bIncludeOnlyOnDiskAssets, bool bSkipARFilteredAssets) const
{
	if (!bIncludeOnlyOnDiskAssets)
	{
		TStringBuilder<FName::StringBufferSize> Builder;
		ObjectPath.ToString(Builder);
		UObject* Asset = FindObject<UObject>(nullptr, *Builder);

		if (Asset)
		{
			if (!bSkipARFilteredAssets || !UE::AssetRegistry::FFiltering::ShouldSkipAsset(Asset))
			{
				return FAssetData(Asset, FAssetData::ECreationFlags::None /** Do not allow blueprint classes */,
					EAssetRegistryTagsCaller::AssetRegistryQuery);
			}
			else
			{
				return FAssetData();
			}
		}
	}

	{
		UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
		const FAssetRegistryState& State = GuardedData.GetState();
		const FAssetData* FoundData = State.GetAssetByObjectPath(ObjectPath);
		return (FoundData && !State.IsPackageUnmountedAndFiltered(FoundData->PackageName)
			&& (!bSkipARFilteredAssets || !GuardedData.ShouldSkipAsset(FoundData->AssetClassPath, FoundData->PackageFlags))) ? *FoundData : FAssetData();
	}
}

FAssetData UAssetRegistryImpl::GetAssetByObjectPath(const FName ObjectPath, bool bIncludeOnlyOnDiskAssets) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	return GetAssetByObjectPath(FSoftObjectPath(ObjectPath.ToString()), bIncludeOnlyOnDiskAssets);
PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

UE::AssetRegistry::EExists UAssetRegistryImpl::TryGetAssetByObjectPath(const FSoftObjectPath& ObjectPath, FAssetData& OutAssetData) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	const bool bAssetRegistryHasFullKnowledge = GuardedData.IsSearchAllAssets() && !GuardedData.IsGathering();
	const FAssetRegistryState& State = GuardedData.GetState();
	const FAssetData* FoundData = State.GetAssetByObjectPath(ObjectPath);
	if (!FoundData)
	{
		if (!bAssetRegistryHasFullKnowledge)
		{
			return UE::AssetRegistry::EExists::Unknown;
		}
		return UE::AssetRegistry::EExists::DoesNotExist;
	}
	OutAssetData = *FoundData;
	return UE::AssetRegistry::EExists::Exists;
}

UE::AssetRegistry::EExists UAssetRegistryImpl::TryGetAssetPackageData(const FName PackageName, FAssetPackageData& OutAssetPackageData, bool bFailIfLockHeld) const
{
	FName OutCorrectCasePackageName;
	return TryGetAssetPackageData(PackageName, OutAssetPackageData, OutCorrectCasePackageName, bFailIfLockHeld);
}

UE::AssetRegistry::EExists UAssetRegistryImpl::TryGetAssetPackageData(const FName PackageName, FAssetPackageData& OutAssetPackageData, FName& OutCorrectCasePackageName, bool bFailIfLockHeld) const
{
	if (!InterfaceLock.TryReadLock())
	{
		if (bFailIfLockHeld)
		{
			return UE::AssetRegistry::EExists::Unknown;
		}
		InterfaceLock.ReadLock();
	}
	ON_SCOPE_EXIT
	{
		InterfaceLock.ReadUnlock();
	};

	const bool bLocalIsGathering = GuardedData.IsGathering();
	const bool bAssetRegistryHasFullKnowledge = GuardedData.IsSearchAllAssets() && !bLocalIsGathering;
	const FAssetRegistryState& State = GuardedData.GetState();
	const FAssetPackageData* FoundData = State.GetAssetPackageData(PackageName, OutCorrectCasePackageName);
	if (!FoundData)
	{
		if (!bAssetRegistryHasFullKnowledge)
		{
			return UE::AssetRegistry::EExists::Unknown;
		}
		return UE::AssetRegistry::EExists::DoesNotExist;
	}
	// Currently when we cook we mark FAssetPackageData as being from the IoDispatcher however that isn't true until the content
	// is staged. The correct location will be determined once the registry scan is completed, but until then we should ignore any 
	// IoDispatcher located results since they might be stale and about to be overwritten by filesystem data found during scanning.
	// Note, we also must ensure that we ignore found results before the engine startup has completed since we might not have started
	// scanning yet and thus can't trust the found results
	else if (FoundData->GetPackageLocation() == FPackageName::EPackageLocationFilter::IoDispatcher && (bLocalIsGathering || !IsEngineStartupModuleLoadingComplete()))
	{
		return UE::AssetRegistry::EExists::Unknown;
	}
	OutAssetPackageData = *FoundData;
	return UE::AssetRegistry::EExists::Exists;
}

bool UAssetRegistryImpl::GetAllAssets(TArray<FAssetData>& OutAssetData, bool bIncludeOnlyOnDiskAssets) const
{
	const double GetAllAssetsStartTime = FPlatformTime::Seconds();
	TSet<FName> PackageNamesToSkip;

	// All in memory assets
	if (!bIncludeOnlyOnDiskAssets)
	{
		bool bStopIterationUnused;
		UE::AssetRegistry::Utils::EnumerateAllMemoryAssets(PackageNamesToSkip, bStopIterationUnused,
			[&OutAssetData](FAssetData&& AssetData)
			{
				OutAssetData.Add(MoveTemp(AssetData));
				return true;
			});
	}

	{
		UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
		GuardedData.EnumerateAllDiskAssets(PackageNamesToSkip,
			[&OutAssetData](const FAssetData& AssetData)
			{
				OutAssetData.Add(AssetData);
				return true;
			});
	}

	UE_LOGF(LogAssetRegistry, VeryVerbose, "GetAllAssets completed in %0.4f seconds", FPlatformTime::Seconds() - GetAllAssetsStartTime);
	return true;
}

bool UAssetRegistryImpl::EnumerateAllAssets(TFunctionRef<bool(const FAssetData&)> Callback, UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags) const
{
	using namespace UE::AssetRegistry;
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("UAssetRegistryImpl::EnumerateAllAssets");

	TSet<FName> PackageNamesToSkip;
	// All in memory assets
	if (!EnumHasAnyFlags(InEnumerateFlags, EEnumerateAssetsFlags::OnlyOnDiskAssets))
	{
		bool bStopIteration;
		UE::AssetRegistry::Utils::EnumerateAllMemoryAssets(PackageNamesToSkip, bStopIteration,
			[&Callback](FAssetData&& AssetData)
			{
				return Callback(AssetData);
			});
		if (bStopIteration)
		{
			return true;
		}
	}

	if (EnumHasAnyFlags(InEnumerateFlags, EEnumerateAssetsFlags::Parallel))
	{
		// We have to call the callback on a copy rather than a reference since the callback may reenter the lock
		TMpscQueue<FAssetData> OnDiskAssetDatasQueue;
		TArray<FAssetData, TInlineAllocator<128>> OnDiskAssetDatas;
		{
			UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
			GuardedData.EnumerateAllDiskAssets(PackageNamesToSkip,
				[&OnDiskAssetDatasQueue](const FAssetData& AssetData)
				{
					OnDiskAssetDatasQueue.Enqueue(AssetData);
					return true;
				}, InEnumerateFlags);
			OnDiskAssetDatas.Reserve(GuardedData.State.GetNumAssets());
		}
		FAssetData AssetData;
		while (OnDiskAssetDatasQueue.Dequeue(AssetData))
		{
			OnDiskAssetDatas.Add(MoveTemp(AssetData));
		}

		ParallelFor(OnDiskAssetDatas.Num(), [&OnDiskAssetDatas, &Callback](int32 Index)
			{
				const FAssetData& AssetData = OnDiskAssetDatas[Index];
				// The Callback returns bContinueIterating, and it would be nice to have a contract that we never
				// call the Callback again after it returns bContinueIterating. But guaranteeing that contract in
				// multithreaded code is not possible without taking a lock which is prohibitively expensive. So
				// for the parallel version of EnumerateAllAssets, we require that the caller handle the Callback
				// being called again even after it returns false. Since we have that requirement, and we don't
				// expect it to be important for performance to early exit as soon as possible, we omit early exiting
				// entirely to get a performance benefit from not needing to frequently test a contended atomic.
				(void)Callback(AssetData);
			});
	}
	else
	{
		// We have to call the callback on a copy rather than a reference since the callback may reenter the lock
		TArray<FAssetData, TInlineAllocator<128>> OnDiskAssetDatas;
		{
			UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
			OnDiskAssetDatas.Reserve(GuardedData.State.GetNumAssets());
			GuardedData.EnumerateAllDiskAssets(PackageNamesToSkip,
				[&OnDiskAssetDatas](const FAssetData& AssetData)
				{
					OnDiskAssetDatas.Add(AssetData);
					return true;
				}, InEnumerateFlags);
		}

		for (const FAssetData& AssetData : OnDiskAssetDatas)
		{
			if (!Callback(AssetData))
			{
				return true;
			}
		}
	}
	return true;
}

namespace UE::AssetRegistry
{

namespace Utils
{

void EnumerateAllMemoryAssets(TSet<FName>& OutPackageNamesWithAssets, bool& bOutStopIteration,
	TFunctionRef<bool(FAssetData&&)> Callback)
{
	checkf(IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread(), TEXT("Enumerating memory assets can only be done on the game thread or in the loader, there are too many GetAssetRegistryTags() still not thread-safe."));
	bOutStopIteration = false;
	for (FThreadSafeObjectIterator ObjIt; ObjIt; ++ObjIt)
	{
		if (ObjIt->IsAsset() && !UE::AssetRegistry::FFiltering::ShouldSkipAsset(*ObjIt))
		{
			FAssetData AssetData(*ObjIt, true /* bAllowBlueprintClass */);
			OutPackageNamesWithAssets.Add(AssetData.PackageName);
			if (!Callback(MoveTemp(AssetData)))
			{
				bOutStopIteration = true;
				return;
			}
		}
	}
}

}

void FAssetRegistryImpl::EnumerateAllDiskAssets(TSet<FName>& PackageNamesToSkip, TFunctionRef<bool(const FAssetData&)> Callback, UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags) const
{
	PackageNamesToSkip.Append(CachedEmptyPackages);
	State.EnumerateAllAssets(PackageNamesToSkip, Callback, InEnumerateFlags);
}

}

void UAssetRegistryImpl::GetPackagesByName(FStringView PackageName, TArray<FName>& OutPackageNames) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	const FAssetRegistryState& State = GuardedData.GetState();
	UE_CLOGF(GuardedData.IsInitialSearchStarted() && !GuardedData.IsInitialSearchCompleted(), LogAssetRegistry, Warning,
		"GetPackagesByName has been called before AssetRegistry gather is complete and it does not wait. " "The search may return incomplete results.");
	State.GetPackagesByName(PackageName, OutPackageNames);

}

FName UAssetRegistryImpl::GetFirstPackageByName(FStringView PackageName) const
{
	FName LongPackageName;
	bool bSearchAllAssets;
	{
		UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
		const FAssetRegistryState& State = GuardedData.GetState();
		UE_CLOGF(GuardedData.IsInitialSearchStarted() && !GuardedData.IsInitialSearchCompleted(), LogAssetRegistry, Warning,
			"GetFirstPackageByName has been called before AssetRegistry gather is complete and it does not wait. " "The search may fail to find the package.");
		LongPackageName = State.GetFirstPackageByName(PackageName);
		bSearchAllAssets = GuardedData.IsSearchAllAssets();
	}
#if WITH_EDITOR
	if (!GIsEditor && !bSearchAllAssets)
	{
		// Temporary support for -game:
		// When running editor.exe with -game, we do not have a cooked AssetRegistry and we do not scan either
		// In that case, fall back to searching on disk if the search in the AssetRegistry (as expected) fails
		// In the future we plan to avoid this situation by having -game run the scan as well
		if (LongPackageName.IsNone())
		{
			UE_LOGF(LogAssetRegistry, Warning,
				"GetFirstPackageByName is being called in `-game` to resolve partial package name. " "This may cause a slow scan on disk. " "Consider using the fully qualified package name for better performance. ");
			FString LongPackageNameString;
			if (FPackageName::SearchForPackageOnDisk(FString(PackageName), &LongPackageNameString))
			{
				LongPackageName = FName(*LongPackageNameString);
			}
		}
	}
#endif
	return LongPackageName;
}

bool UAssetRegistryImpl::GetDependencies(const FAssetIdentifier& AssetIdentifier, TArray<FAssetIdentifier>& OutDependencies, UE::AssetRegistry::EDependencyCategory Category, const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	return GuardedData.GetState().GetDependencies(AssetIdentifier, OutDependencies, Category, Flags);
}

bool UAssetRegistryImpl::GetDependencies(const FAssetIdentifier& AssetIdentifier, TArray<FAssetDependency>& OutDependencies, UE::AssetRegistry::EDependencyCategory Category, const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	return GuardedData.GetState().GetDependencies(AssetIdentifier, OutDependencies, Category, Flags);
}

static void ConvertAssetIdentifiersToPackageNames(const TArray<FAssetIdentifier>& AssetIdentifiers, TArray<FName>& OutPackageNames)
{
	// add all PackageNames :
	OutPackageNames.Reserve(OutPackageNames.Num() + AssetIdentifiers.Num());
	for (const FAssetIdentifier& AssetId : AssetIdentifiers)
	{
		if (AssetId.PackageName != NAME_None)
		{
			OutPackageNames.Add(AssetId.PackageName);
		}
	}

	// make unique ; sort in previous contents of OutPackageNames to unique against them too
	OutPackageNames.Sort( FNameFastLess() );

	int UniqueNum = Algo::Unique( OutPackageNames );
	OutPackageNames.SetNum(UniqueNum, EAllowShrinking::No);
}

bool UAssetRegistryImpl::GetDependencies(FName PackageName, TArray<FName>& OutDependencies, UE::AssetRegistry::EDependencyCategory Category, const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	TArray<FAssetIdentifier> TempDependencies;
	if (!GetDependencies(FAssetIdentifier(PackageName), TempDependencies, Category, Flags))
	{
		return false;
	}
	ConvertAssetIdentifiersToPackageNames(TempDependencies, OutDependencies);
	return true;
}

bool UAssetRegistryImpl::ContainsDependency(FName PackageName, FName QueryDependencyName,
	UE::AssetRegistry::EDependencyCategory Category, const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	return GuardedData.GetState().ContainsDependency(FAssetIdentifier(PackageName),
		FAssetIdentifier(QueryDependencyName), Category, Flags);
}

bool IAssetRegistry::K2_GetDependencies(FName PackageName, const FAssetRegistryDependencyOptions& DependencyOptions, TArray<FName>& OutDependencies) const
{
	UE::AssetRegistry::FDependencyQuery Flags;
	bool bResult = false;
	if (DependencyOptions.GetPackageQuery(Flags))
	{
		bResult = GetDependencies(PackageName, OutDependencies, UE::AssetRegistry::EDependencyCategory::Package, Flags) || bResult;
	}
	if (DependencyOptions.GetSearchableNameQuery(Flags))
	{
		bResult = GetDependencies(PackageName, OutDependencies, UE::AssetRegistry::EDependencyCategory::SearchableName, Flags) || bResult;
	}
	if (DependencyOptions.GetManageQuery(Flags))
	{
		bResult = GetDependencies(PackageName, OutDependencies, UE::AssetRegistry::EDependencyCategory::Manage, Flags) || bResult;
	}
	return bResult;
}

bool UAssetRegistryImpl::GetReferencers(const FAssetIdentifier& AssetIdentifier, TArray<FAssetIdentifier>& OutReferencers, UE::AssetRegistry::EDependencyCategory Category, const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	return GuardedData.GetState().GetReferencers(AssetIdentifier, OutReferencers, Category, Flags);
}

bool UAssetRegistryImpl::GetReferencers(const FAssetIdentifier& AssetIdentifier, TArray<FAssetDependency>& OutReferencers, UE::AssetRegistry::EDependencyCategory Category, const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	return GuardedData.GetState().GetReferencers(AssetIdentifier, OutReferencers, Category, Flags);
}

bool UAssetRegistryImpl::GetReferencers(FName PackageName, TArray<FName>& OutReferencers, UE::AssetRegistry::EDependencyCategory Category, const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	TArray<FAssetIdentifier> TempReferencers;

	if (!GetReferencers(FAssetIdentifier(PackageName), TempReferencers, Category, Flags))
	{
		return false;
	}
	ConvertAssetIdentifiersToPackageNames(TempReferencers, OutReferencers);
	return true;
}

bool IAssetRegistry::K2_GetReferencers(FName PackageName, const FAssetRegistryDependencyOptions& ReferenceOptions, TArray<FName>& OutReferencers) const
{
	UE::AssetRegistry::FDependencyQuery Flags;
	bool bResult = false;
	if (ReferenceOptions.GetPackageQuery(Flags))
	{
		bResult = GetReferencers(PackageName, OutReferencers, UE::AssetRegistry::EDependencyCategory::Package, Flags) || bResult;
	}
	if (ReferenceOptions.GetSearchableNameQuery(Flags))
	{
		bResult = GetReferencers(PackageName, OutReferencers, UE::AssetRegistry::EDependencyCategory::SearchableName, Flags) || bResult;
	}
	if (ReferenceOptions.GetManageQuery(Flags))
	{
		bResult = GetReferencers(PackageName, OutReferencers, UE::AssetRegistry::EDependencyCategory::Manage, Flags) || bResult;
	}

	return bResult;
}

TOptional<FAssetPackageData> UAssetRegistryImpl::GetAssetPackageDataCopy(FName PackageName) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	const FAssetPackageData* AssetPackageData = GuardedData.GetState().GetAssetPackageData(PackageName);
	return AssetPackageData ? *AssetPackageData : TOptional<FAssetPackageData>();
}

TArray<TOptional<FAssetPackageData>> UAssetRegistryImpl::GetAssetPackageDatasCopy(TArrayView<FName> PackageNames) const
{
	TArray<TOptional<FAssetPackageData>> OutAssetPackagesData;
	OutAssetPackagesData.Reserve(PackageNames.Num());

	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	for (FName PackageName : PackageNames)
	{
		if (const FAssetPackageData* AssetPackageData = GuardedData.GetState().GetAssetPackageData(PackageName))
		{
			OutAssetPackagesData.Emplace(*AssetPackageData);
		}
		else
		{
			OutAssetPackagesData.Add(TOptional<FAssetPackageData>());
		}
	}

	return OutAssetPackagesData;
}

void UAssetRegistryImpl::EnumerateAllPackages(TFunctionRef<void(FName PackageName, const FAssetPackageData& PackageData)> Callback, UE::AssetRegistry::EEnumeratePackagesFlags InEnumerateFlags) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	if(EnumHasAnyFlags(InEnumerateFlags, UE::AssetRegistry::EEnumeratePackagesFlags::Parallel))
	{
		const TMap<FName, const FAssetPackageData*>& DataMap = GuardedData.GetState().GetAssetPackageDataMap();
		int32 MaxId = DataMap.GetMaxIndex();
		ParallelFor(MaxId, [&DataMap, &Callback](int32 Index)
		{
			FSetElementId Id = FSetElementId::FromInteger(Index);
			if (DataMap.IsValidId(Id))
			{
				const TPair<FName, const FAssetPackageData*>& Pair = DataMap.Get(Id);
				Callback(Pair.Key, *Pair.Value);
			}
		});
	}
	else
	{
		for (const TPair<FName, const FAssetPackageData*>& Pair : GuardedData.GetState().GetAssetPackageDataMap())
		{
			Callback(Pair.Key, *Pair.Value);
		}
	}
}

bool UAssetRegistryImpl::DoesPackageExistOnDisk(FName PackageName, FString* OutCorrectCasePackageName, FString* OutExtension) const
{
	auto CalculateExtension = [](const FString& PackageNameStr, TConstArrayView<FAssetData> Assets) -> FString
	{
		FTopLevelAssetPath ClassRedirector = UE::AssetRegistry::GetClassPathObjectRedirector();
		bool bContainsMap = false;
		bool bContainsRedirector = false;
		for (const FAssetData& Asset : Assets)
		{
			bContainsMap |= ((Asset.PackageFlags & PKG_ContainsMap) != 0);
			bContainsRedirector |= (Asset.AssetClassPath == ClassRedirector);
		}
		if (!bContainsMap && bContainsRedirector)
		{
			// presence of map -> .umap
			// But we can only assume lack of map -> .uasset if we know the type of every object in the package.
			// If we don't, because there was a redirector, we have to check the package on disk

			// Note, the 'internal' version of DoesPackageExist must be used to avoid re-entering the AssetRegistry's lock resulting in deadlock
			FPackagePath PackagePath;
			if (FPackageName::InternalDoesPackageExistEx(PackageNameStr, FPackageName::EPackageLocationFilter::Any, 
				false /*bMatchCaseOnDisk*/, &PackagePath) != FPackageName::EPackageLocationFilter::None)
			{
				return FString(PackagePath.GetExtensionString(EPackageSegment::Header));
			}
		}
		return bContainsMap ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();
	};

#if WITH_EDITOR
	if (GIsEditor)
	{
		// The editor always gathers PackageAssetDatas and uses those because they exactly match files on disk, whereas AssetsByPackageName
		// includes memory-only assets that have added themselves to the AssetRegistry's State.
		FString PackageNameStr = PackageName.ToString();
		if (FPackageName::IsScriptPackage(PackageNameStr))
		{
			// Script packages are an exception; the AssetRegistry creates AssetPackageData for them but they exist only in memory
			return false;
		}

		FName CorrectCasePackageName;
		const FAssetPackageData* AssetPackageData;
		{
			UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
			AssetPackageData = GuardedData.GetState().GetAssetPackageData(PackageName, CorrectCasePackageName);
		}
		const static bool bVerifyNegativeResults = FParse::Param(FCommandLine::Get(), TEXT("AssetRegistryValidatePackageExists"));
		if (bVerifyNegativeResults && !AssetPackageData)
		{
			// Note, the 'internal' version of DoesPackageExist must be used to avoid re-entering the AssetRegistry's lock resulting in deadlock
			FPackagePath PackagePath;
			if (FPackageName::InternalDoesPackageExistEx(PackageNameStr, FPackageName::EPackageLocationFilter::Any, 
				false /*bMatchCaseOnDisk*/, &PackagePath) != FPackageName::EPackageLocationFilter::None)
			{
				UE_LOGF(LogAssetRegistry, Warning, "Package %ls exists on disk but does not exist in the AssetRegistry", *PackageNameStr);
				if (OutCorrectCasePackageName)
				{
					*OutCorrectCasePackageName = PackagePath.GetLocalFullPath();
				}
				if (OutExtension)
				{
					*OutExtension = PackagePath.GetExtensionString(EPackageSegment::Header);
				}
				return true;
			}
		}

		if (!AssetPackageData)
		{
			return false;
		}

		if (OutCorrectCasePackageName)
		{
			*OutCorrectCasePackageName = CorrectCasePackageName.ToString();
		}
		if (OutExtension)
		{
			if (AssetPackageData->Extension == EPackageExtension::Unspecified || AssetPackageData->Extension == EPackageExtension::Custom)
			{
				// Note, the 'internal' version of DoesPackageExist must be used to avoid re-entering the AssetRegistry's lock resulting in deadlock
				FPackagePath PackagePath;
				if(FPackageName::InternalDoesPackageExistEx(PackageNameStr, FPackageName::EPackageLocationFilter::Any, 
					false /* bMatchCaseOnDisk*/, &PackagePath) != FPackageName::EPackageLocationFilter::None)
				{
					*OutExtension = PackagePath.GetExtensionString(EPackageSegment::Header);
				}
				else
				{
					UE_LOGF(LogAssetRegistry, Error,
						"UAssetRegistryImpl::DoesPackageExistOnDisk failed to find the extension for %ls. The package exists in the AssetRegistry but does not exist on disk.",
						*PackageNameStr);
					TArray<FAssetData> Assets;
					GetAssetsByPackageName(PackageName, Assets, /*bIncludeOnlyDiskAssets*/ true);
					*OutExtension = CalculateExtension(PackageNameStr, Assets);
				}
			}
			else
			{
				*OutExtension = LexToString(AssetPackageData->Extension);
			}
		}
		return true;
	}
	else
#endif
	{
		// Runtime Game and Programs use GetAssetsByPackageName, which will match the files on disk since these configurations do not
		// add loaded assets to the AssetRegistryState
		TArray<FAssetData> Assets;
		GetAssetsByPackageName(PackageName, Assets, /*bIncludeOnlyDiskAssets*/ true);
		if (Assets.Num() == 0)
		{
			return false;
		}
		FString PackageNameStr = PackageName.ToString();
		if (OutCorrectCasePackageName)
		{
			// In Game does not handle matching case, but it still needs to return a value for the CorrectCase field if asked
			*OutCorrectCasePackageName = PackageNameStr;
		}
		if (OutExtension)
		{
			*OutExtension = CalculateExtension(PackageNameStr, Assets);
		}
		return true;
	}
}

FSoftObjectPath UAssetRegistryImpl::GetRedirectedObjectPath(const FSoftObjectPath& ObjectPath)
{
	// Fast path, if a full registry scan was triggered & has completed
	// In that case, we can skip further scanning while looking for a redirected path
	if (!GuardedData.IsGathering())
	{
		UE::AssetRegistry::FInterfaceReadScopeLock ReadScopeLock(InterfaceLock);
		if (GuardedData.IsSearchAllAssets())
		{
			return GuardedData.GetRedirectedObjectPath(ReadScopeLock, ObjectPath);
		}
	}

	FSoftObjectPath RedirectedObjectPath;
	UE::AssetRegistry::Impl::FEventContext EventContext;
	UE::AssetRegistry::Impl::FClassInheritanceContext InheritanceContext;
	UE::AssetRegistry::Impl::FClassInheritanceBuffer InheritanceBuffer;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock WriteScopeLock(InterfaceLock);
		GetInheritanceContextWithRequiredLock(WriteScopeLock, InheritanceContext, InheritanceBuffer);
		RedirectedObjectPath = GuardedData.GetAndScanRedirectedObjectPath(WriteScopeLock, ObjectPath, &EventContext, &InheritanceContext);
	}	
	Broadcast(EventContext);

	return RedirectedObjectPath;
}

namespace UE::AssetRegistry
{

FSoftObjectPath FAssetRegistryImpl::GetRedirectedObjectPath(FInterfaceReadScopeLock& ScopeLock, const FSoftObjectPath& ObjectPath)
{
	return GetRedirectedObjectPath(ScopeLock, ObjectPath, nullptr, nullptr);
}

FSoftObjectPath FAssetRegistryImpl::GetAndScanRedirectedObjectPath(FInterfaceWriteScopeLock& ScopeLock, const FSoftObjectPath& ObjectPath,
	UE::AssetRegistry::Impl::FEventContext* EventContext, UE::AssetRegistry::Impl::FClassInheritanceContext* InheritanceContext)
{
	return GetRedirectedObjectPath(ScopeLock, ObjectPath, EventContext, InheritanceContext);
}

template <typename TScopeLock>
FSoftObjectPath FAssetRegistryImpl::GetRedirectedObjectPath(TScopeLock& ScopeLock, const FSoftObjectPath& ObjectPath, 
	UE::AssetRegistry::Impl::FEventContext* EventContext, UE::AssetRegistry::Impl::FClassInheritanceContext* InheritanceContext)
{
	static_assert(std::is_same_v<TScopeLock, FInterfaceWriteScopeLock> || std::is_same_v<TScopeLock, FInterfaceReadScopeLock>, "Only expect scope lock types of either FInterfaceWriteScopeLock or FInterfaceReadScopeLock");
	constexpr bool bNeedsScanning = std::is_same_v<TScopeLock, FInterfaceWriteScopeLock>;
	check(!bNeedsScanning || (EventContext && InheritanceContext));

	FSoftObjectPath RedirectedPath = ObjectPath;

	// For legacy behavior, for the first object pointed to, we look up the object in memory
	// before checking the on-disk assets
	UObject* Asset = ObjectPath.ResolveObject();
	if (Asset)
	{
		RedirectedPath = FSoftObjectPath::ConstructFromObject(Asset);
		UObjectRedirector* Redirector = Cast<UObjectRedirector>(Asset);
		if (!Redirector || !Redirector->DestinationObject)
		{
			return RedirectedPath;
		}
		// For legacy behavior, for all redirects after the initial request, we only check on-disk assets
		RedirectedPath = FSoftObjectPath(Redirector->DestinationObject);
	}

	FString SubPathString;

	auto RetrieveAssetData = [&]()
	{
		const FAssetData* AssetData = State.GetAssetByObjectPath(RedirectedPath);
		if (!AssetData && RedirectedPath.IsSubobject())
		{
			// If we found no Asset because it is a subobject, then look for its toplevelobject's Asset
			SubPathString = RedirectedPath.GetSubPathString();
			RedirectedPath = FSoftObjectPath::ConstructFromAssetPath(RedirectedPath.GetAssetPath());
			AssetData = State.GetAssetByObjectPath(RedirectedPath);
		}
		return AssetData;
	};

	const FAssetData* AssetData = RetrieveAssetData();

	if constexpr (bNeedsScanning)
	{
		if (!AssetData)
		{
			UE::AssetRegistry::Impl::FScanPathContext Context(*EventContext, *InheritanceContext, {}, { RedirectedPath.ToString() },
				UE::AssetRegistry::EScanFlags::IgnoreInvalidPathWarning);
			ScanPathsSynchronous(&ScopeLock, Context);

			AssetData = RetrieveAssetData();
		}
	}

	// Most of the time this will either not be a redirector or only have one redirect, so optimize for that case
	TArray<FSoftObjectPath, TInlineAllocator<2>> SeenPaths = { RedirectedPath };

	// Need to follow chain of redirectors
	while (AssetData && AssetData->IsRedirector())
	{
		FString Dest;

		if (!AssetData->GetTagValue(UE::AssetRegistry::Impl::DestinationObjectFName, Dest))
		{
			break;
		}
		
		// The FSoftObjectPath functions handle stripping class name if necessary
		RedirectedPath = Dest;

		if (SeenPaths.Contains(RedirectedPath))
		{
			// Recursive, bail
			break;
		}

		AssetData = State.GetAssetByObjectPath(RedirectedPath);
		if constexpr (bNeedsScanning)
		{
			if (!AssetData)
			{
				UE::AssetRegistry::Impl::FScanPathContext Context(*EventContext, *InheritanceContext, {}, { RedirectedPath.ToString() });
				ScanPathsSynchronous(&ScopeLock, Context);

				AssetData = State.GetAssetByObjectPath(RedirectedPath);
			}
		}

		SeenPaths.Add(RedirectedPath);
	}

	if (!SubPathString.IsEmpty())
	{
		if (!RedirectedPath.IsSubobject())
		{
			RedirectedPath.SetSubPathString(SubPathString);
		}
		else
		{
			// A complicated case; the redirector pointed to a subobject. Append old subobject path onto the new one
			// Appending old to new will always use '.' because only the first subobject uses ':'
			RedirectedPath.SetSubPathString(RedirectedPath.GetSubPathString() + TEXT(".") + SubPathString);
		}
	}
	return RedirectedPath;
}
}

bool UAssetRegistryImpl::GetAncestorClassNames(FTopLevelAssetPath ClassName, TArray<FTopLevelAssetPath>& OutAncestorClassNames) const
{
	UE::AssetRegistry::Impl::FClassInheritanceContext InheritanceContext;
	UE::AssetRegistry::Impl::FClassInheritanceBuffer InheritanceBuffer;
	UE::AssetRegistry::FInterfaceRWScopeLock InterfaceScopeLock(InterfaceLock, SLT_ReadOnly);
	const_cast<UAssetRegistryImpl*>(this)->GetInheritanceContextWithRequiredLock(
		InterfaceScopeLock, InheritanceContext, InheritanceBuffer);
	return GuardedData.GetAncestorClassNames(InheritanceContext, ClassName, OutAncestorClassNames);
}

namespace UE::AssetRegistry
{

bool FAssetRegistryImpl::GetAncestorClassNames(Impl::FClassInheritanceContext& InheritanceContext, FTopLevelAssetPath ClassName,
	TArray<FTopLevelAssetPath>& OutAncestorClassNames) const
{
	// Assume we found the class unless there is an error
	bool bFoundClass = true;

	InheritanceContext.ConditionalUpdate();
	const TMap<FTopLevelAssetPath, FTopLevelAssetPath>& InheritanceMap = InheritanceContext.Buffer->InheritanceMap;

	// Make sure the requested class is in the inheritance map
	if (!InheritanceMap.Contains(ClassName))
	{
		bFoundClass = false;
	}
	else
	{
		// Now follow the map pairs until we cant find any more parents
		const FTopLevelAssetPath* CurrentClassName = &ClassName;
		const uint32 MaxInheritanceDepth = 65536;
		uint32 CurrentInheritanceDepth = 0;
		while (CurrentInheritanceDepth < MaxInheritanceDepth && CurrentClassName != nullptr)
		{
			CurrentClassName = InheritanceMap.Find(*CurrentClassName);

			if (CurrentClassName)
			{
				if (CurrentClassName->IsNull())
				{
					// No parent, we are at the root
					CurrentClassName = nullptr;
				}
				else
				{
					OutAncestorClassNames.Add(*CurrentClassName);
				}
			}
			CurrentInheritanceDepth++;
		}

		if (CurrentInheritanceDepth == MaxInheritanceDepth)
		{
			UE_LOGF(LogAssetRegistry, Error, "IsChildClass exceeded max inheritance depth. There is probably an infinite loop of parent classes.");
			bFoundClass = false;
		}
	}

	return bFoundClass;
}
}

void UAssetRegistryImpl::GetDerivedClassNames(const TArray<FTopLevelAssetPath>& ClassNames, const TSet<FTopLevelAssetPath>& ExcludedClassNames,
	TSet<FTopLevelAssetPath>& OutDerivedClassNames) const
{
	UE::AssetRegistry::Impl::FClassInheritanceContext InheritanceContext;
	UE::AssetRegistry::Impl::FClassInheritanceBuffer InheritanceBuffer;
	UE::AssetRegistry::FInterfaceRWScopeLock InterfaceScopeLock(InterfaceLock, SLT_ReadOnly);
	const_cast<UAssetRegistryImpl*>(this)->GetInheritanceContextWithRequiredLock(
		InterfaceScopeLock, InheritanceContext, InheritanceBuffer);
	GuardedData.GetSubClasses(InheritanceContext, ClassNames, ExcludedClassNames, OutDerivedClassNames);
}

void UAssetRegistryImpl::GetAllCachedPaths(TArray<FString>& OutPathList) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	const FPathTree& CachedPathTree = GuardedData.GetCachedPathTree();
	OutPathList.Reserve(OutPathList.Num() + CachedPathTree.NumPaths());
	CachedPathTree.EnumerateAllPaths([&OutPathList](FName Path)
	{
		OutPathList.Emplace(Path.ToString());
		return true;
	});
}

void UAssetRegistryImpl::EnumerateAllCachedPaths(TFunctionRef<bool(FString)> Callback) const
{
	EnumerateAllCachedPaths([&Callback](FName Path)
	{
		return Callback(Path.ToString());
	});
}

void UAssetRegistryImpl::EnumerateAllCachedPaths(TFunctionRef<bool(FName)> Callback) const
{
	TArray<FName> FoundPaths;
	{
		UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
		const FPathTree& CachedPathTree = GuardedData.GetCachedPathTree();
		FoundPaths.Reserve(CachedPathTree.NumPaths());
		CachedPathTree.EnumerateAllPaths([&FoundPaths](FName Path)
		{
			FoundPaths.Add(Path);
			return true;
		});
	}
	for (FName Path : FoundPaths)
	{
		if (!Callback(Path))
		{
			return;
		}
	}
}

void UAssetRegistryImpl::GetSubPaths(const FString& InBasePath, TArray<FString>& OutPathList, bool bInRecurse) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	const FPathTree& CachedPathTree = GuardedData.GetCachedPathTree();
	CachedPathTree.EnumerateSubPaths(*InBasePath, [&OutPathList](FName Path)
	{
		OutPathList.Emplace(Path.ToString());
		return true;
	}, bInRecurse);
}

void UAssetRegistryImpl::GetSubPaths(const FName& InBasePath, TArray<FName>& OutPathList, bool bInRecurse) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	const FPathTree& CachedPathTree = GuardedData.GetCachedPathTree();
	CachedPathTree.EnumerateSubPaths(InBasePath, [&OutPathList](FName Path)
	{
		OutPathList.Emplace(Path);
		return true;
	}, bInRecurse);
}

void UAssetRegistryImpl::EnumerateSubPaths(const FString& InBasePath, TFunctionRef<bool(FString)> Callback, bool bInRecurse) const
{
	TArray<FName, TInlineAllocator<64>> SubPaths;
	{
		UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
		const FPathTree& CachedPathTree = GuardedData.GetCachedPathTree();
		CachedPathTree.EnumerateSubPaths(FName(*InBasePath), [&SubPaths](FName PathName)
		{
			SubPaths.Add(PathName);
			return true;
		}, bInRecurse);
	}
	for (FName PathName : SubPaths)
	{
		if (!Callback(PathName.ToString()))
		{
			break;
		}
	}
}

void UAssetRegistryImpl::EnumerateSubPaths(const FName InBasePath, TFunctionRef<bool(FName)> Callback, bool bInRecurse) const
{
	TArray<FName, TInlineAllocator<64>> SubPaths;
	{
		UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
		const FPathTree& CachedPathTree = GuardedData.GetCachedPathTree();
		CachedPathTree.EnumerateSubPaths(InBasePath, [&SubPaths](FName PathName)
		{
			SubPaths.Add(PathName);
			return true;
		}, bInRecurse);
	}
	for (FName PathName : SubPaths)
	{
		if (!Callback(PathName))
		{
			break;
		}
	}
}

void UAssetRegistryImpl::RunAssetsThroughFilter(TArray<FAssetData>& AssetDataList, const FARFilter& Filter) const
{
	if (Filter.IsEmpty())
	{
		return;
	}
	FARCompiledFilter CompiledFilter;
	CompileFilter(Filter, CompiledFilter);
	UE::AssetRegistry::Utils::RunAssetsThroughFilter(AssetDataList, CompiledFilter, UE::AssetRegistry::Utils::EFilterMode::Inclusive);
}

void UAssetRegistryImpl::UseFilterToExcludeAssets(TArray<FAssetData>& AssetDataList, const FARFilter& Filter) const
{
	if (Filter.IsEmpty())
	{
		return;
	}
	FARCompiledFilter CompiledFilter;
	CompileFilter(Filter, CompiledFilter);
	UseFilterToExcludeAssets(AssetDataList, CompiledFilter);
}

void UAssetRegistryImpl::UseFilterToExcludeAssets(TArray<FAssetData>& AssetDataList, const FARCompiledFilter& CompiledFilter) const
{
	UE::AssetRegistry::Utils::RunAssetsThroughFilter(AssetDataList, CompiledFilter, UE::AssetRegistry::Utils::EFilterMode::Exclusive);
}

bool UAssetRegistryImpl::IsAssetIncludedByFilter(const FAssetData& AssetData, const FARCompiledFilter& Filter) const
{
	return UE::AssetRegistry::Utils::RunAssetThroughFilter(AssetData, Filter, UE::AssetRegistry::Utils::EFilterMode::Inclusive);
}

bool UAssetRegistryImpl::IsAssetExcludedByFilter(const FAssetData& AssetData, const FARCompiledFilter& Filter) const
{
	return UE::AssetRegistry::Utils::RunAssetThroughFilter(AssetData, Filter, UE::AssetRegistry::Utils::EFilterMode::Exclusive);
}

namespace UE::AssetRegistry::Utils
{

bool RunAssetThroughFilter(const FAssetData& AssetData, const FARCompiledFilter& Filter, const EFilterMode FilterMode)
{
	const bool bPassFilterValue = FilterMode == EFilterMode::Inclusive;
	if (Filter.IsEmpty())
	{
		return bPassFilterValue;
	}

	const bool bFilterResult = RunAssetThroughFilter_Unchecked(AssetData, Filter, bPassFilterValue);
	return bFilterResult == bPassFilterValue;
}

bool RunAssetThroughFilter_Unchecked(const FAssetData& AssetData, const FARCompiledFilter& Filter, const bool bPassFilterValue)
{
	// Package Names
	if (Filter.PackageNames.Num() > 0)
	{
		const bool bPassesPackageNames = Filter.PackageNames.Contains(AssetData.PackageName);
		if (bPassesPackageNames != bPassFilterValue)
		{
			return !bPassFilterValue;
		}
	}

	// Package Paths
	if (Filter.PackagePaths.Num() > 0)
	{
		const bool bPassesPackagePaths = Filter.PackagePaths.Contains(AssetData.PackagePath);
		if (bPassesPackagePaths != bPassFilterValue)
		{
			return !bPassFilterValue;
		}
	}

	// ObjectPaths
	if (Filter.SoftObjectPaths.Num() > 0)
	{
		const bool bPassesObjectPaths = Filter.SoftObjectPaths.Contains(AssetData.GetSoftObjectPath());
		if (bPassesObjectPaths != bPassFilterValue)
		{
			return !bPassFilterValue;
		}
	}

	// Classes
	if (Filter.ClassPaths.Num() > 0)
	{
		const bool bPassesClasses = Filter.ClassPaths.Contains(AssetData.AssetClassPath);
		if (bPassesClasses != bPassFilterValue)
		{
			return !bPassFilterValue;
		}
	}

	// Tags and values
	if (Filter.TagsAndValues.Num() > 0)
	{
		bool bPassesTags = false;
		for (const auto& TagsAndValuePair : Filter.TagsAndValues)
		{
			bPassesTags |= TagsAndValuePair.Value.IsSet()
				? AssetData.TagsAndValues.ContainsKeyValue(TagsAndValuePair.Key, TagsAndValuePair.Value.GetValue())
				: AssetData.TagsAndValues.Contains(TagsAndValuePair.Key);
			if (bPassesTags)
			{
				break;
			}
		}
		if (bPassesTags != bPassFilterValue)
		{
			return !bPassFilterValue;
		}
	}

	return bPassFilterValue;
}

void RunAssetsThroughFilter(TArray<FAssetData>& AssetDataList, const FARCompiledFilter& CompiledFilter, const EFilterMode FilterMode)
{
	if (!IsFilterValid(CompiledFilter))
	{
		return;
	}

	const int32 OriginalArrayCount = AssetDataList.Num();
	const bool bPassFilterValue = FilterMode == EFilterMode::Inclusive;

	AssetDataList.RemoveAll([&CompiledFilter, bPassFilterValue](const FAssetData& AssetData)
		{
			const bool bFilterResult = RunAssetThroughFilter_Unchecked(AssetData, CompiledFilter, bPassFilterValue);
			return bFilterResult != bPassFilterValue;
		});
	if (OriginalArrayCount > AssetDataList.Num())
	{
		AssetDataList.Shrink();
	}
}

}

void UAssetRegistryImpl::CompileFilter(const FARFilter& InFilter, FARCompiledFilter& OutCompiledFilter) const
{
	UE::AssetRegistry::Impl::FClassInheritanceContext InheritanceContext;
	UE::AssetRegistry::Impl::FClassInheritanceBuffer InheritanceBuffer;
	UE::AssetRegistry::FInterfaceRWScopeLock InterfaceScopeLock(InterfaceLock, SLT_ReadOnly);
	if (InFilter.bRecursiveClasses)
	{
		const_cast<UAssetRegistryImpl*>(this)->GetInheritanceContextWithRequiredLock(
			InterfaceScopeLock, InheritanceContext, InheritanceBuffer);
	}
	else
	{
		// CompileFilter takes an inheritance context, but only to handle filters with recursive classes
		// which we are not using here, so leave the InheritanceContext empty
	}
	GuardedData.CompileFilter(InheritanceContext, InFilter, OutCompiledFilter);
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::CompileFilter(Impl::FClassInheritanceContext& InheritanceContext, const FARFilter& InFilter,
	FARCompiledFilter& OutCompiledFilter) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAssetRegistryImpl::CompileFilter);

	OutCompiledFilter.Clear();
	OutCompiledFilter.PackageNames.Append(InFilter.PackageNames);
	OutCompiledFilter.PackagePaths.Reserve(InFilter.PackagePaths.Num());
	for (FName PackagePath : InFilter.PackagePaths)
	{
		OutCompiledFilter.PackagePaths.Add(FPathTree::NormalizePackagePath(PackagePath));
	}
	OutCompiledFilter.SoftObjectPaths.Append(InFilter.SoftObjectPaths);

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	OutCompiledFilter.SoftObjectPaths.Append(UE::SoftObjectPath::Private::ConvertObjectPathNames(InFilter.ObjectPaths));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!ensureAlwaysMsgf(InFilter.ClassNames.Num() == 0, TEXT("Asset Registry Filter using ClassNames instead of ClassPaths. First class name: \"%s\""), *InFilter.ClassNames[0].ToString()))
	{
		OutCompiledFilter.ClassPaths.Reserve(InFilter.ClassNames.Num());
		for (FName ClassName : InFilter.ClassNames)
		{
			if (!ClassName.IsNone())
			{
				FTopLevelAssetPath ClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(ClassName.ToString(), ELogVerbosity::Warning, TEXT("Compiling Asset Registry Filter"));
				if (!ClassPathName.IsNull())
				{
					OutCompiledFilter.ClassPaths.Add(ClassPathName);
				}
				else
				{
					UE_LOGF(LogAssetRegistry, Error, "Failed to resolve class path for short class name \"%ls\" when compiling asset registry filter", *ClassName.ToString());
				}
			}
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	OutCompiledFilter.ClassPaths.Append(InFilter.ClassPaths);
	OutCompiledFilter.TagsAndValues = InFilter.TagsAndValues;
	OutCompiledFilter.bIncludeOnlyOnDiskAssets = InFilter.bIncludeOnlyOnDiskAssets;
	OutCompiledFilter.WithoutPackageFlags = InFilter.WithoutPackageFlags;
	OutCompiledFilter.WithPackageFlags = InFilter.WithPackageFlags;

	if (InFilter.bRecursivePaths)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAssetRegistryImpl::CompileFilter::AddPaths);
		
		// Add the sub-paths of all the input paths to the expanded list
		for (const FName& PackagePath : InFilter.PackagePaths)
		{
			CachedPathTree.GetSubPaths(PackagePath, OutCompiledFilter.PackagePaths);
		}
	}

	if (InFilter.bRecursiveClasses)
	{
		// Add the sub-classes of all the input classes to the expanded list, excluding any that were requested
		if (InFilter.RecursiveClassPathsExclusionSet.Num() > 0 && InFilter.ClassPaths.Num() == 0)
		{
			TArray<FTopLevelAssetPath> ClassNamesObject;
			ClassNamesObject.Add(UE::AssetRegistry::GetClassPathObject());

			GetSubClasses(InheritanceContext, ClassNamesObject, InFilter.RecursiveClassPathsExclusionSet, OutCompiledFilter.ClassPaths);
		}
		else
		{
			GetSubClasses(InheritanceContext, InFilter.ClassPaths, InFilter.RecursiveClassPathsExclusionSet, OutCompiledFilter.ClassPaths);
		}
	}
}

}

EAssetAvailability::Type UAssetRegistryImpl::GetAssetAvailability(const FAssetData& AssetData) const
{
	return UE::AssetRegistry::Utils::GetAssetAvailability(AssetData);
}

namespace UE::AssetRegistry::Utils
{

EAssetAvailability::Type GetAssetAvailability(const FAssetData& AssetData)
{
#if ENABLE_PLATFORM_CHUNK_INSTALL
	IPlatformChunkInstall* ChunkInstall = FPlatformMisc::GetPlatformChunkInstall();

	EChunkLocation::Type BestLocation = EChunkLocation::DoesNotExist;

	// check all chunks to see which has the best locality
	for (int32 PakchunkId : AssetData.GetChunkIDs())
	{
		EChunkLocation::Type ChunkLocation = ChunkInstall->GetPakchunkLocation(PakchunkId);

		// if we find one in the best location, early out
		if (ChunkLocation == EChunkLocation::BestLocation)
		{
			BestLocation = ChunkLocation;
			break;
		}

		if (ChunkLocation > BestLocation)
		{
			BestLocation = ChunkLocation;
		}
	}

	switch (BestLocation)
	{
	case EChunkLocation::LocalFast:
		return EAssetAvailability::LocalFast;
	case EChunkLocation::LocalSlow:
		return EAssetAvailability::LocalSlow;
	case EChunkLocation::NotAvailable:
		return EAssetAvailability::NotAvailable;
	case EChunkLocation::DoesNotExist:
		return EAssetAvailability::DoesNotExist;
	default:
		check(0);
		return EAssetAvailability::LocalFast;
	}
#else
	return EAssetAvailability::LocalFast;
#endif
}

}

float UAssetRegistryImpl::GetAssetAvailabilityProgress(const FAssetData& AssetData, EAssetAvailabilityProgressReportingType::Type ReportType) const
{
	return UE::AssetRegistry::Utils::GetAssetAvailabilityProgress(AssetData, ReportType);
}

namespace UE::AssetRegistry::Utils
{

float GetAssetAvailabilityProgress(const FAssetData& AssetData, EAssetAvailabilityProgressReportingType::Type ReportType)
{
	check(ReportType == EAssetAvailabilityProgressReportingType::PercentageComplete || ReportType == EAssetAvailabilityProgressReportingType::ETA);

#if ENABLE_PLATFORM_CHUNK_INSTALL
	IPlatformChunkInstall* ChunkInstall = FPlatformMisc::GetPlatformChunkInstall();
	EChunkProgressReportingType::Type ChunkReportType = GetChunkAvailabilityProgressType(ReportType);

	bool IsPercentageComplete = (ChunkReportType == EChunkProgressReportingType::PercentageComplete) ? true : false;

	float BestProgress = UE_MAX_FLT;

	// check all chunks to see which has the best time remaining
	for (int32 PakchunkID : AssetData.GetChunkIDs())
	{
		float Progress = ChunkInstall->GetChunkProgress(PakchunkID, ChunkReportType);

		// need to flip percentage completes for the comparison
		if (IsPercentageComplete)
		{
			Progress = 100.0f - Progress;
		}

		if (Progress <= 0.0f)
		{
			BestProgress = 0.0f;
			break;
		}

		if (Progress < BestProgress)
		{
			BestProgress = Progress;
		}
	}

	// unflip percentage completes
	if (IsPercentageComplete)
	{
		BestProgress = 100.0f - BestProgress;
	}
	return BestProgress;

#else
	if (ReportType == EAssetAvailabilityProgressReportingType::PercentageComplete)
	{
		return 100.0f;
	}
	else
	{
		return 0.0f;
	}
#endif

}

}

bool UAssetRegistryImpl::GetAssetAvailabilityProgressTypeSupported(EAssetAvailabilityProgressReportingType::Type ReportType) const
{
	return UE::AssetRegistry::Utils::GetAssetAvailabilityProgressTypeSupported(ReportType);
}

namespace UE::AssetRegistry::Utils
{

bool GetAssetAvailabilityProgressTypeSupported(EAssetAvailabilityProgressReportingType::Type ReportType)
{
#if ENABLE_PLATFORM_CHUNK_INSTALL
	IPlatformChunkInstall* ChunkInstall = FPlatformMisc::GetPlatformChunkInstall();
	return ChunkInstall->GetProgressReportingTypeSupported(GetChunkAvailabilityProgressType(ReportType));
#else
	return true;
#endif
}

}

void UAssetRegistryImpl::PrioritizeAssetInstall(const FAssetData& AssetData) const
{
	UE::AssetRegistry::Utils::PrioritizeAssetInstall(AssetData);
}

namespace UE::AssetRegistry::Utils
{

void PrioritizeAssetInstall(const FAssetData& AssetData)
{
#if ENABLE_PLATFORM_CHUNK_INSTALL
	IPlatformChunkInstall* ChunkInstall = FPlatformMisc::GetPlatformChunkInstall();

	const TConstArrayView<int32> ChunkIDs = AssetData.GetChunkIDs();
	if (ChunkIDs.Num() == 0)
	{
		return;
	}

	ChunkInstall->PrioritizePakchunk(ChunkIDs[0], EChunkPriority::Immediate);
#endif
}

}

bool UAssetRegistryImpl::HasVerseFiles(FName PackagePath, bool bRecursive /*= false*/) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	return GuardedData.GetVerseFilesByPath(PackagePath, /*OutFilePaths=*/nullptr, bRecursive);
}

bool UAssetRegistryImpl::GetVerseFilesByPath(FName PackagePath, TArray<FName>& OutFilePaths, bool bRecursive /*= false*/) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	return GuardedData.GetVerseFilesByPath(PackagePath, &OutFilePaths, bRecursive);
}

namespace UE::AssetRegistry
{

bool FAssetRegistryImpl::GetVerseFilesByPath(FName PackagePath, TArray<FName>* OutFilePaths, bool bRecursive) const
{
	TSet<FName> PathList;
	PathList.Reserve(32);
	PathList.Add(PackagePath);
	if (bRecursive)
	{
		CachedPathTree.GetSubPaths(PackagePath, PathList, true);
	}

	bool bFoundAnything = false;
	for (const FName& PathName : PathList)
	{
		const TArray<FName>* FilePaths = CachedVerseFilesByPath.Find(PathName);
		if (FilePaths)
		{
			bFoundAnything = true;
			if (OutFilePaths)
			{
				OutFilePaths->Append(*FilePaths);
			}
			else
			{
				break;
			}
		}
	}
	return bFoundAnything;
}

}

bool UAssetRegistryImpl::AddPath(const FString& PathToAdd)
{
	UE::AssetRegistry::Impl::FEventContext EventContext;
	bool bResult;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		bResult = GuardedData.AddPath(EventContext, UE::String::RemoveFromEnd(FStringView(PathToAdd), TEXTVIEW("/")));
	}
	Broadcast(EventContext);
	return bResult;
}

namespace UE::AssetRegistry
{

bool FAssetRegistryImpl::AddPath(Impl::FEventContext& EventContext, FStringView PathToAdd)
{
	bool bIsDenied = false;
	// If no GlobalGatherer, then we are in the game or non-cook commandlet and we do not implement deny listing
	if (GlobalGatherer.IsValid())
	{
		FString LocalPathToAdd;
		if (FPackageName::TryConvertLongPackageNameToFilename(PathToAdd, LocalPathToAdd))
		{
			bIsDenied = GlobalGatherer->IsOnDenyList(LocalPathToAdd);
		}
	}
	if (bIsDenied)
	{
		return false;
	}
	return AddAssetPath(EventContext, FName(PathToAdd));
}

}

bool UAssetRegistryImpl::RemovePath(const FString& PathToRemove)
{
	UE::AssetRegistry::Impl::FEventContext EventContext;
	bool bResult;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		bResult = GuardedData.RemoveAssetPath(EventContext, FName(UE::String::RemoveFromEnd(FStringView(PathToRemove), TEXTVIEW("/"))));
	}
	Broadcast(EventContext);
	return bResult;
}

bool UAssetRegistryImpl::PathExists(const FString& PathToTest) const
{
	return PathExists(FName(*PathToTest));
}

bool UAssetRegistryImpl::PathExists(const FName PathToTest) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	return GuardedData.GetCachedPathTree().PathExists(PathToTest);
}

void UAssetRegistryImpl::ScanPathsSynchronous(const TArray<FString>& InPaths, bool bForceRescan,
	bool bIgnoreDenyListScanFilters)
{
	// The contract of this older version of ScanSynchronous always set the WaitForInMemoryObjects flag.
	UE::AssetRegistry::EScanFlags ScanFlags = UE::AssetRegistry::EScanFlags::WaitForInMemoryObjects;

	if (bForceRescan)
	{
		ScanFlags |= UE::AssetRegistry::EScanFlags::ForceRescan;
	}

	if (bIgnoreDenyListScanFilters)
	{
		ScanFlags |= UE::AssetRegistry::EScanFlags::IgnoreDenyListScanFilters;
	}

	ScanPathsSynchronousInternal(InPaths, TArray<FString>(), ScanFlags);
}

void UAssetRegistryImpl::ScanFilesSynchronous(const TArray<FString>& InFilePaths, bool bForceRescan)
{
	// The contract of this older version of ScanSynchronous always set the WaitForInMemoryObjects flag.
	UE::AssetRegistry::EScanFlags ScanFlags = UE::AssetRegistry::EScanFlags::WaitForInMemoryObjects;

	if (bForceRescan)
	{
		ScanFlags |= UE::AssetRegistry::EScanFlags::ForceRescan;
	}

	ScanPathsSynchronousInternal(TArray<FString>(), InFilePaths, ScanFlags);
}

void UAssetRegistryImpl::ScanSynchronous(const TArray<FString>& InPaths, const TArray<FString>& InFilePaths, UE::AssetRegistry::EScanFlags InScanFlags)
{
	ScanPathsSynchronousInternal(InPaths, InFilePaths, InScanFlags);
}

void UAssetRegistryImpl::ScanPathsSynchronousInternal(const TArray<FString>& InDirs, const TArray<FString>& InFiles,
	UE::AssetRegistry::EScanFlags InScanFlags)
{
	UE_SCOPED_IO_ACTIVITY(*WriteToString<256>("Scan Paths"));

	TRACE_CPUPROFILER_EVENT_SCOPE(UAssetRegistryImpl::ScanPathsSynchronousInternal);
	UE_TRACK_REFERENCING_OPNAME_SCOPED(PackageAccessTrackingOps::NAME_ResetContext);
	const double SearchStartTime = FPlatformTime::Seconds();

	const bool bWaitForInMemoryObjects = !!(InScanFlags & UE::AssetRegistry::EScanFlags::WaitForInMemoryObjects);

	UE::AssetRegistry::Impl::FEventContext EventContext;
	UE::AssetRegistry::Impl::FClassInheritanceContext InheritanceContext;
	UE::AssetRegistry::Impl::FClassInheritanceBuffer InheritanceBuffer;
	UE::AssetRegistry::Impl::FScanPathContext Context(EventContext, InheritanceContext, InDirs, InFiles,
		InScanFlags, nullptr /* OutFindAssets */);

	bool bInitialSearchStarted;
	bool bInitialSearchCompleted;
	bool bAdditionalMountSearchInProgress;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		GetInheritanceContextWithRequiredLock(InterfaceScopeLock, InheritanceContext, InheritanceBuffer);

		bInitialSearchStarted = GuardedData.IsInitialSearchStarted();
		bInitialSearchCompleted = GuardedData.IsInitialSearchCompleted();
		bAdditionalMountSearchInProgress = GuardedData.IsAdditionalMountSearchInProgress();
		// make sure any outstanding async preload is complete
		GuardedData.ConditionalLoadPremadeAssetRegistry(InterfaceScopeLock, EventContext);
		GuardedData.ScanPathsSynchronous(&InterfaceScopeLock, Context);
	}
	if (Context.LocalPaths.IsEmpty())
	{
		return;
	}

#if WITH_EDITOR
	if (bWaitForInMemoryObjects)
	{
		UE::AssetRegistry::Impl::FInterruptionContext InterruptionContext;
		ProcessLoadedAssetsToUpdateCache(EventContext, Context.Status, InterruptionContext);
	}
#endif
	Broadcast(EventContext);

	// Log stats
	FString PathsString;
	if (Context.LocalPaths.Num() > 1)
	{
		PathsString = FString::Printf(TEXT("'%s' and %d other paths"), *Context.LocalPaths[0], Context.LocalPaths.Num() - 1);
	}
	else
	{
		PathsString = FString::Printf(TEXT("'%s'"), *Context.LocalPaths[0]);
	}


	double Duration = FPlatformTime::Seconds() - SearchStartTime;
	UE::Telemetry::AssetRegistry::FSynchronousScanTelemetry Telemetry; 
	Telemetry.Directories = MakeArrayView(InDirs);
	Telemetry.Files = MakeArrayView(InFiles);
	Telemetry.Flags = InScanFlags;
	Telemetry.NumFoundAssets = Context.NumFoundAssets;
	Telemetry.Duration = Duration;
	Telemetry.bInitialSearchStarted = bInitialSearchStarted;
	Telemetry.bInitialSearchCompleted = bInitialSearchCompleted;
	Telemetry.bAdditionalMountSearchInProgress = bAdditionalMountSearchInProgress;
	FTelemetryRouter::Get().ProvideTelemetry(Telemetry);
	UE_LOGF(LogAssetRegistry, Verbose, "ScanPathsSynchronous completed scanning %ls to find %d assets in %0.4f seconds", *PathsString,
		Context.NumFoundAssets, Duration);
}

void UAssetRegistryImpl::PrioritizeSearchPath(const FString& PathToPrioritize)
{
	LLM_SCOPE(ELLMTag::AssetRegistry);
	UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
	GuardedData.PrioritizeSearchPath(PathToPrioritize);
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::PrioritizeSearchPath(const FString& PathToPrioritize)
{
	if (!GlobalGatherer.IsValid())
	{
		return;
	}
	GlobalGatherer->PrioritizeSearchPath(PathToPrioritize);
}

}

void UAssetRegistryImpl::AssetCreated(UObject* NewAsset)
{
	if (ensure(NewAsset) && NewAsset->IsAsset())
	{
		// Add the newly created object to the package file cache because its filename can already be
		// determined by its long package name.
		// @todo AssetRegistry We are assuming it will be saved in a single asset package.
		UPackage* NewPackage = NewAsset->GetOutermost();

		// Mark this package as newly created.
		NewPackage->SetPackageFlags(PKG_NewlyCreated);

		const FString NewPackageName = NewPackage->GetName();

		bool bShouldSkipAssset;
		UE::AssetRegistry::Impl::FEventContext EventContext;
		{
			LLM_SCOPE(ELLMTag::AssetRegistry);
			UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
			// If this package was marked as an empty package before, it is no longer empty, so remove it from the list
			GuardedData.RemoveEmptyPackage(NewPackage->GetFName());

			// Add the path to the Path Tree, in case it wasn't already there
			GuardedData.AddAssetPath(EventContext, *FPackageName::GetLongPackagePath(NewPackageName));
			bShouldSkipAssset = GuardedData.ShouldSkipAsset(NewAsset);
		}

		Broadcast(EventContext);
		if (!bShouldSkipAssset)
		{
			checkf(IsInGameThread(), TEXT("AssetCreated is not yet implemented as callable from other threads"));
			// Let subscribers know that the new asset was added to the registry
			FAssetData AssetData = FAssetData(NewAsset, FAssetData::ECreationFlags::AllowBlueprintClass,
				EAssetRegistryTagsCaller::AssetRegistryQuery);
			AssetAddedEvent.Broadcast(AssetData);
			OnAssetsAdded().Broadcast({ AssetData });

			// Notify listeners that an asset was just created
			InMemoryAssetCreatedEvent.Broadcast(NewAsset);
		}
	}
}

void UAssetRegistryImpl::AssetDeleted(UObject* DeletedAsset)
{
	checkf(GIsEditor, TEXT("Updating the AssetRegistry is only available in editor"));
	if (!ensure(DeletedAsset) || !DeletedAsset->IsAsset())
	{
		return;
	}

	UPackage* DeletedObjectPackage = DeletedAsset->GetOutermost();
	bool bIsEmptyPackage = DeletedObjectPackage != nullptr && UPackage::IsEmptyPackage(DeletedObjectPackage, DeletedAsset);

	bool bShouldSkipAsset;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);

		// Deleting the last asset in a package causes the package to be garbage collected.
		// If the UPackage object is GCed, it will be considered 'Unloaded' which will cause it to
		// be fully loaded from disk when save is invoked.
		// We want to keep the package around so we can save it empty or delete the file.
		if (bIsEmptyPackage)
		{
			GuardedData.AddEmptyPackage(DeletedObjectPackage->GetFName());
		}
		bShouldSkipAsset = GuardedData.ShouldSkipAsset(DeletedAsset);
	}

#if WITH_EDITOR
	// Update Redirectors
	if (FAssetData::IsRedirector(DeletedAsset))
	{
		// Need to remove from GRedirectCollector
		GRedirectCollector.RemoveAssetPathRedirection(FSoftObjectPath::ConstructFromObject(DeletedAsset));
	}
#endif

	if (!bShouldSkipAsset)
	{
		FAssetData AssetDataDeleted = FAssetData(DeletedAsset, FAssetData::ECreationFlags::AllowBlueprintClass,
			EAssetRegistryTagsCaller::AssetRegistryQuery);

		checkf(IsInGameThread(), TEXT("AssetDeleted is not yet implemented as callable from other threads"));
		// Let subscribers know that the asset was removed from the registry
		AssetRemovedEvent.Broadcast(AssetDataDeleted);
		OnAssetsRemoved().Broadcast({AssetDataDeleted});

		// Notify listeners that an in-memory asset was just deleted
		InMemoryAssetDeletedEvent.Broadcast(DeletedAsset);
	}
}

void UAssetRegistryImpl::AssetRenamed(const UObject* RenamedAsset, const FString& OldObjectPath)
{
	checkf(GIsEditor, TEXT("Updating the AssetRegistry is only available in editor"));
	if (ensure(RenamedAsset) && RenamedAsset->IsAsset())
	{
		// Add the renamed object to the package file cache because its filename can already be
		// determined by its long package name.
		// @todo AssetRegistry We are assuming it will be saved in a single asset package.
		UPackage* NewPackage = RenamedAsset->GetOutermost();
		const FString NewPackageName = NewPackage->GetName();
		const FString Filename = FPackageName::LongPackageNameToFilename(NewPackageName, FPackageName::GetAssetPackageExtension());

		// We want to keep track of empty packages so we can properly merge cached assets with in-memory assets
		UPackage* OldPackage = nullptr;
		FString OldPackageName;
		FString OldAssetName;
		if (OldObjectPath.Split(TEXT("."), &OldPackageName, &OldAssetName))
		{
			OldPackage = FindPackage(nullptr, *OldPackageName);
		}

		// Call IsEmptyPackage outside of the lock; it can call LoadPackage internally.
		bool bOldPackageIsEmpty = OldPackage && UPackage::IsEmptyPackage(OldPackage);

		bool bShouldSkipAsset;
		UE::AssetRegistry::Impl::FEventContext EventContext;
		{
			LLM_SCOPE(ELLMTag::AssetRegistry);
			UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
			GuardedData.RemoveEmptyPackage(NewPackage->GetFName());

			if (bOldPackageIsEmpty)
			{
				GuardedData.AddEmptyPackage(OldPackage->GetFName());
			}

			// Add the path to the Path Tree, in case it wasn't already there
			GuardedData.AddAssetPath(EventContext, *FPackageName::GetLongPackagePath(NewPackageName));
			bShouldSkipAsset = GuardedData.ShouldSkipAsset(RenamedAsset);
		}

		Broadcast(EventContext);
		if (!bShouldSkipAsset)
		{
			checkf(IsInGameThread(), TEXT("AssetRenamed is not yet implemented as callable from other threads"));
			AssetRenamedEvent.Broadcast(
				FAssetData(RenamedAsset, FAssetData::ECreationFlags::AllowBlueprintClass,
					EAssetRegistryTagsCaller::AssetRegistryQuery),
				OldObjectPath
			);
		}
	}
}

void UAssetRegistryImpl::AssetsSaved(TArray<FAssetData>&& Assets)
{
#if WITH_EDITOR
	UE::AssetRegistry::Impl::FEventContext EventContext;
	{
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		GuardedData.AssetsSaved(EventContext, MoveTemp(Assets));
	}
	Broadcast(EventContext);
#endif
}

void UAssetRegistryImpl::AssetUpdateTags(UObject* Object, EAssetRegistryTagsCaller Caller)
{
#if WITH_EDITOR
	FAssetData AssetData(Object, FAssetData::ECreationFlags::None, Caller);
	TArray<FAssetData> Assets;
	Assets.Add(MoveTemp(AssetData));

	UE::AssetRegistry::Impl::FEventContext EventContext;
	{
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		GuardedData.AssetsSaved(EventContext, MoveTemp(Assets));
	}
	Broadcast(EventContext);
#endif
}

namespace UE::AssetRegistry
{

#if WITH_EDITOR
void FAssetRegistryImpl::AssetsSaved(UE::AssetRegistry::Impl::FEventContext& EventContext, TArray<FAssetData>&& Assets)
{
	LLM_SCOPE(ELLMTag::AssetRegistry);
	for (FAssetData& NewAssetData : Assets)
	{
		FCachedAssetKey Key(NewAssetData);
		FAssetData* DataFromGather = State.GetMutableAssetByObjectPath(Key);

		AssetDataObjectPathsUpdatedOnLoad.Add(NewAssetData.GetSoftObjectPath());

		if (!DataFromGather)
		{
			FAssetData* ClonedAssetData = new FAssetData(MoveTemp(NewAssetData));
			AddAssetData(EventContext, ClonedAssetData);
		}
		else
		{
			UpdateAssetData(EventContext, DataFromGather, MoveTemp(NewAssetData), false /* bKeepDeletedTags */);
		}
	}
}
#endif

}

void UAssetRegistryImpl::AssetTagsFinalized(const UObject& FinalizedAsset)
{
#if WITH_EDITOR
	if (!FinalizedAsset.IsAsset())
	{
		return;
	}
	LLM_SCOPE(ELLMTag::AssetRegistry);
	AddLoadedAssetToProcess(FinalizedAsset);
#endif
}

bool UAssetRegistryImpl::VerseCreated(const FString& FilePathOnDisk)
{
	checkf(GIsEditor, TEXT("Updating the AssetRegistry is only available in editor"));
	if (!FAssetDataGatherer::IsVerseFile(FilePathOnDisk))
	{
		return false;
	}

	FString PackageName;
	if (!FPackageName::TryConvertFilenameToLongPackageName(FilePathOnDisk, PackageName, /*OutFailureReason*/nullptr, FPackageName::EConvertFlags::AllowDots))
	{
		return false;
	}

	FNameBuilder VersePackagePathName;
	VersePackagePathName.Append(PackageName);
	VersePackagePathName.Append(FPathViews::GetExtension(FilePathOnDisk, /*bIncludeDot=*/true));

	UE::AssetRegistry::Impl::FEventContext EventContext;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		GuardedData.AddVerseFile(EventContext, *VersePackagePathName);
	}
	Broadcast(EventContext);

	return true;
}

bool UAssetRegistryImpl::VerseDeleted(const FString& FilePathOnDisk)
{
	checkf(GIsEditor, TEXT("Updating the AssetRegistry is only available in editor"));
	if (!FAssetDataGatherer::IsVerseFile(FilePathOnDisk))
	{
		return false;
	}

	FString PackageName;
	if (!FPackageName::TryConvertFilenameToLongPackageName(FilePathOnDisk, PackageName, /*OutFailureReason*/nullptr, FPackageName::EConvertFlags::AllowDots))
	{
		return false;
	}

	FNameBuilder VersePackagePathName;
	VersePackagePathName.Append(PackageName);
	VersePackagePathName.Append(FPathViews::GetExtension(FilePathOnDisk, /*bIncludeDot=*/true));

	UE::AssetRegistry::Impl::FEventContext EventContext;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		GuardedData.RemoveVerseFile(EventContext, *VersePackagePathName);
	}
	Broadcast(EventContext);

	return true;
}

void UAssetRegistryImpl::PackageDeleted(UPackage* DeletedPackage)
{
	checkf(GIsEditor, TEXT("Updating the AssetRegistry is only available in editor"));
	UE::AssetRegistry::Impl::FEventContext EventContext;
	if (ensure(DeletedPackage))
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		GuardedData.RemovePackageData(EventContext, DeletedPackage->GetFName());
	}
	Broadcast(EventContext);
}

bool UAssetRegistryImpl::IsLoadingAssets() const
{
	return GuardedData.IsLoadingAssets();
}

bool UAssetRegistryImpl::IsGathering() const
{
	return GuardedData.IsGathering();
}

bool UAssetRegistryImpl::HasSerializedDiscoveryCache() const
{
	return GuardedData.HasSerializedDiscoveryCache();
}

namespace UE::AssetRegistry
{

bool FAssetRegistryImpl::IsLoadingAssets() const
{
	return !IsInitialSearchCompleted();
}

bool FAssetRegistryImpl::IsGathering() const
{
	return !IsInitialSearchCompleted() || IsAdditionalMountSearchInProgress();
}

bool FAssetRegistryImpl::HasSerializedDiscoveryCache() const
{
	return GlobalGatherer->HasSerializedDiscoveryCache();
}
}

UE::AssetRegistry::Impl::EGatherStatus UAssetRegistryImpl::TickOnBackgroundThread()
{
	UE::AssetRegistry::Impl::EGatherStatus Status = UE::AssetRegistry::Impl::EGatherStatus::TickActiveGatherActive;

	do
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		if (GatheredDataProcessingLock.TryLock())
		{
			ON_SCOPE_EXIT { GatheredDataProcessingLock.Unlock(); };
			UE::AssetRegistry::Impl::FEventContext EventContext;
			UE::AssetRegistry::Impl::FClassInheritanceContext InheritanceContext;
			UE::AssetRegistry::Impl::FClassInheritanceBuffer InheritanceBuffer;
			UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock, UE::AssetRegistry::Private::PriorityLow);
			GetInheritanceContextWithRequiredLock(InterfaceScopeLock, InheritanceContext, InheritanceBuffer);

			UE::AssetRegistry::Impl::FInterruptionContext::ShouldExitEarlyCallbackType EarlyExitHelper = 
				[this]()->bool
				{
					if (InterfaceLock.HasWaiters() || IsBackgroundProcessingPaused())
					{
#if WITH_EDITOR
						++GuardedData.GetBackgroundTickInterruptionsCount();
#endif
						return true;
					}
					return false;
				};

			UE::AssetRegistry::Impl::FTickContext TickContext(GuardedData, EventContext, InheritanceContext);
			if (UE::AssetRegistry::Impl::MaxSecondsPerTickBackgroundThread > 0.f)
			{
				TickContext.InterruptionContext.SetLimitedTickTime(FPlatformTime::Seconds(),
					UE::AssetRegistry::Impl::MaxSecondsPerTickBackgroundThread);
			}
			TickContext.InterruptionContext.SetEarlyExitCallback(EarlyExitHelper);
			TickContext.bHandleDeferred = true;
			Status = GuardedData.TickGatherer(TickContext);

			if (!EventContext.IsEmpty())
			{
				FScopeLock DeferredEventsLock(&DeferredEventsCriticalSection);
				DeferredEvents.Append(MoveTemp(EventContext));
				EventContext.Clear();
				RequestTick();
			}
		}
		else 
		{
			// If the game thread is holding the processing lock,
			// let's just exit and let the thread run function decide what to do 
			return UE::AssetRegistry::Impl::EGatherStatus::UnableToProgress;
		}

		if (IsBackgroundProcessingPaused())
		{
			return UE::AssetRegistry::Impl::EGatherStatus::UnableToProgress;
		}

		// This ensures that if there are multiple waiters we don't get in ahead of them
		while(InterfaceLock.HasWaiters())
		{
			if (IsBackgroundProcessingPaused())
			{
				return UE::AssetRegistry::Impl::EGatherStatus::UnableToProgress;
			}
			FPlatformProcess::Yield();
		}
	} while (Status == UE::AssetRegistry::Impl::EGatherStatus::TickActiveGatherIdle);

	return Status;
}

UAssetRegistryImpl::FEnumerateAssetsEvent& UAssetRegistryImpl::OnEnumerateAssetsEvent()
{
	return EnumerateAssetsEvent;
}

void UAssetRegistryImpl::Tick(float DeltaTime)
{
	checkf(IsInGameThread(), TEXT("The tick function executes deferred loads and events and must be on the game thread to do so."));
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("Asset Registry Tick");

	UE::AssetRegistry::Impl::EGatherStatus Status = UE::AssetRegistry::Impl::EGatherStatus::TickActiveGatherActive;
	double TickStartTime = -1; // Force a full flush if DeltaTime < 0
	if (DeltaTime >= 0)
	{
		TickStartTime = FPlatformTime::Seconds();
	}

	bool bInterruptedOrShouldProcessDeferredEvents = false;
	float LocalMaxSecondsPerFrame = UE::AssetRegistry::Impl::MaxSecondsPerFrame;

	// Ticks in !WITH_EDITOR are done on request. Mark that we have satisfied the request; any further requests that come in from
	// other threads while we are ticking will cause an additional queued request. RequestTick is not expected to be called from
	// this thread, because all callers of it come only when writing to DeferredEvents, which we will not do in this thread because
	// we can process the events immediately on this thread.
#if !WITH_EDITOR
	{
		FScopeLock DeferredEventsLock(&DeferredEventsCriticalSection);
		ClearRequestTick();
	}
#endif

	do
	{
		bInterruptedOrShouldProcessDeferredEvents = false;

		UE::AssetRegistry::Impl::FEventContext EventContext;

		bool bHasEnteredGatheredDataProcessingLock = false;
#if WITH_EDITOR
		bool bTakeOverGather = GuardedData.IsGameThreadTakeOverGatherEachTick();
		if (!bTakeOverGather)
#endif
		{
			// When we are not trying to block on the gather, we allow the background thread to keep working on
			// tickgatherer, and we only enter the lock and tickgatherer here on the game thread if the background
			// thread is not already in the lock
			bHasEnteredGatheredDataProcessingLock = GatheredDataProcessingLock.TryLock();
		}
#if WITH_EDITOR
		else
		{
			// When we want to block on the gather results, we take over TickGatherer from the background thread.
			// For the GlobalGatherer's side of this race, see TickOnBackgroundThread and the code that calls it in
			// FAssetDataGatherer::Run.
			{
				// First we use an FInterfaceWriteScopeLock with the default High Priority to register ourselves as
				// waiting on the InterfaceLock.
				UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
				// The GlobalGatherer will see that we are waiting on entering the lock and will exit the lock as soon
				// as possible to allow us to take it. After we take the lock, it will race with us to reenter the
				// GatheredDataProcessingLock and then enter the InterfaceLock, and will block on the InterfaceLock as
				// long as we are still holding it.
				// By requesting pause We tell the GlobalGatherer to leave the GatheredDataProcessingLock and not try
				// to reenter it until we request resume.
				GuardedData.RequestPauseBackgroundProcessing();
				// We drop the InterfaceLock to allow the globalgatherer to continue on if it is waiting on it.
			}
			// After dropping the InterfaceLock, we block on the GatheredDataProcessingLock, waiting for the
			// GlobalGatherer to notice that backgroundprocessing is paused and get out of both of the locks.
			GatheredDataProcessingLock.Lock();
			bHasEnteredGatheredDataProcessingLock = true;
			// We unpause after we finish ticking
		}
#endif

		if (bHasEnteredGatheredDataProcessingLock)
		{
			LLM_SCOPE(ELLMTag::AssetRegistry);
			UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
			UE::AssetRegistry::Impl::FClassInheritanceContext InheritanceContext;
			UE::AssetRegistry::Impl::FClassInheritanceBuffer InheritanceBuffer;
			GetInheritanceContextWithRequiredLock(InterfaceScopeLock, InheritanceContext, InheritanceBuffer);

			// Process any deferred events
			{
				FScopeLock DeferredEventsLock(&DeferredEventsCriticalSection);
				EventContext = MoveTemp(DeferredEvents);
				DeferredEvents.Clear();
			}

			if (EventContext.IsEmpty())
			{
				// Tick the Gatherer
				UE::AssetRegistry::Impl::FTickContext TickContext(GuardedData, EventContext, InheritanceContext);
				LocalMaxSecondsPerFrame = GuardedData.MaxSecondsPerFrame;
				TickContext.InterruptionContext.SetLimitedTickTime(TickStartTime, LocalMaxSecondsPerFrame);
				TickContext.bHandleCompletion = true;
				TickContext.bHandleDeferred = true;
				Status = GuardedData.TickGatherer(TickContext);
				bInterruptedOrShouldProcessDeferredEvents = TickContext.InterruptionContext.WasInterrupted();
			}
			else
			{
				// Skip the TickGather to deal with the DeferredEvents first
				bInterruptedOrShouldProcessDeferredEvents = true;
			}
			
#if WITH_EDITOR
			if (bTakeOverGather)
			{
				// As soon as we execute this unpause, the globalgatherer can race to reenter the locks
				// but it will block entering GatheredDataProcessingLock until we unlock it on the next line.
				GuardedData.RequestResumeBackgroundProcessing();
			}
#endif
			GatheredDataProcessingLock.Unlock();
		}
		else
		{
			{
				FScopeLock DeferredEventsLock(&DeferredEventsCriticalSection);
				EventContext.Append(MoveTemp(DeferredEvents));
				DeferredEvents.Clear();
			}
			GuardedData.ProcessGameThreadRequestedClasses();
		}

#if WITH_EDITOR
		if (!bInterruptedOrShouldProcessDeferredEvents)
		{
			UE::AssetRegistry::Impl::FInterruptionContext InterruptionContext;
			InterruptionContext.SetLimitedTickTime(TickStartTime, LocalMaxSecondsPerFrame);
			ProcessLoadedAssetsToUpdateCache(EventContext, Status, InterruptionContext);
			bInterruptedOrShouldProcessDeferredEvents = bInterruptedOrShouldProcessDeferredEvents
				|| InterruptionContext.WasInterrupted();
		}
#endif

		{
			TRACE_CPUPROFILER_EVENT_SCOPE_STR("Asset Registry Event Broadcast");
			Broadcast(EventContext, true /* bAllowFileLoadedEvent */);
		}
	} while ((bInterruptedOrShouldProcessDeferredEvents || Status == UE::AssetRegistry::Impl::EGatherStatus::WaitingForEvents) &&
		(TickStartTime < 0 || (FPlatformTime::Seconds() - TickStartTime) <= LocalMaxSecondsPerFrame));
}

void UAssetRegistryImpl::RequestTick()
{
	// Called from within DeferredEventsCriticalSection
	// RequestTick is not used if WITH_EDITOR; the AssetRegistry is ticked every frame from
	// UEditorEngine::Tick or a WITH_EDITOR block in UGameEngine::Tick.
#if !WITH_EDITOR
	if (TickRequestHandle.IsValid())
	{
		// Already Set
		return;
	}
	TickRequestHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this](float)
		{
			// DeltaTime: non-negative to indicate we have a time limit, as small as possible to do just the minimum.
			constexpr float DeltaTime = 0.0f;
			this->Tick(DeltaTime);
			return false; // Do not continue ticking
		}));
#endif
}

void UAssetRegistryImpl::ClearRequestTick()
{
	// Called from within DeferredEventsCriticalSection
	// RequestTick/ClearRequestTick are not used if WITH_EDITOR; the AssetRegistry is ticked every frame from
	// UEditorEngine::Tick or a WITH_EDITOR block in UGameEngine::Tick.
#if !WITH_EDITOR
	if (!TickRequestHandle.IsValid())
	{
		return;
	}
	FTSTicker::RemoveTicker(TickRequestHandle);
	TickRequestHandle.Reset();
#endif
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::WaitForGathererIdleIfSynchronous()
{
	if (GlobalGatherer && GlobalGatherer->IsSynchronous())
	{
		GlobalGatherer->WaitForIdle();
	}
}

void FAssetRegistryImpl::WaitForGathererIdle(float TimeoutSeconds)
{
	if (GlobalGatherer)
	{
		GlobalGatherer->WaitForIdle(TimeoutSeconds);
	}
}

bool FAssetRegistryImpl::ClassRequiresGameThreadProcessing(const UClass* Class) const
{
	// This function is not called. See FAssetDataGatherer::TickInternal for where
	// it would be called if it were fully implemented.
	return true;
}

void FAssetRegistryImpl::UpdateMaxSecondsPerFrame()
{
	float NewMaxSecondsPerFrame = UE::AssetRegistry::Impl::MaxSecondsPerFrame;
#if WITH_EDITOR
	bool bGatherOnGameThreadOnly = false;
	GConfig->GetBool(TEXT("AssetRegistry"), TEXT("GatherOnGameThreadOnly"), bGatherOnGameThreadOnly, GEngineIni);
	bool bLocalGameThreadTakeOverGatherEachTick = false;

	// Skip this block even if we're still gathering as part of AdditionalMountSearch; 
	// this block applies only during initial search
	if (bInitialSearchStarted && !IsInitialSearchCompleted())
	{
		bool bBlockingInitialLoad;
		GConfig->GetBool(TEXT("AssetRegistry"), TEXT("BlockingInitialLoad"), bBlockingInitialLoad, GEditorPerProjectIni);
		if (bBlockingInitialLoad)
		{
			bLocalGameThreadTakeOverGatherEachTick = true;
			NewMaxSecondsPerFrame = UE::AssetRegistry::Impl::MaxSecondsPerFrameToUseInBlockingInitialLoad;
			if (MaxSecondsPerFrame < NewMaxSecondsPerFrame)
			{
				UE_LOGF(LogAssetRegistry, Display,
					"EditorPerProjectUserSettings.ini:[AssetRegistry]:BlockingInitialLoad=true, setting AssetRegistry load to blocking. The editor will not be interactive until the initial scan completes.");
			}
		}
	}
	if (GlobalGatherer)
	{
		GlobalGatherer->SetGatherOnGameThreadOnly(bGatherOnGameThreadOnly);
	}
	SetGameThreadTakeOverGatherEachTick(bLocalGameThreadTakeOverGatherEachTick);

#endif
	MaxSecondsPerFrame = NewMaxSecondsPerFrame;
}

Impl::EGatherStatus FAssetRegistryImpl::TickGatherer(Impl::FTickContext& TickContext)
{
	using namespace UE::AssetRegistry::Impl;

	if (!GlobalGatherer.IsValid())
	{
		return EGatherStatus::Complete;
	}

	// Gather results from the background search
	GlobalGatherer->GetAndTrimSearchResults(BackgroundResults, TickContext.ResultContext);
	TickContext.SetNumGatherFromDiskPendingDirty(); // Invalidate cache if used in an earlier TickGatherer call
	if (!IsGathering() && !TickContext.ResultContext.bIsSearching && TickContext.GetNumPending() == 0)
	{
		// This is the common case of AssetRegistry ticks after the gather completed. We were already complete before
		// the tick, and found nothing new to gather. Early exit now to avoid doing spending time checking each of the
		// pieces of gathered data.
		return EGatherStatus::Complete;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE_STR_CONDITIONAL("FAssetRegistryImpl::TickGatherer", IsGathering());
	// Initialize per-tick TickContext data and alias some of its fields to reduce duplication
	FEventContext& EventContext = TickContext.EventContext;
	FInterruptionContext& InOutInterruptionContext = TickContext.InterruptionContext;
	TickContext.MountPointsForVerifyAfterGather.Reset();
	TickContext.bHadAssetsToProcess = BackgroundResults.Assets.Num() > 0 || BackgroundResults.Dependencies.Num() > 0;
	TickContext.bIsInGameThread = IsInGameThread();
	TickContext.TimingStartTime = -1.;
	ON_SCOPE_EXIT
	{
		TickContext.RecordTimer();
	};

	// We will be modifying the arrays that contribute to NumGatherFromDiskPending below, and we will need the updated
	// value after we finish them or when we early exit, so mark it dirty now.
	TickContext.SetNumGatherFromDiskPendingDirty();

	// Report the search times
	for (double SearchTime : TickContext.ResultContext.SearchTimes)
	{
		UE_LOGF(LogAssetRegistry, Verbose, "### Background search completed in %0.4f seconds", SearchTime);
	}

	// Add discovered paths
	if (BackgroundResults.Paths.Num())
	{
		TickContext.LazyStartTimer();
		PathDataGathered(EventContext, BackgroundResults.Paths, InOutInterruptionContext,
			TickContext.MountPointsForVerifyAfterGather);
	}
	if (InOutInterruptionContext.ShouldExitEarly())
	{
		return TickContext.UpdateIntermediateStatus();
	}

	ConditionalRetryGatherAssetsForGameThread(TickContext);

	bool bRetryAssetGathering = true;
	int32 OriginalDeferredAssetsCount = 0;
	int32 NumRetries = 0;
	while (bRetryAssetGathering)
	{
		bRetryAssetGathering = false;

		// Process the normal results and defer anything that isn't ready
		TickContext.RunAssetSearchDataGathered(BackgroundResults.Assets, DeferredAssets);
		if (InOutInterruptionContext.ShouldExitEarly())
		{
			return TickContext.UpdateIntermediateStatus();
		}

		if (TickContext.bIsInGameThread)
		{
			TickContext.RunAssetSearchDataGathered(BackgroundResults.AssetsForGameThread, DeferredAssetsForGameThread);
			if (InOutInterruptionContext.ShouldExitEarly())
			{
				return TickContext.UpdateIntermediateStatus();
			}

			// If we cleared out AssetsForGameThread, mark that we no longer need GameThreadRequestedClasses.
			ConditionalClearGameThreadRequestedClasses(TickContext);
		}

		TSet<FName>* PackagesNeedingDependencyCalculationPointer = nullptr;
#if WITH_EDITOR
		PackagesNeedingDependencyCalculationPointer = &PackagesNeedingDependencyCalculation;
#endif
		TickContext.RunDependencyDataGathered(BackgroundResults.Dependencies, DeferredDependencies,
			PackagesNeedingDependencyCalculationPointer);
		if (InOutInterruptionContext.ShouldExitEarly())
		{
			return TickContext.UpdateIntermediateStatus();
		}

		if (TickContext.bIsInGameThread)
		{
#if WITH_EDITOR
			PackagesNeedingDependencyCalculationPointer = &PackagesNeedingDependencyCalculationOnGameThread;
#endif
			TickContext.RunDependencyDataGathered(BackgroundResults.DependenciesForGameThread,
				DeferredDependenciesForGameThread, PackagesNeedingDependencyCalculationPointer);
			if (InOutInterruptionContext.ShouldExitEarly())
			{
				return TickContext.UpdateIntermediateStatus();
			}
		}

		// Retry deferred assets if we've finished all the other assets; we need to do this in the current tick
		// so we avoid spuriously reporting status == UnableToProgress
		if (BackgroundResults.Assets.IsEmpty()
			&& (!TickContext.bIsInGameThread || BackgroundResults.AssetsForGameThread.IsEmpty())
			&& TickContext.bHandleDeferred)
		{
			if (!DeferredAssets.IsEmpty() || !DeferredDependencies.IsEmpty() ||
				(TickContext.bIsInGameThread && (!DeferredAssetsForGameThread.IsEmpty() || !DeferredDependenciesForGameThread.IsEmpty())))
			{
				if (bProcessedAnyAssetsAfterRetryDeferred)
				{
					bRetryAssetGathering = true;
				}
				else
				{
					if (!bForceCompletionEvenIfClassNotLoaded && bPreloadingComplete && IsEngineStartupModuleLoadingComplete())
					{
						bForceCompletionEvenIfClassNotLoaded = true;
						bRetryAssetGathering = true;
					}
				}
				if (bRetryAssetGathering)
				{
					bProcessedAnyAssetsAfterRetryDeferred = false;
					if (NumRetries == 0)
					{
						OriginalDeferredAssetsCount = DeferredAssets.Num() + DeferredAssetsForGameThread.Num()
							+ 10; // fudge factor to make sure an edge case of 0 does not cause a problem
					}
					if (NumRetries++ >= OriginalDeferredAssetsCount)
					{
						UE_LOGF(LogAssetRegistry, Error, "Runaway loop detected in handling of deferred assets");
						// This will cause us to return UnableToProgress status
						break;
					}
					BackgroundResults.Assets.Append(MoveTemp(DeferredAssets));
					BackgroundResults.AssetsForGameThread.Append(MoveTemp(DeferredAssetsForGameThread));
					BackgroundResults.Dependencies.Append(MoveTemp(DeferredDependencies));
					BackgroundResults.DependenciesForGameThread.Append(MoveTemp(DeferredDependenciesForGameThread));
				}
			}
		}
	}

	// Load cooked packages that do not have asset data
	if (BackgroundResults.CookedPackageNamesWithoutAssetData.Num())
	{
		TickContext.LazyStartTimer();
		CookedPackageNamesWithoutAssetDataGathered(EventContext, BackgroundResults.CookedPackageNamesWithoutAssetData, InOutInterruptionContext);
		if (InOutInterruptionContext.ShouldExitEarly())
		{
			return TickContext.UpdateIntermediateStatus();
		}
	}

	// Add Verse files
	if (BackgroundResults.VerseFiles.Num())
	{
		TickContext.LazyStartTimer();

		VerseFilesGathered(EventContext, BackgroundResults.VerseFiles, InOutInterruptionContext);
		if (InOutInterruptionContext.ShouldExitEarly())
		{
			return TickContext.UpdateIntermediateStatus();
		}
	}

	// Store blocked files to be reported
	if (BackgroundResults.BlockedFiles.Num())
	{
		EventContext.BlockedFiles.Append(MoveTemp(BackgroundResults.BlockedFiles));
		BackgroundResults.BlockedFiles.Reset();
	}

	bool bDiskGatherComplete = !TickContext.ResultContext.bIsSearching
		&& TickContext.GetNumGatherFromDiskPending() == 0;
	if (!bDiskGatherComplete)
	{
		// We're still gathering from disk or deferred data in the arrays above, we can't run the on-disk-gather
		// complete code below so there is nothing further to do for now.
		return TickContext.UpdateIntermediateStatus();
	}

	// Execute calculations in the gather that have to happen after the gather from disk is completed; the gather is
	// not marked complete until these are done.

	if (!IsEngineStartupModuleLoadingComplete() || !bPreloadingComplete)
	{
		// We can't complete the gather until we've finished startup, because more modules might mount up to that
		// point, and when mounted add more paths that we need to gather.
		// We also can't do some of the calculations until then. LoadCalculatedDependencies has to wait because
		// modules might add new entries to RegisteredDependencyGathererClasses as they are loaded. 
		// We also cannot complete, or run calculations requiring knowledge of all assets, if we have a preloaded
		// AssetRegistry and we have not finished preloading it.

		// Since we can not progress on some of the calculations, return now.
		return TickContext.UpdateIntermediateStatus();
	}

#if WITH_EDITOR
	// Load Calculated Dependencies for types that register for them via REGISTER_ASSETDEPENDENCY_GATHERER
	if (PackagesNeedingDependencyCalculation.Num() || PackagesNeedingDependencyCalculationOnGameThread.Num())
	{
		TickContext.LazyStartTimer();
		// Only assets whose classes have a RegisteredDependencyGathererClasses entry need to run through
		// LoadCalculatedDependencies. Furthermore, we must always perform their gather on the game thread.
		// PruneAndCoalesce does the non-game thread calculations we can, including removing assets that don't need
		// any calculations, and moves the remaining issues to the container that needs action from the game thread.
		PruneAndCoalescePackagesRequiringDependencyCalculation(PackagesNeedingDependencyCalculation,
			PackagesNeedingDependencyCalculationOnGameThread, InOutInterruptionContext);
		if (InOutInterruptionContext.ShouldExitEarly())
		{
			return TickContext.UpdateIntermediateStatus();
		}
		// All remaining assets should now be in PackagesNeedingDependencyCalculationOnGameThread.
		ensure(PackagesNeedingDependencyCalculation.Num() == 0);

		if (PackagesNeedingDependencyCalculationOnGameThread.Num() && !TickContext.bIsInGameThread)
		{
			// Since we can not progress, return now.
			return TickContext.UpdateIntermediateStatus();
		}

		LoadCalculatedDependencies(nullptr, TickContext.InheritanceContext,
			&PackagesNeedingDependencyCalculationOnGameThread, InOutInterruptionContext);
		if (InOutInterruptionContext.ShouldExitEarly())
		{
			return TickContext.UpdateIntermediateStatus();
		}
	}

	if (bGatherNeedsApplyObjectRedirects)
	{
		ApplyObjectRedirectsToGatheredAssetDatas();
		if (InOutInterruptionContext.ShouldExitEarly())
		{
			return TickContext.UpdateIntermediateStatus();
		}
	}
#endif

	// Finishing the gather is only possible on the game thread (and only for callsites of TickGather that opt into it)
	// because we need to not mark the gather complete until we have broadcast the events, and broadcasting events by
	// contract is only allowed from game thread.
	if (!TickContext.bIsInGameThread || !TickContext.bHandleCompletion)
	{
		return TickContext.SetIntermediateStatus(EGatherStatus::TickGameThreadActiveGatherIdle);
	}

	if (!IsInitialSearchCompleted())
	{
		if (!EventContext.AssetEvents.IsEmpty())
		{
			// Don't mark the InitialSearch completed until we have sent all the AssetDataAdded events
			// that arose from the final tick of the gatherer. Some callers might do more expensive
			// work for assets added after the initial search completed, and we don't want them to do
			// that more expensive work on the last batch of assets before completion.
			return TickContext.SetIntermediateStatus(EGatherStatus::WaitingForEvents);
		}

		TickContext.RecordTimer(); // OnInitialSearchComplete reads data set by RecordTimer
		OnInitialSearchCompleted(EventContext);
	}
	else if (IsAdditionalMountSearchInProgress())
	{
		TickContext.RecordTimer(); // OnAdditionalMountSearchComplete reads data set by RecordTimer
		OnAdditionalMountSearchCompleted(EventContext);
	}

	// Give ProgressUpdateData a final call before going idle so that listeners can record 0 remaining.
	EventContext.ProgressUpdateData.Emplace(
		HighestPending,	// NumTotalAssets
		HighestPending,	// NumAssetsProcessedByAssetRegistry
		0,				// NumAssetsPendingDataLoad
		false			// bIsDiscoveringAssetFiles
	);
	// Clear HighestPending 
	HighestPending = 0;
	this->GatherStatus = EGatherStatus::Complete;
	return EGatherStatus::Complete;
}

void FAssetRegistryImpl::ProcessGameThreadRequestedClasses()
{
	if (!bGameThreadRequestedClassesShouldProcess.load(std::memory_order_relaxed))
	{
		return;
	}

	TSet<FTopLevelAssetPath> ClassesToProcess;
	{
		FScopeLock GameThreadRequestedClassesScopeLock(&GameThreadRequestedClassesLock);
		ClassesToProcess = MoveTemp(GameThreadRequestedClasses);
		GameThreadRequestedClasses.Empty();
		bGameThreadRequestedClassesShouldProcess.store(false, std::memory_order_relaxed);
	}

	for (const FTopLevelAssetPath& AssetClassPath : ClassesToProcess)
	{
		UClass* AssetClass = FindObject<UClass>(AssetClassPath, EFindObjectFlags::ExactClass);
		if (!AssetClass)
		{
			continue;
		}

		// Process each of the possible properties we are waiting on for the GameThreadRequestedClass
		if (AssetClass->GetSparseClassDataStruct())
		{
			AssetClass->GetSparseClassData(EGetSparseClassDataMethod::ArchetypeIfNull);
		}
	}
	bGameThreadRequestedClassesShouldRetry.store(true, std::memory_order_release);
}

void FAssetRegistryImpl::ConditionalRetryGatherAssetsForGameThread(Impl::FTickContext& TickContext)
{
	if (!TickContext.bIsInGameThread && bGameThreadRequestedClassesShouldRetry.load(std::memory_order_relaxed))
	{
		// When we find we were instructed to retry, block on an acquire memory fence so that the
		// modifications the game thread made to the classes are transferred from the other thread before
		// we continue on.
		std::atomic_thread_fence(std::memory_order_acquire);
		bGameThreadRequestedClassesShouldRetry.store(false, std::memory_order_release);
		for (TPair<FName, TUniquePtr<FAssetData>>& GameThreadPair : BackgroundResults.AssetsForGameThread)
		{
			BackgroundResults.Assets.Add(GameThreadPair.Key, MoveTemp(GameThreadPair.Value));
		}
		BackgroundResults.AssetsForGameThread.Empty();
	}
}

void FAssetRegistryImpl::ConditionalClearGameThreadRequestedClasses(Impl::FTickContext& TickContext)
{
	if (BackgroundResults.AssetsForGameThread.IsEmpty()
		&& bGameThreadRequestedClassesShouldProcess.load(std::memory_order_relaxed))
	{
		FScopeLock GameThreadRequestedClassesScopeLock(&GameThreadRequestedClassesLock);
		GameThreadRequestedClasses.Empty();
		bGameThreadRequestedClassesShouldProcess.store(false, std::memory_order_relaxed);
	}
}

void FAssetRegistryImpl::RequestGameThreadProcessClass(const FTopLevelAssetPath& RequestedGameThreadClass)
{
	FScopeLock GameThreadRequestedClassesScopeLock(&GameThreadRequestedClassesLock);
	GameThreadRequestedClasses.Add(RequestedGameThreadClass);
	bGameThreadRequestedClassesShouldProcess.store(true, std::memory_order_relaxed);
}

namespace Impl // UE::AssetRegistry::Impl
{

FTickContext::FTickContext(FAssetRegistryImpl& InGuardedData, Impl::FEventContext& InEventContext,
	Impl::FClassInheritanceContext& InInheritanceContext)
	: GuardedData(InGuardedData)
	, EventContext(InEventContext)
	, InheritanceContext(InInheritanceContext)
{
}

void FTickContext::LazyStartTimer()
{
	if (TimingStartTime <= 0.)
	{
		TimingStartTime = FPlatformTime::Seconds();
	}
};
void FTickContext::RecordTimer()
{
	if (TimingStartTime > 0.)
	{
		GuardedData.StoreGatherResultsTimeSeconds += static_cast<float>(FPlatformTime::Seconds() - TimingStartTime);
		TimingStartTime = -1.;
	}
};

void FTickContext::SetNumGatherFromDiskPendingDirty()
{
	bNumGatherFromDiskPendingDirty = true;
}

int32 FTickContext::GetNumGatherFromDiskPending()
{
	if (bNumGatherFromDiskPendingDirty)
	{
		bNumGatherFromDiskPendingDirty = false;
		NumGatherFromDiskPending = CalculateNumGatherFromDiskPending();
	}
	return NumGatherFromDiskPending;
}

int32 FTickContext::GetNumPending()
{
	int32 NumPending = GetNumGatherFromDiskPending();
#if WITH_EDITOR
	int32 NumNeedingCalculation = GuardedData.PackagesNeedingDependencyCalculation.Num()
		+ GuardedData.PackagesNeedingDependencyCalculationOnGameThread.Num();
	NumPending += NumNeedingCalculation > 0 ? 1 : 0;
#endif
	return NumPending;
}

int32 FTickContext::CalculateNumGatherFromDiskPending()
{
	// Structure that holds temporary data for the current call to TickGatherer
	UE::AssetDataGather::FResultContext& C = this->ResultContext;
	// Structure that holds persistent data stored across ticks.
	UE::AssetDataGather::FResults& B = GuardedData.BackgroundResults;
	// Reference to GuardedData for persistent data stored as fields on GuardedData rather than fields on the
	// FResults struct
	FAssetRegistryImpl& G = GuardedData;

	return
		C.NumFilesToSearch + C.NumPathsToSearch

		+ B.Paths.Num() + B.Assets.Num() + B.AssetsForGameThread.Num() + B.Dependencies.Num()
		+ B.DependenciesForGameThread.Num() + B.CookedPackageNamesWithoutAssetData.Num()
		+ B.VerseFiles.Num() + B.BlockedFiles.Num()

		+ G.DeferredAssets.Num() + G.DeferredAssetsForGameThread.Num() + G.DeferredDependencies.Num()
		+ G.DeferredDependenciesForGameThread.Num()
		;
};

EGatherStatus FTickContext::SetIntermediateStatus(EGatherStatus Status)
{
	// Report total pending and highest pending for this run so we can show a good progress bar
	int32 LocalNumPending = GetNumPending();
	int32& HighestPending = GuardedData.HighestPending;
	HighestPending = FMath::Max(HighestPending, LocalNumPending);
	if (ResultContext.bIsSearching || bHadAssetsToProcess)
	{
		EventContext.ProgressUpdateData.Emplace(
			HighestPending,						// NumTotalAssets
			HighestPending - LocalNumPending,	// NumAssetsProcessedByAssetRegistry
			LocalNumPending / 2,				// NumAssetsPendingDataLoad, divided by 2 because assets are double counted due to dependencies
			ResultContext.bIsDiscoveringFiles	// bIsDiscoveringAssetFiles
		);
	}
	GuardedData.GatherStatus = Status;
	return Status;
};

EGatherStatus FTickContext::UpdateIntermediateStatus()
{
	EGatherStatus Status;
	if (InterruptionContext.WasInterrupted())
	{
		// When interrupted we don't know the current status, so just keep the previous status, unless
		// the previous status is a temporary status, in which case just switch it over to TickActive
		switch (GuardedData.GatherStatus)
		{
		case EGatherStatus::WaitingForEvents: [[fallthrough]];
		case EGatherStatus::UnableToProgress:
			Status = ResultContext.bAbleToProgress ? EGatherStatus::TickActiveGatherActive
				: EGatherStatus::TickActiveGatherIdle;
			break;
		default:
			Status = GuardedData.GatherStatus;
			break;
		}
	}
	else if (ResultContext.bIsSearching && !ResultContext.bAbleToProgress)
	{
		// Gather from disk is still in progress but is blocked
		Status = EGatherStatus::UnableToProgress;
	}
	else if (ResultContext.bIsSearching)
	{
		// We still have work we can do in the gather
		Status = EGatherStatus::TickActiveGatherActive;
	}
	else
	{
		// We have received all of the assets from disk, but we are blocked on some of them, or we are blocked
		// on some followup calculations we need to do, because we have to wait for EngineStartup or some other
		// condition outside our control.
		Status = EGatherStatus::UnableToProgress;
	}

	if (Status == Impl::EGatherStatus::TickActiveGatherIdle)
	{
		// if there's no additional work the gatherer thread can perform, change the status from TickActiveGatherIdle
		// to TickGameThreadActiveGatherIdle.
		UE::AssetDataGather::FResults& B = GuardedData.BackgroundResults;
		if (GuardedData.DeferredAssets.Num() == 0
#if WITH_EDITOR
			&& GuardedData.PackagesNeedingDependencyCalculation.Num() == 0
#endif
			&& GuardedData.DeferredDependencies.Num() == 0
			&& B.Assets.Num() == 0
			&& B.Dependencies.Num() == 0
			&& B.CookedPackageNamesWithoutAssetData.Num() == 0
			&& B.Paths.Num() == 0)
		{
			Status = Impl::EGatherStatus::TickGameThreadActiveGatherIdle;
		}
	}
	return SetIntermediateStatus(Status);
};

void FTickContext::RunAssetSearchDataGathered(TMultiMap<FName, TUniquePtr<FAssetData>>& InAssetResults,
	TMultiMap<FName, TUniquePtr<FAssetData>>& OutDeferredAssetResults)
{
	// Process the asset results
	if (InAssetResults.Num())
	{
		LazyStartTimer();

		EDeferFlag DeferFlag;
		if (bIsInGameThread)
		{
			DeferFlag = EDeferFlag::CanExecuteGameThread;
		}
		else if (!GuardedData.bForceCompletionEvenIfClassNotLoaded)
		{
			// If we are still able to wait on missing classes, then defer rather than kicking to game thread.
			// The Gamethread might end up creating the required data for us if we continue waiting.
			DeferFlag = EDeferFlag::RetryLater;
		}
		else if (!GuardedData.bForceCompletionEvenIfNotOnGameThread)
		{
			DeferFlag = EDeferFlag::QueueToGameThread;
		}
		else
		{
			DeferFlag = EDeferFlag::BypassNow;
		}

		GuardedData.AssetSearchDataGathered(EventContext, InAssetResults, OutDeferredAssetResults,
			InterruptionContext, MountPointsForVerifyAfterGather, DeferFlag);
	}
}


void FTickContext::RunDependencyDataGathered(TMultiMap<FName, FPackageDependencyData>& DependenciesToProcess,
	TMultiMap<FName, FPackageDependencyData>& OutDeferredDependencies,
	TSet<FName>* OutPackagesNeedingDependencyCalculation)
{
	// Add dependencies
	if (DependenciesToProcess.Num())
	{
		LazyStartTimer();

		GuardedData.DependencyDataGathered(DependenciesToProcess, OutDeferredDependencies,
			OutPackagesNeedingDependencyCalculation, InterruptionContext, MountPointsForVerifyAfterGather);
	}
}

} // namespace Impl within UE::AssetRegistry

#if WITH_EDITOR
void FAssetRegistryImpl::ApplyObjectRedirectsToGatheredAssetDatas()
{
	bGatherNeedsApplyObjectRedirects = false;

	State.EnumerateAllMutableAssets([](FAssetData& AssetData)
		{
			if (!AssetData.TaggedAssetBundles)
			{
				return;
			}

			TArray<FAssetBundleEntry>& Bundles = AssetData.TaggedAssetBundles->Bundles;
			int32 NumBundles = Bundles.Num();
			int32 FirstChangedBundle = -1;
			int32 FirstChangedAssetPath = -1;
			FSoftObjectPath FirstChangedAssetPathValue;
			for (FirstChangedBundle = 0; FirstChangedBundle < NumBundles; ++FirstChangedBundle)
			{
				FAssetBundleEntry& Bundle = Bundles[FirstChangedBundle];
				int32 NumAssetPaths = Bundle.AssetPaths.Num();
				for (FirstChangedAssetPath = 0; FirstChangedAssetPath < NumAssetPaths; ++FirstChangedAssetPath)
				{
					FSoftObjectPath Original(Bundle.AssetPaths[FirstChangedAssetPath]);
					FSoftObjectPath Redirected = GRedirectCollector.GetAssetPathRedirection(Original);
					if (Redirected.IsValid())
					{
						FirstChangedAssetPathValue = Redirected;
						break;
					}
				}
				if (FirstChangedAssetPath < NumAssetPaths)
				{
					break;
				}
				FirstChangedAssetPath = -1;
			}

			if (FirstChangedBundle == NumBundles)
			{
				return;
			}

			FAssetBundleData NewData;
			NewData.Bundles.Reserve(NumBundles);
			int32 BundleIndex;
			for (BundleIndex = 0; BundleIndex < FirstChangedBundle; ++BundleIndex)
			{
				NewData.Bundles.Add(Bundles[BundleIndex]);
			}
			for (; BundleIndex < NumBundles; ++BundleIndex)
			{
				FAssetBundleEntry& NewBundle = NewData.Bundles.Emplace_GetRef(Bundles[BundleIndex]);
				TArray<FTopLevelAssetPath>& NewAssetPaths = NewBundle.AssetPaths;

				int32 NumAssetPaths = NewAssetPaths.Num();
				int32 AssetPathIndex = FirstChangedAssetPath != -1 ? FirstChangedAssetPath : 0;
				for (; AssetPathIndex < NumAssetPaths; ++AssetPathIndex)
				{
					FSoftObjectPath Redirected;
					FSoftObjectPath Original(NewAssetPaths[AssetPathIndex]);
					if (FirstChangedAssetPath == -1)
					{
						Redirected = GRedirectCollector.GetAssetPathRedirection(Original);
					}
					else
					{
						check(AssetPathIndex == FirstChangedAssetPath);
						Redirected = FirstChangedAssetPathValue;
						FirstChangedAssetPathValue = FSoftObjectPath();
						FirstChangedAssetPath = -1;
					}

					if (Redirected.IsNull())
					{
						continue;
					}
					if (!Redirected.GetSubPathUtf8String().IsEmpty())
					{
						UE_LOGF(LogAssetRegistry, Warning,
							"AssetBundleEntry Name %ls.%ls[%d] was redirected from %ls -> %ls, but the redirected target is not a TopLevelAssetPath, and AssetBundleEntry Names must be TopLevelAssetPaths. Ignoring the redirector.",
							*AssetData.ToSoftObjectPath().ToString(), *NewBundle.BundleName.ToString(), AssetPathIndex, *Original.ToString(), *Redirected.ToString());
						continue;
					}
					NewAssetPaths[AssetPathIndex] = Redirected.GetAssetPath();
				}
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				NewBundle.BundleAssets.Reset();
				Algo::Transform(NewBundle.AssetPaths, NewBundle.BundleAssets, [](FTopLevelAssetPath Path) { return FSoftObjectPath(Path, {}); });
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
			AssetData.TaggedAssetBundles = MakeShared<FAssetBundleData, ESPMode::ThreadSafe>(MoveTemp(NewData));
		});
}
#endif

void FAssetRegistryImpl::Shrink()
{
	BackgroundResults.Shrink();
	DeferredAssets.Shrink();
	DeferredAssetsForGameThread.Shrink();
	DeferredDependencies.Shrink();
	DeferredDependenciesForGameThread.Shrink();
#if WITH_EDITOR
	PackagesNeedingDependencyCalculation.Shrink();
	PackagesNeedingDependencyCalculationOnGameThread.Shrink();
#endif
	State.Shrink();

	GatherDependencyDataScratchPad->Shrink();
}

void FAssetRegistryImpl::OnInitialSearchCompleted(Impl::FEventContext& EventContext)
{
	// Handle any deferred loading operations
	SetPerformanceMode(Impl::EPerformanceMode::MostlyStatic);

	LogSearchDiagnostics(InitialSearchStartTime, /* bInitialScan */ true);
	CSV_TRACE_REGION_END(TEXT("Scan"), TEXT("AssetRegistry"));

	GlobalGatherer->UpdateCacheForSaving();
	GlobalGatherer->OnInitialSearchCompleted();
	Shrink();

	EventContext.bFileLoadedEventBroadcast = true;
	EventContext.bKnownGathersCompleteEventBroadcast = true;

	bInitialSearchCompleted.store(true, std::memory_order_relaxed);
	UpdateMaxSecondsPerFrame();

#if WITH_EDITOR
	ResolveRedirectionsInCachedBPInheritanceMap();
#endif
}

void FAssetRegistryImpl::OnAdditionalMountSearchCompleted(Impl::FEventContext& EventContext)
{
	// Handle any deferred loading operations
	SetPerformanceMode(Impl::EPerformanceMode::MostlyStatic);

	LogSearchDiagnostics(AdditionalMountSearchStartTime, /* bInitialScan */ false);
	CSV_TRACE_REGION_END(TEXT("Additional Mount Search"), TEXT("AssetRegistry")); // Matching CSV_TRACE_REGION_BEGIN in OnMountPointMounted

	GlobalGatherer->UpdateCacheForSaving();
	GlobalGatherer->OnAdditionalMountSearchCompleted();
	Shrink();

	EventContext.bKnownGathersCompleteEventBroadcast = true;

	bAdditionalMountSearchInProgress.store(false, std::memory_order_relaxed);
}

void FAssetRegistryImpl::LogSearchDiagnostics(double StartTime, bool bInitialScan)
{
	FAssetGatherDiagnostics Diagnostics = GlobalGatherer->GetDiagnostics();
	float Total = Diagnostics.DiscoveryTimeSeconds + Diagnostics.GatherTimeSeconds + StoreGatherResultsTimeSeconds;
	UE::Telemetry::AssetRegistry::FGatherTelemetry Telemetry;
	Telemetry.TotalSearchDurationSeconds = FPlatformTime::Seconds() - StartTime;
	Telemetry.TotalWorkTimeSeconds = Total;
	Telemetry.DiscoveryTimeSeconds = Diagnostics.DiscoveryTimeSeconds;
	Telemetry.GatherTimeSeconds = Diagnostics.GatherTimeSeconds;
	Telemetry.StoreTimeSeconds = StoreGatherResultsTimeSeconds;
	Telemetry.NumCachedDirectories = Diagnostics.NumCachedDirectories;
	Telemetry.NumUncachedDirectories = Diagnostics.NumUncachedDirectories;
	Telemetry.NumCachedAssetFiles = Diagnostics.NumCachedAssetFiles;
	Telemetry.NumUncachedAssetFiles = Diagnostics.NumUncachedAssetFiles;
	Telemetry.bIsInitialScan = bInitialScan;
	FTelemetryRouter::Get().ProvideTelemetry(Telemetry);
#if !NO_LOGGING
	TStringBuilder<256> Message;
	Message.Appendf(TEXT("AssetRegistryGather time %.4fs: AssetDataDiscovery %0.4fs, AssetDataGather %0.4fs, StoreResults %0.4fs. Wall time %0.4fs.")
		TEXT("\n\tNumCachedDirectories %d. NumUncachedDirectories %d. NumCachedFiles %d. NumUncachedFiles %d."),
		Total, Diagnostics.DiscoveryTimeSeconds, Diagnostics.GatherTimeSeconds, StoreGatherResultsTimeSeconds,
		Diagnostics.WallTimeSeconds, Diagnostics.NumCachedDirectories, Diagnostics.NumUncachedDirectories,
		Diagnostics.NumCachedAssetFiles, Diagnostics.NumUncachedAssetFiles);
#if WITH_EDITOR
	Message.Appendf(TEXT("\n\tBackgroundTickInterruptions %d."), BackgroundTickInterruptionsCount);
#endif // WITH_EDITOR

	UE_LOGF(LogAssetRegistry, Log, "%ls", *Message);

	if (bVerboseLogging)
	{
		UE_LOGF(LogAssetRegistry, Verbose, "TagMemoryUse:");
		TagSizeByClass.ValueSort([](int64 A, int64 B) { return A > B; });
		constexpr int64 MinSizeToLog = 1000 * 1000;
		for (const TPair<FTopLevelAssetPath, int64>& Pair : TagSizeByClass)
		{
			if (Pair.Value < MinSizeToLog)
			{
				break;
			}
			UE_LOGF(LogAssetRegistry, Verbose, "%ls: %.1f MB",
				*Pair.Key.ToString(), (float)Pair.Value / (1000.f * 1000.f));
		}
	}
#endif // !NO_LOGGING
}

void FAssetRegistryImpl::TickGatherPackage(UE::AssetRegistry::FInterfaceWriteScopeLock& ScopeLock, Impl::FEventContext& EventContext, const FString& PackageName, const FString& LocalPath)
{
	if (!GlobalGatherer.IsValid())
	{
		return;
	}

	bool bIsInGameThread = IsInGameThread();

	// Release the lock while performing expensive calls that don't need the lock
	ScopeLock.Lock.WriteUnlock();
	GlobalGatherer->WaitOnPath(LocalPath);
	ScopeLock.Lock.WriteLock();
	check(GlobalGatherer.IsValid());

	double TimingStartTime = -1.;
	auto LazyStartTimer = [&TimingStartTime]()
	{
		if (TimingStartTime <= 0.)
		{
			TimingStartTime = FPlatformTime::Seconds();
		}
	};
	ON_SCOPE_EXIT
	{
		if (TimingStartTime > 0.)
		{
			StoreGatherResultsTimeSeconds += static_cast<float>(FPlatformTime::Seconds() - TimingStartTime);
			TimingStartTime = -1.;
		}
	};

	FName PackageFName(PackageName);

	// Gather results from the background search
	GlobalGatherer->GetPackageResults(BackgroundResults);

	// The package could be in either the main or the ForGameThread containers but it will only appear in one or the other
	// Either way, we put it into these two local containers and if we have to defer it, we'll put it into the game thread versions
	TArray<TUniquePtr<FAssetData>*> PackageAssets;
	TArray<FPackageDependencyData> PackageDependencyDatas;
	BackgroundResults.Assets.MultiFindPointer(PackageFName, PackageAssets);
	BackgroundResults.AssetsForGameThread.MultiFindPointer(PackageFName, PackageAssets);
	// We can't remove the assets until we've finished the transfer into the PackageAssetsMap below
	BackgroundResults.Dependencies.MultiFind(PackageFName, PackageDependencyDatas);
	BackgroundResults.Dependencies.Remove(PackageFName);
	BackgroundResults.DependenciesForGameThread.MultiFind(PackageFName, PackageDependencyDatas);
	BackgroundResults.DependenciesForGameThread.Remove(PackageFName);

	DeferredAssets.MultiFindPointer(PackageFName, PackageAssets);
	DeferredAssetsForGameThread.MultiFindPointer(PackageFName, PackageAssets);
	DeferredDependencies.MultiFind(PackageFName, PackageDependencyDatas);
	DeferredDependencies.Remove(PackageFName);
	DeferredDependenciesForGameThread.MultiFind(PackageFName, PackageDependencyDatas);
	DeferredDependenciesForGameThread.Remove(PackageFName);

	TOptional<TSet<FString>> MountPointsForVerifyAfterGather;
	if (PackageAssets.Num() > 0)
	{
		LazyStartTimer();
		TMultiMap<FName, TUniquePtr<FAssetData>> PackageAssetsMap;
		PackageAssetsMap.Reserve(PackageAssets.Num());
		for (TUniquePtr<FAssetData>* PackageAsset : PackageAssets)
		{
			PackageAssetsMap.Add(PackageFName, MoveTemp(*PackageAsset));
		}
		// Ownership transfer is now complete so remove these packages from the results arrays
		BackgroundResults.Assets.Remove(PackageFName);
		BackgroundResults.AssetsForGameThread.Remove(PackageFName);
		DeferredAssets.Remove(PackageFName);
		DeferredAssetsForGameThread.Remove(PackageFName);

		TMultiMap<FName, TUniquePtr<FAssetData>> DeferredPackageAssetsMap;
		UE::AssetRegistry::Impl::FInterruptionContext InterruptionContext;
		AssetSearchDataGathered(EventContext, PackageAssetsMap, DeferredPackageAssetsMap,
			InterruptionContext, MountPointsForVerifyAfterGather,
			bIsInGameThread ? EDeferFlag::CanExecuteGameThread : EDeferFlag::RetryLater);
		if (DeferredPackageAssetsMap.Num())
		{
			UE_LOG(LogAssetRegistry, Warning, TEXT("Attempted to add package '%s' to the registry before its UClass was available. \
Could not execute PostLoadAssetRegistryTags. We will try again later. Until then, dependency data will also be unavailable."),
				*PackageName);
			FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
			DeferredAssetsForGameThread.Append(MoveTemp(DeferredPackageAssetsMap));
			// If we are deferring this data we won't process the dependency data below anyway (we'll early out of DependencyDataGathered)
			// so put the dependency data back into the BackgroundResults.DependenciesForGameThread which is where we will expect
			// to find it when we reprocess the DeferredAssetsForGameThread after clearing the rest of the results queue.
			for (FPackageDependencyData& Data : PackageDependencyDatas)
			{
				BackgroundResults.DependenciesForGameThread.Add(PackageFName, Data);
			}
			PackageDependencyDatas.Empty();
		}
	}
	if (PackageDependencyDatas.Num() > 0)
	{
		LazyStartTimer();
		TMultiMap<FName, FPackageDependencyData> PackageDependencyDatasMap;
		PackageDependencyDatasMap.Reserve(PackageDependencyDatas.Num());
		for (FPackageDependencyData& DependencyData : PackageDependencyDatas)
		{
			PackageDependencyDatasMap.Add(PackageFName, MoveTemp(DependencyData));
		}
		TSet<FName>* OutPackagesNeedingDependencyCalculation = nullptr;
#if WITH_EDITOR
		OutPackagesNeedingDependencyCalculation = &PackagesNeedingDependencyCalculation;
#endif
		UE::AssetRegistry::Impl::FInterruptionContext InterruptionContext;
		DependencyDataGathered(PackageDependencyDatasMap, DeferredDependenciesForGameThread,
			OutPackagesNeedingDependencyCalculation, InterruptionContext, MountPointsForVerifyAfterGather);
	}
}

#if WITH_EDITOR
void FAssetRegistryImpl::LoadCalculatedDependencies(TArray<FName>* AssetPackageNamesToCalculate, 
	Impl::FClassInheritanceContext& InheritanceContext, TSet<FName>* InPackagesNeedingDependencyCalculation,
	Impl::FInterruptionContext& InOutInterruptionContext)
{
	auto CheckForTimeUp = [&InOutInterruptionContext](bool bHadActivity)
	{
		// Only Check TimeUp when we found something to do, otherwise we waste time calling FPlatformTime::Seconds
		if (!bHadActivity)
		{
			return false;
		}
		return InOutInterruptionContext.ShouldExitEarly();
	};

	RebuildAssetDependencyGathererMapIfNeeded();

	if (AssetPackageNamesToCalculate)
	{
		for (FName PackageName : *AssetPackageNamesToCalculate)
		{
			// We do not remove the package from InPackagesNeedingDependencyCalculation, because
			// we are only calculating an interim result when AssetsToCalculate is non-null
			// We will run again on each of these PackageNames when TickGatherer finishes gathering all dependencies
			if (InPackagesNeedingDependencyCalculation->Contains(PackageName))
			{
				bool bHadActivity;
				LoadCalculatedDependencies(PackageName, InheritanceContext, bHadActivity);
				if (CheckForTimeUp(bHadActivity))
				{
					return;
				}
			}
		}
	}
	else
	{
		for (TSet<FName>::TIterator It = InPackagesNeedingDependencyCalculation->CreateIterator(); It; ++It)
		{
			bool bHadActivity;
			LoadCalculatedDependencies(*It, InheritanceContext, bHadActivity);
			It.RemoveCurrent();
			if (CheckForTimeUp(bHadActivity))
			{
				return;
			}
		}
		check(InPackagesNeedingDependencyCalculation->IsEmpty());
	}
}

void FAssetRegistryImpl::LoadCalculatedDependencies(FName PackageName,
	Impl::FClassInheritanceContext& InheritanceContext, bool& bOutHadActivity)
{
	bOutHadActivity = false;
	
	auto GetCompiledFilter = [this, &InheritanceContext](const FARFilter& InFilter) -> FARCompiledFilter
	{
		FARCompiledFilter CompiledFilter;
		CompileFilter(InheritanceContext, InFilter, CompiledFilter);
		return CompiledFilter;
	};

	FReadScopeLock GathererClassScopeLock(RegisteredDependencyGathererClassesLock);

	TArray<UE::AssetDependencyGatherer::Private::FRegisteredAssetDependencyGatherer*, TInlineAllocator<2>> Gatherers;
	TArray<FString> DirectoryReferences;
	State.EnumerateAssetsByPackageName(PackageName,
		[this, &Gatherers, &GetCompiledFilter, &bOutHadActivity, PackageName, &DirectoryReferences](const FAssetData* AssetData)
	{
		Gatherers.Reset();

		// Check the class name instead of trying to load the actual class as that is slow
		// This code could be moved somewhere where it doesn't need to re-query the asset data, but it needs to happen after both dependencies and data are handled
		RegisteredDependencyGathererClasses.MultiFind(AssetData->AssetClassPath, Gatherers);

		TArray<IAssetDependencyGatherer::FGathereredDependency> GatheredDependencies;
		TArray<FString> GathererDirectoryReferences;

		IAssetDependencyGatherer::FGatherDependenciesContext Context { *AssetData, State, GetCachedPathTree(), GetCompiledFilter, GatheredDependencies, GathererDirectoryReferences};

		for (UE::AssetDependencyGatherer::Private::FRegisteredAssetDependencyGatherer* Gatherer : Gatherers)
		{
			bOutHadActivity = true;
			GatheredDependencies.Reset();
			GathererDirectoryReferences.Reset();

			Gatherer->GatherDependencies(Context);
			
			if (GatheredDependencies.Num())
			{
				FDependsNode* SourceNode = State.CreateOrFindDependsNode(FAssetIdentifier(PackageName));
				bool bWasSourceDependencyListSorted = SourceNode->IsDependencyListSorted(EDependencyCategory::Package);
				SourceNode->SetIsDependencyListSorted(EDependencyCategory::Package, false);

				for (const IAssetDependencyGatherer::FGathereredDependency& GatheredDepencency : GatheredDependencies)
				{
					FDependsNode* TargetNode = State.CreateOrFindDependsNode(FAssetIdentifier(GatheredDepencency.PackageName));
					EDependencyProperty DependencyProperties = GatheredDepencency.Property;
					SourceNode->AddDependency(TargetNode, EDependencyCategory::Package, DependencyProperties);
					TargetNode->AddReferencer(SourceNode);
				}

				SourceNode->SetIsDependencyListSorted(EDependencyCategory::Package, bWasSourceDependencyListSorted);
			}

			DirectoryReferences.Append(MoveTemp(GathererDirectoryReferences));
		}

		return true; // Keep iterating the assets in the package
	});
	SetDirectoriesWatchedByPackage(PackageName, MoveTemp(DirectoryReferences));
}

void FAssetRegistryImpl::SetDirectoriesWatchedByPackage(FName PackageName, TConstArrayView<FString> Directories)
{
	TArray<FString> DirectoryLocalPaths;
	DirectoryLocalPaths.Reserve(Directories.Num());
	for (const FString& DirectoryLocalPathOrLongPackageName : Directories)
	{
		FString DirectoryLocalPath;
		if (!FPackageName::TryConvertLongPackageNameToFilename(DirectoryLocalPathOrLongPackageName, DirectoryLocalPath))
		{
			UE_LOGF(LogAssetRegistry, Warning,
				"AddDirectoryReferencer called with LongPackageName %ls that cannot be mapped to a local path. Ignoring it.",
				*DirectoryLocalPathOrLongPackageName);
		}
		DirectoryLocalPath = UE::AssetRegistry::CreateStandardFilename(DirectoryLocalPath);
		DirectoryLocalPaths.Add(MoveTemp(DirectoryLocalPath));
	}

	uint32 PackageNameHash = GetTypeHash(PackageName);
	TArray<FString>* WatchedDirectories = DirectoriesWatchedByPackage.FindByHash(PackageNameHash, PackageName);

	// Remove all old values from the reverse map
	if (WatchedDirectories)
	{
		for (const FString& WatchedDirectory : *WatchedDirectories)
		{
			TSet<FName>* DirectoryWatchers = PackagesWatchingDirectory.Find(WatchedDirectory);
			if (DirectoryWatchers)
			{
				DirectoryWatchers->RemoveByHash(PackageNameHash, PackageName);
			}
		}
	}

	if (Directories.IsEmpty())
	{
		if (WatchedDirectories)
		{
			DirectoriesWatchedByPackage.RemoveByHash(PackageNameHash, PackageName);
		}
		return;
	}

	if (!WatchedDirectories)
	{
		WatchedDirectories = &DirectoriesWatchedByPackage.FindOrAddByHash(PackageNameHash, PackageName);
	}
	*WatchedDirectories = MoveTemp(DirectoryLocalPaths);

	for (const FString& WatchedDirectory : *WatchedDirectories)
	{
		TSet<FName>& DirectoryWatchers = PackagesWatchingDirectory.FindOrAdd(WatchedDirectory);
		DirectoryWatchers.AddByHash(PackageNameHash, PackageName);
	}
}

#endif


}

void UAssetRegistryImpl::Serialize(FArchive& Ar)
{
	if (Ar.IsObjectReferenceCollector())
	{
		// The Asset Registry does not have any object references, and its serialization function is expensive
		return;
	}
	UE::AssetRegistry::Impl::FEventContext EventContext;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		GuardedData.Serialize(Ar, EventContext);
	}
	Broadcast(EventContext);
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::Serialize(FArchive& Ar, Impl::FEventContext& EventContext)
{
	check(!Ar.IsObjectReferenceCollector()); // Caller should not call in this case
	if (Ar.IsLoading())
	{
		State.Load(Ar);
		CachePathsFromState(EventContext, State);
		UpdatePersistentMountPoints();
	}
	else if (Ar.IsSaving())
	{
		State.Save(Ar, SerializationOptions);
	}
}

}

/** Append the assets from the incoming state into our own */
void UAssetRegistryImpl::AppendState(const FAssetRegistryState& InState, UE::AssetRegistry::EAppendMode AppendMode)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("UAssetRegistryImpl::AppendState");
	UE::AssetRegistry::Impl::FEventContext EventContext;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		GuardedData.AppendState(EventContext, InState, AppendMode, /*bEmitAssetEvents*/true);
	}
	Broadcast(EventContext);
}

bool UAssetRegistryImpl::ActiveMountsIsEnabled()
{
	return ActiveMountsLoader.IsEnabled();
}

void UAssetRegistryImpl::ActiveMountsInitialize()
{
	using namespace UE::AssetRegistry;

	UE::AssetRegistry::Impl::FActiveMountEvents ActiveMountEvents;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		ActiveMountsLoader.OnActiveMountsInitialize(*this, ActiveMountEvents, InterfaceScopeLock);
	}
	ActiveMountEvents.Broadcast();
}

bool UAssetRegistryImpl::ActiveMountsRegisterAndLoadFilePathSynchronous(FStringView StateFilePath)
{
	if (StateFilePath.IsEmpty())
	{
		ensureMsgf(false, TEXT("AppendStateForActiveMounts called with an invalid empty StateFilePath."));
		return false;
	}

	UE::AssetRegistry::Impl::FEventContext EventContext;
	UE::AssetRegistry::Impl::FActiveMountEvents ActiveMountEvents;
	LLM_SCOPE(ELLMTag::AssetRegistry);
	bool bResult = ActiveMountsLoader.TryRegisterAssetProviderAndAppendStateOutsideARLock(*this, EventContext,
		ActiveMountEvents, StateFilePath);
	Broadcast(EventContext);
	ActiveMountEvents.Broadcast();
	return bResult;
}

void UAssetRegistryImpl::ActiveMountsRegisterAndLoadFilePathAsync(FStringView StateFilePath,
	TFunction<void(bool bSucceeded)> OnComplete, UE::Tasks::ETaskPriority TaskPriority)
{
	if (StateFilePath.IsEmpty())
	{
		ensureMsgf(false, TEXT("AppendStateForActiveMounts called with an invalid empty StateFilePath."));
		if (OnComplete)
		{
			OnComplete(false);
		}
		return;
	}

	UE::Tasks::Launch(UE_SOURCE_LOCATION,
		[StateFilePath = FString(StateFilePath), OnComplete = MoveTemp(OnComplete)]
		{
			bool bSuccess = false;
			IAssetRegistry* GlobalAR = IAssetRegistry::Get();
			if (GlobalAR)
			{
				bSuccess = GlobalAR->ActiveMountsRegisterAndLoadFilePathSynchronous(StateFilePath);
			}
			if (OnComplete)
			{
				OnComplete(bSuccess);
			}
		}, TaskPriority);
}

void UAssetRegistryImpl::ActiveMountsLoadAsync(TConstArrayView<FStringView> MountPointLongPackageNames,
	TFunction<void(bool bCanceled)> OnComplete)
{
	using namespace UE::AssetRegistry;

	LLM_SCOPE(ELLMTag::AssetRegistry);
	TArray<TRefCountPtr<UE::PackageName::IMountPoint>> MountPoints;
	MountPoints.Reserve(MountPointLongPackageNames.Num());
	for (FStringView MountPointName : MountPointLongPackageNames)
	{
		TRefCountPtr<UE::PackageName::IMountPoint> MountPoint =
			FPackageName::FindMountPointByRootPackageName(MountPointName);
		if (MountPoint)
		{
			MountPoints.Add(MoveTemp(MountPoint));
		}
	}

	UE::AssetRegistry::Impl::FActiveMountEvents ActiveMountEvents;
	ActiveMountsLoader.LoadAsync(*this, ActiveMountEvents, MountPoints, MoveTemp(OnComplete));
	ActiveMountEvents.Broadcast();
}

void UAssetRegistryImpl::ActiveMountsReloadAllAsync(TFunction<void(bool bCanceled)> OnComplete)
{
	UE::AssetRegistry::Impl::FActiveMountEvents ActiveMountEvents;
	ActiveMountsLoader.ReloadAllAsync(*this, ActiveMountEvents, MoveTemp(OnComplete));
	ActiveMountEvents.Broadcast();
}

void UAssetRegistryImpl::ActiveMountsUnloadAsync(TConstArrayView<FStringView> MountPointLongPackageNames,
	TFunction<void(bool bCanceled)> OnComplete)
{
	using namespace UE::AssetRegistry;

	LLM_SCOPE(ELLMTag::AssetRegistry);
	TArray<TRefCountPtr<UE::PackageName::IMountPoint>> MountPoints;
	MountPoints.Reserve(MountPointLongPackageNames.Num());
	for (FStringView MountPointName : MountPointLongPackageNames)
	{
		TRefCountPtr<UE::PackageName::IMountPoint> MountPoint =
			FPackageName::FindMountPointByRootPackageName(MountPointName);
		if (MountPoint)
		{
			MountPoints.Add(MoveTemp(MountPoint));
		}
	}

	UE::AssetRegistry::Impl::FActiveMountEvents ActiveMountEvents;
	ActiveMountsLoader.UnloadAsync(*this, ActiveMountEvents, MountPoints, MoveTemp(OnComplete));
	ActiveMountEvents.Broadcast();
}

void UAssetRegistryImpl::ActiveMountsUnloadAllUnmountedAsync(TFunction<void(bool bCanceled)> OnComplete)
{
	UE::AssetRegistry::Impl::FActiveMountEvents ActiveMountEvents;
	ActiveMountsLoader.UnloadAllUnmountedAsync(*this, ActiveMountEvents, MoveTemp(OnComplete));
	ActiveMountEvents.Broadcast();
}

bool UAssetRegistryImpl::ActiveMountsIsLoaded(FStringView MountPointLongPackageName)
{
	using namespace UE::AssetRegistry;

	TRefCountPtr<UE::PackageName::IMountPoint> MountPoint =
		FPackageName::FindMountPointByRootPackageName(MountPointLongPackageName);
	if (!MountPoint)
	{
		return false;
	}

	LLM_SCOPE(ELLMTag::AssetRegistry);
	UE::AssetRegistry::Impl::FActiveMountEvents ActiveMountEvents;
	bool bResult = ActiveMountsLoader.IsLoaded(*this, ActiveMountEvents, MountPoint);
	ActiveMountEvents.Broadcast();
	return bResult;
}

bool UAssetRegistryImpl::ActiveMountsIsInProgress()
{
	return ActiveMountsLoader.IsInProgress(*this);
}

void UAssetRegistryImpl::ActiveMountsReportWhenIdle(TFunction<void()> OnIdle)
{
	UE::AssetRegistry::Impl::FActiveMountEvents ActiveMountEvents;
	ActiveMountsLoader.ReportWhenIdle(*this, ActiveMountEvents, MoveTemp(OnIdle));
	ActiveMountEvents.Broadcast();
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::AppendState(Impl::FEventContext& EventContext, const FAssetRegistryState& InState, 
	FAssetRegistryState::EInitializationMode Mode, bool bEmitAssetEvents)
{
	FAssetRegistryAppendResult LocalAppendResult;
	FAssetRegistryAppendResult* AppendResultPtr = bEmitAssetEvents ? &LocalAppendResult : nullptr;

#if WITH_EDITOR
	const FAssetRegistrySerializationOptions& Options = DevelopmentSerializationOptions;
	//Development options have all filters disabled, so there is no need to resolve runtime class filter options
#else
	FAssetRegistrySerializationOptions& Options = SerializationOptions;
	const bool bSuppressWarnings = !UE::AssetRegistry::Utils::HasEngineModuleLoaded();
	Utils::UpdateSerializationOptions(Options, CachedParsedFilterRules, bSuppressWarnings);
#endif
	State.InitializeFromExisting(
		InState,
		Options,
		Mode,
		AppendResultPtr);

	CachePathsFromState(EventContext, InState, &Options);
	CachedPathTree.Shrink();

	if (AppendResultPtr)
	{
		for (const FAssetData* AssetData : AppendResultPtr->AddedAssets)
		{
			EventContext.AssetEvents.Emplace(*AssetData, UE::AssetRegistry::Impl::FEventContext::EEvent::Added);
		}
		for (const FAssetData* AssetData : AppendResultPtr->UpdatedAssets)
		{
			EventContext.AssetEvents.Emplace(*AssetData, UE::AssetRegistry::Impl::FEventContext::EEvent::Updated);
		}
	}
}

void FAssetRegistryImpl::ConsumeState(Impl::FEventContext& EventContext, FAssetRegistryState&& InState,
	bool bEmitAssetEvents)
{
	FAssetRegistryAppendResult LocalAppendResult;
	FAssetRegistryAppendResult* AppendResultPtr = bEmitAssetEvents ? &LocalAppendResult : nullptr;

#if WITH_EDITOR
	const FAssetRegistrySerializationOptions& InOptions = DevelopmentSerializationOptions;
	//Development options have all filters disabled, so there is no need to resolve runtime class filter options
#else
	FAssetRegistrySerializationOptions& InOptions = SerializationOptions;
	const bool bSuppressWarnings = !UE::AssetRegistry::Utils::HasEngineModuleLoaded();
	Utils::UpdateSerializationOptions(InOptions, CachedParsedFilterRules, bSuppressWarnings);
#endif

	// Since we're moving assets rather than copying them, we do not filter them by tags and we do not remove any
	// any entries from their tags.
	FAssetRegistrySerializationOptions ConsumeOptions = InOptions;
	ConsumeOptions.bFilterAssetDataWithNoTags = false;
	ConsumeOptions.CookFilterlistTagsByClass.Empty();

	CachePathsFromState(EventContext, InState, &ConsumeOptions);

	State.ConsumeExisting(
		MoveTemp(InState),
		ConsumeOptions,
		AppendResultPtr);

	CachedPathTree.Shrink();

	if (AppendResultPtr)
	{
		for (const FAssetData* AssetData : AppendResultPtr->AddedAssets)
		{
			EventContext.AssetEvents.Emplace(*AssetData, UE::AssetRegistry::Impl::FEventContext::EEvent::Added);
		}
		for (const FAssetData* AssetData : AppendResultPtr->UpdatedAssets)
		{
			EventContext.AssetEvents.Emplace(*AssetData, UE::AssetRegistry::Impl::FEventContext::EEvent::Updated);
		}
	}
}

void FAssetRegistryImpl::CachePathsFromState(Impl::FEventContext& EventContext, const FAssetRegistryState& InState,
	const FAssetRegistrySerializationOptions* InOptions)
{
	SCOPED_BOOT_TIMING("FAssetRegistryImpl::CachePathsFromState");

	// Refreshes ClassGeneratorNames if out of date due to module load
	CollectCodeGeneratorClasses();

	if (!InOptions)
	{
#if WITH_EDITOR
		InOptions = &DevelopmentSerializationOptions;
#else
		InOptions = &SerializationOptions;
#endif
	}
	const FAssetRegistrySerializationOptions& Options = *InOptions;

	// Add paths to cache
	InState.EnumerateAllAssets([this, &Options, &EventContext](const FAssetData& AssetData)
	{
		if (Options.bFilterAssetDataWithNoTags && AssetData.TagsAndValues.Num() == 0 &&
			!FPackageName::IsLocalizedPackage(WriteToString<256>(AssetData.PackageName)))
		{
			// based on current serialization settings this should be skipped.
			return;
		}

		AddAssetPath(EventContext, AssetData.PackagePath);

		// Populate the class map if adding blueprint
		if (ClassGeneratorNames.Contains(AssetData.AssetClassPath))
		{
			FAssetRegistryExportPath GeneratedClass = AssetData.GetTagValueRef<FAssetRegistryExportPath>(FBlueprintTags::GeneratedClassPath);
			FAssetRegistryExportPath ParentClass = AssetData.GetTagValueRef<FAssetRegistryExportPath>(FBlueprintTags::ParentClassPath);

			if (GeneratedClass && ParentClass)
			{
				AddCachedBPClassParent(GeneratedClass.ToTopLevelAssetPath(), ParentClass.ToTopLevelAssetPath());

				// Invalidate caching because CachedBPInheritanceMap got modified
				TempCachedInheritanceBuffer.bDirty = true;
			}
		}
	});
}

}

SIZE_T UAssetRegistryImpl::GetAllocatedSize(bool bLogDetailed) const
{
	SIZE_T StateSize = 0;
	SIZE_T StaticSize = 0;
	SIZE_T SearchSize = 0;
	{
		UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
		GuardedData.GetAllocatedSize(bLogDetailed, StateSize, StaticSize, SearchSize);
		StaticSize += sizeof(UAssetRegistryImpl);
#if WITH_EDITOR
		StaticSize += OnDirectoryChangedDelegateHandles.GetAllocatedSize();
#endif
	}

	if (bLogDetailed)
	{
		UE_LOG(LogAssetRegistry, Log, TEXT("AssetRegistry Static Size: %" SIZE_T_FMT "k"), StaticSize / 1024);
		UE_LOG(LogAssetRegistry, Log, TEXT("AssetRegistry Search Size: %" SIZE_T_FMT "k"), SearchSize / 1024);
	}

	return StateSize + StaticSize + SearchSize;
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::GetAllocatedSize(bool bLogDetailed, SIZE_T& StateSize, SIZE_T& StaticSize, SIZE_T& SearchSize) const
{
	StateSize = State.GetAllocatedSize(bLogDetailed);

	if (bLogDetailed)
	{
		UE_LOG(LogAssetRegistry, Log, TEXT("AssetRegistry CachedEmptyPackages Size: %" SIZE_T_FMT "k"), CachedEmptyPackages.GetAllocatedSize() / 1024);
		UE_LOG(LogAssetRegistry, Log, TEXT("AssetRegistry CachedBPInheritanceMap Size: %" SIZE_T_FMT "k"), CachedBPInheritanceMap.GetAllocatedSize() / 1024);
		UE_LOG(LogAssetRegistry, Log, TEXT("AssetRegistry ClassGeneratorNames Size: %" SIZE_T_FMT "k"), ClassGeneratorNames.GetAllocatedSize() / 1024);
		UE_LOG(LogAssetRegistry, Log, TEXT("AssetRegistry BackgroundResults Size: %" SIZE_T_FMT "k"), BackgroundResults.GetAllocatedSize() / 1024);
		UE_LOG(LogAssetRegistry, Log, TEXT("AssetRegistry CachedPathTree Size: %" SIZE_T_FMT "k"), CachedPathTree.GetAllocatedSize() / 1024);
		UE_LOG(LogAssetRegistry, Log, TEXT("AssetRegistry CachedParsedFilterRules Size: %" SIZE_T_FMT "k"), CachedParsedFilterRules.GetAllocatedSize() / 1024);
	}
	StaticSize = CachedEmptyPackages.GetAllocatedSize() + CachedBPInheritanceMap.GetAllocatedSize() + ClassGeneratorNames.GetAllocatedSize();
	SearchSize = BackgroundResults.GetAllocatedSize() + CachedPathTree.GetAllocatedSize() + CachedParsedFilterRules.GetAllocatedSize();

	if (bIsTempCachingEnabled && !bIsTempCachingAlwaysEnabled)
	{
		SIZE_T TempCacheMem = TempCachedInheritanceBuffer.GetAllocatedSize();
		StaticSize += TempCacheMem;
		UE_LOGF(LogAssetRegistry, Warning, "Asset Registry Temp caching enabled, wasting memory: %lldk", TempCacheMem / 1024);
	}

	if (GlobalGatherer.IsValid())
	{
		SearchSize += sizeof(*GlobalGatherer);
		SearchSize += GlobalGatherer->GetAllocatedSize();
		
		if (bLogDetailed)
		{
			UE_LOG(LogAssetRegistry, Log, TEXT("AssetRegistry GlobalGatherer Size: %" SIZE_T_FMT "k"), GlobalGatherer->GetAllocatedSize() / 1024);
		}
	}

	StaticSize += SerializationOptions.CookFilterlistTagsByClass.GetAllocatedSize();
	for (const TPair<FTopLevelAssetPath, TSet<FName>>& Pair : SerializationOptions.CookFilterlistTagsByClass)
	{
		StaticSize += Pair.Value.GetAllocatedSize();
	}
	StaticSize += SerializationOptions.AssetBundlesDenyList.GetAllocatedSize();
	for (const FString& Filter : SerializationOptions.AssetBundlesDenyList)
	{
		StaticSize += Filter.GetAllocatedSize();
	}
}

}

void UAssetRegistryImpl::LoadPackageRegistryData(FArchive& Ar, FLoadPackageRegistryData& InOutData) const
{
	FPackageReader Reader;
	if (Reader.OpenPackageFile(&Ar))
	{
		UE::AssetRegistry::Utils::ReadAssetFile(Reader, InOutData);
	}
}

void UAssetRegistryImpl::LoadPackageRegistryData(const FString& PackageFilename, FLoadPackageRegistryData& InOutData) const
{
	FPackageReader Reader;
	if (Reader.OpenPackageFile(PackageFilename))
	{
		UE::AssetRegistry::Utils::ReadAssetFile(Reader, InOutData);
	}
}

namespace UE::AssetRegistry::Utils
{

bool ReadAssetFile(FPackageReader& PackageReader, IAssetRegistry::FLoadPackageRegistryData& InOutData)
{
	TArray<FAssetData*> AssetDataList;
	TArray<FString> CookedPackageNamesWithoutAssetDataGathered;

	const bool bGetDependencies = (InOutData.bGetDependencies);
	FPackageDependencyData DependencyData;

	bool bReadOk = FAssetDataGatherer::ReadAssetFile(PackageReader, AssetDataList, DependencyData,
		CookedPackageNamesWithoutAssetDataGathered,
		InOutData.bGetDependencies ? FPackageReader::EReadOptions::Dependencies : FPackageReader::EReadOptions::None);

	if (bReadOk)
	{
		// Copy & free asset data to the InOutData
		InOutData.Data.Reset(AssetDataList.Num());
		for (FAssetData* AssetData : AssetDataList)
		{
			InOutData.Data.Emplace(*AssetData);
		}

		AssetDataList.Reset();

		if (InOutData.bGetDependencies)
		{
			InOutData.DataDependencies.Reset(DependencyData.PackageDependencies.Num());
			for (FPackageDependencyData::FPackageDependency& Dependency : DependencyData.PackageDependencies)
			{
				InOutData.DataDependencies.Emplace(Dependency.PackageName);
			}
		}
	}

	// Cleanup the allocated asset data
	for (FAssetData* AssetData : AssetDataList)
	{
		delete AssetData;
	}

	return bReadOk;
}

}

void UAssetRegistryImpl::InitializeTemporaryAssetRegistryState(FAssetRegistryState& OutState,
	const FAssetRegistrySerializationOptions& Options, bool bRefreshExisting,
	const TSet<FName>& RequiredPackages, const TSet<FName>& RemovePackages) const
{
	using FAssetDataMap = UE::AssetRegistry::Private::FAssetDataMap;

	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	const FAssetRegistryState& State = GuardedData.GetState();
	if (!RequiredPackages.IsEmpty() || !RemovePackages.IsEmpty())
	{
		if (bRefreshExisting)
		{
			// InitializeFromExistingAndPrune does not support EInitializationMode so we have to Initialize and then Prune
			OutState.InitializeFromExisting(State.CachedAssets, State.CachedDependsNodes, State.CachedPackageData, Options,
				FAssetRegistryState::EInitializationMode::OnlyUpdateExisting);
			OutState.PruneAssetData(RequiredPackages, RemovePackages, Options);
		}
		else
		{
			TSet<int32> UnusedChunksToKeep;
			OutState.InitializeFromExistingAndPrune(State, RequiredPackages, RemovePackages, UnusedChunksToKeep, Options);
			OutState.Shrink();
		}
	}
	else
	{
		OutState.InitializeFromExisting(State.CachedAssets, State.CachedDependsNodes, State.CachedPackageData, Options,
			bRefreshExisting ? FAssetRegistryState::EInitializationMode::OnlyUpdateExisting : FAssetRegistryState::EInitializationMode::Rebuild);
	}
}

void UAssetRegistryImpl::CopyManageDependenciesToState(FAssetRegistryState& InOutState) const
{
	using namespace UE::AssetRegistry;

	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	const FAssetRegistryState& State = GuardedData.GetState();
	TArray<FAssetDependency> Dependencies;
	for (const TPair<FAssetIdentifier, FDependsNode*>& Pair : State.CachedDependsNodes)
	{
		FDependsNode* ManagerNode = Pair.Value;
		Dependencies.Reset();
		ManagerNode->GetDependencies(Dependencies, EDependencyCategory::Manage);
		// Remove dependencies to packages that do not exist in InOutState, unless they are PrimaryAssets. The
		// AssetRegistryGenerator's state does not pre-add the PrimaryAsset nodes, this function needs to add them.
		// AddDependencies will create the node if it does not already exist.
		Dependencies.RemoveAllSwap([&InOutState](const FAssetDependency& AssetDependency)
			{
				if (AssetDependency.AssetId.GetPrimaryAssetId().IsValid())
				{
					return false;
				}
				if (InOutState.CachedPackageData.Contains(AssetDependency.AssetId.PackageName))
				{
					return false;
				}
				return true;
			});
		if (!Dependencies.IsEmpty())
		{
			InOutState.AddDependencies(Pair.Key, Dependencies);
		}
	}
}

#if ASSET_REGISTRY_STATE_DUMPING_ENABLED
void UAssetRegistryImpl::DumpState(const TArray<FString>& Arguments, TArray<FString>& OutPages, int32 LinesPerPage) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	GuardedData.GetState().Dump(Arguments, OutPages, LinesPerPage);
}
#endif

TSet<FName> UAssetRegistryImpl::GetCachedEmptyPackagesCopy() const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	return GuardedData.GetCachedEmptyPackages();
}

bool UAssetRegistryImpl::ContainsTag(FName TagName) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	return GuardedData.GetState().ContainsTag(TagName);
}

namespace UE::AssetRegistry
{

namespace Impl
{

FScanPathContext::FScanPathContext(FEventContext& InEventContext, FClassInheritanceContext& InInheritanceContext, 
	const TArray<FString>& InDirs, const TArray<FString>& InFiles, UE::AssetRegistry::EScanFlags InScanFlags,
	TArray<FSoftObjectPath>* FoundAssets)
	: EventContext(InEventContext)
	, InheritanceContext(InInheritanceContext)
	, OutFoundAssets(FoundAssets)
	, bForceRescan(!!(InScanFlags & UE::AssetRegistry::EScanFlags::ForceRescan))
	, bIgnoreDenyListScanFilters(!!(InScanFlags & UE::AssetRegistry::EScanFlags::IgnoreDenyListScanFilters))
	, bIgnoreInvalidPathWarning(!!(InScanFlags & UE::AssetRegistry::EScanFlags::IgnoreInvalidPathWarning))
{
	if (OutFoundAssets)
	{
		OutFoundAssets->Empty();
	}

	bool bLogCallstack = false;
	ON_SCOPE_EXIT
	{
		if (bLogCallstack)
		{
			FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
		}
	};
	if (bIgnoreDenyListScanFilters && !bForceRescan)
	{
		// This restriction is necessary because we have not yet implemented some of the required behavior to handle bIgnoreDenyListScanFilters without bForceRescan;
		// For skipping of directories that we have already scanned, we would have to check whether the directory has been set to be monitored with the proper flag (ignore deny list or not)
		// rather than just checking whether it has been set to be monitored at all
		UE_LOGF(LogAssetRegistry, Warning, "ScanPathsSynchronous: bIgnoreDenyListScanFilters==true is only valid when bForceRescan==true. Setting bForceRescan=true.");
		bForceRescan = true;
		bLogCallstack = true;
	}

	FString LocalPath;
	FString PackageName;
	FString Extension;
	FPackageName::EFlexNameType FlexNameType;
	LocalFiles.Reserve(InFiles.Num());
	PackageFiles.Reserve(InFiles.Num());
	for (const FString& InFile : InFiles)
	{
		if (InFile.IsEmpty())
		{
			continue;
		}
		else if (!FPackageName::TryConvertToMountedPath(InFile, &LocalPath, &PackageName, nullptr, nullptr, &Extension, &FlexNameType))
		{
			if (!bIgnoreInvalidPathWarning)
			{
				UE_LOGF(LogAssetRegistry, Warning, "ScanPathsSynchronous: %ls is not in a mounted path, will not scan.", *InFile);
				bLogCallstack = true;
			}
			continue;
		}
		if (FPackageName::IsTempPackage(PackageName))
		{
			if (!bIgnoreInvalidPathWarning)
			{
				UE_LOGF(LogAssetRegistry, Warning, "ScanPathsSynchronous: %ls is in the /Temp path, will not scan.", *InFile);
				bLogCallstack = true;
			}
			continue;
		}
		if (Extension.IsEmpty())
		{
			// The empty extension is not a valid Package extension; it might exist, but we will pay the price to check it
			if (!IFileManager::Get().FileExists(*LocalPath))
			{
				// Find the extension
				// Note, the 'internal' version of DoesPackageExist must be used to avoid re-entering the AssetRegistry's lock resulting in deadlock
				FPackagePath PackagePath = FPackagePath::FromLocalPath(LocalPath);
				if (FPackageName::InternalDoesPackageExistEx(PackagePath, FPackageName::EPackageLocationFilter::Any,
					false /* bMatchCaseOnDisk */, &PackagePath) == FPackageName::EPackageLocationFilter::None)
				{
					// Requesting to scan a non-existent package is not a condition we need to warn about, because it rarely indicates an error,
					// and is often used to check whether a package exists in the state before the scan has finished. Silently ignore it,
					// even if !bIgnoreInvalidPathWarning.
					continue;
				}
				Extension = PackagePath.GetExtensionString(EPackageSegment::Header);
			}
		}
		LocalFiles.Add(LocalPath + Extension);
		PackageFiles.Add(PackageName);
	}
	LocalDirs.Reserve(InDirs.Num());
	PackageDirs.Reserve(InDirs.Num());
	for (const FString& InDir : InDirs)
	{
		if (InDir.IsEmpty())
		{
			continue;
		}
		else if (!FPackageName::TryConvertToMountedPath(InDir, &LocalPath, &PackageName, nullptr, nullptr, &Extension, &FlexNameType))
		{
			if (!bIgnoreInvalidPathWarning)
			{
				UE_LOGF(LogAssetRegistry, Warning, "ScanPathsSynchronous: %ls is not in a mounted path, will not scan.", *InDir);
				bLogCallstack = true;
			}
			continue;
		}
		if (FPackageName::IsTempPackage(PackageName))
		{
			if (!bIgnoreInvalidPathWarning)
			{
				UE_LOGF(LogAssetRegistry, Warning, "ScanPathsSynchronous: %ls is in the /Temp path, will not scan.", *InDir);
				bLogCallstack = true;
			}
			continue;
		}
		LocalDirs.Add(LocalPath + Extension);
		PackageDirs.Add(PackageName + Extension);
	}
}

}

void FAssetRegistryImpl::ScanPathsSynchronous(FInterfaceWriteScopeLock* ScopeLock, Impl::FScanPathContext& Context)
{
	// We always expect a valid lock to be provided except before initialization is complete.
	check(ScopeLock || !bInitializationComplete);
	LLM_SCOPE(ELLMTag::AssetRegistry);

	if (!TryConstructGathererIfNeeded())
	{
		return;
	}
	FAssetDataGatherer& Gatherer = *GlobalGatherer;

	Context.LocalPaths.Reserve(Context.LocalFiles.Num() + Context.LocalDirs.Num());
	Context.LocalPaths.Append(MoveTemp(Context.LocalDirs));
	Context.LocalPaths.Append(MoveTemp(Context.LocalFiles));
	if (Context.LocalPaths.IsEmpty())
	{
		return;
	}
	Gatherer.AddRequiredMountPoints(Context.LocalPaths);

	auto IsInRequestedDir = [&Context](const FAssetData& AssetData)
		{
			TStringBuilder<NAME_SIZE> PackageNameStr;
			AssetData.PackageName.ToString(PackageNameStr);
			FStringView PackageName(PackageNameStr.ToString(), PackageNameStr.Len());

			// First see if the asset is in a requested directory
			for (const FString& RequestedPackageDir : Context.PackageDirs)
			{
				if (FPathViews::IsParentPathOf(RequestedPackageDir, PackageName))
				{
					return true;
				}
			}

			// If not found above, check if the asset was requested specifically
			return Context.PackageFiles.ContainsByHash(GetTypeHash(PackageName), PackageName);
		};

	TSet<FSoftObjectPath> GatheredAssets;
	TSet<FName> GatheredVerseFiles;
	FDelegateHandle GatheredResultsHandle;
	if (Context.bForceRescan)
	{
		GatheredResultsHandle = Gatherer.GetGatheredResultsEvent().AddLambda(
			[&GatheredAssets, &GatheredVerseFiles, &IsInRequestedDir, OutFoundAssets = Context.OutFoundAssets](const UE::AssetDataGather::FResults& Results)
			{
				auto AddGatheredAssets = [&IsInRequestedDir, OutFoundAssets](TSet<FSoftObjectPath>& InOutGatheredAssets, const TMultiMap<FName, TUniquePtr<FAssetData>>& AssetMap)
					{
						for (TMultiMap<FName, TUniquePtr<FAssetData>>::TConstIterator It = AssetMap.CreateConstIterator(); It; ++It)
						{
							const FAssetData* AssetData = It.Value().Get();
							if (OutFoundAssets)
							{
								// The gatherer may have added other assets that were scanned as part of the ongoing background scan,
								// so remove any assets that were not in the requested paths
								if (IsInRequestedDir(*AssetData))
								{
									UE_LOGF(LogAssetRegistry, VeryVerbose, "FAssetRegistryImpl::ScanPathsSynchronous: Found Asset: %ls",
										*AssetData->GetObjectPathString());
									OutFoundAssets->Add(AssetData->GetSoftObjectPath());
								}
							}

							InOutGatheredAssets.Add(AssetData->GetSoftObjectPath());
						}
					};
				AddGatheredAssets(GatheredAssets, Results.Assets);
				AddGatheredAssets(GatheredAssets, Results.AssetsForGameThread);

				for (FName VerseFile : Results.VerseFiles)
				{
					GatheredVerseFiles.Add(VerseFile);
				}
			});
	}
	ON_SCOPE_EXIT
	{
		if (Context.bForceRescan)
		{
			Gatherer.GetGatheredResultsEvent().Remove(GatheredResultsHandle);
		}
	};

	// Conditionally release the lock while performing expensive calls that don't need the lock. 
	// If no lock was passed we assume the caller does not want us to unlock.
	if (ScopeLock)
	{
		ScopeLock->Lock.WriteUnlock();
	}
	Gatherer.ScanPathsSynchronous(Context.LocalPaths, Context.bForceRescan, Context.bIgnoreDenyListScanFilters);
	if (ScopeLock)
	{
		ScopeLock->Lock.WriteLock();
		Context.InheritanceContext.OnLockReentered(*this);
	}

	// If we are forcing a rescan, then delete any old assets that no longer exist. If we are not forcing a rescan,
	// then there should not be any old assets that no longer exist, so we skip the cost of searching for them.
	TSet<FSoftObjectPath> ExistingAssets;
	TSet<FName> ExistingVerseFiles;
	if (Context.bForceRescan)
	{
		// Initialize ExistingAssets to the list of all assets in the given paths.
		if (!Context.PackageDirs.IsEmpty())
		{
			FARFilter Filter;
			Filter.bIncludeOnlyOnDiskAssets = true;
			Filter.bRecursivePaths = true;
			for (const FString& PackageDir : Context.PackageDirs)
			{
				Filter.PackagePaths.Add(FName(*PackageDir));
			}
			FARCompiledFilter CompiledFilter;
			CompileFilter(Context.InheritanceContext, Filter, CompiledFilter);
			TArray<FAssetData> AssetsInPaths;
			State.EnumerateAssets(CompiledFilter, TSet<FName>() /* PackageNamesToSkip */,
				[&ExistingAssets](const FAssetData& AssetData)
				{
					ExistingAssets.Add(AssetData.ToSoftObjectPath());
					return true;
				}, UE::AssetRegistry::EEnumerateAssetsFlags::AllowUnfilteredArAssets);
			for (FName PackagePath : CompiledFilter.PackagePaths)
			{
				TArray<FName>* VerseFiles = CachedVerseFilesByPath.Find(PackagePath);
				if (VerseFiles)
				{
					ExistingVerseFiles.Append(*VerseFiles);
				}
			}
		}
		for (const FString& PackageName : Context.PackageFiles)
		{
			State.EnumerateAssetsByPackageName(FName(*PackageName),
				[&ExistingAssets](const FAssetData* AssetData)
				{
					ExistingAssets.Add(AssetData->ToSoftObjectPath());
					return true;
				});
			for (const TCHAR* Extension : FAssetDataGatherer::GetVerseFileExtensions())
			{
				FName VerseName(*WriteToString<256>(PackageName, Extension), FNAME_Find);
				if (!VerseName.IsNone() && CachedVerseFiles.Contains(VerseName))
				{
					ExistingVerseFiles.Add(VerseName);
				}
			}
		}
	}

	Impl::FTickContext TickContext(*this, Context.EventContext, Context.InheritanceContext);
	Context.Status = TickGatherer(TickContext);

	// Temporary hack/partial solution. The expectation is that this function will return cause all assets
	// under the specified directories to be ingested into the registry. However, one of the early steps
	// in ingestion is an attempt to PostLoadAssetRegistryTags. This step requires that we already have loaded
	// the AssetClass UClass for an asset. That may not have happened yet. In the past, we would just have skipped
	// over that step and continued, but now we defer the asset for processing at a later time. However, that means
	// that after running TickGatherer, even without timeslicing, our end state might be that only some assets have
	// been scanned and others have been deferred and so would be unavailable to subsequent queries. Ideally we would
	// solve this by loading the classes that these assets depend on. Instead, we are deferring that task and for now
	// we manually identify any deferred assets that fall under the paths we are scanning and ask the asset registry
	// to process them ignoring any failures of TryPostLoadAssetRegistryTags. We then run a second full Tick to finish
	// out their processing. See UE-210249 for the desired fix.
	TArray<FName> FoundAssetPackageNames;
	{
		// Find any assets that were deferred but fall into the paths we are interested in. Extract them from 
		// the DeferredAssets and DeferredAssetsForGameThread containers

		TMultiMap<FName, TUniquePtr<FAssetData>> CollectedDeferredAssets;
		for (auto Iter = DeferredAssets.CreateIterator(); Iter; ++Iter)
		{
			if (IsInRequestedDir(*Iter.Value()))
			{
				FoundAssetPackageNames.Add(Iter.Key());
				CollectedDeferredAssets.Add(MoveTemp(*Iter));
				Iter.RemoveCurrent();
			}
		}
		for (auto Iter = DeferredAssetsForGameThread.CreateIterator(); Iter; ++Iter)
		{
			if (IsInRequestedDir(*Iter.Value()))
			{
				FoundAssetPackageNames.Add(Iter.Key());
				CollectedDeferredAssets.Add(MoveTemp(*Iter));
				Iter.RemoveCurrent();
			}
		}

		int32 OriginalNumDeferredAssetsForGameThread = DeferredAssetsForGameThread.Num();
		TOptional<TSet<FString>> MountPointsForVerifyAfterGather;

		// We don't call AssetsFoundCallback because even for deferred assets it will already have been called.
		// We pass DeferredAssetsForGameThread as the OutDeferred parameter, but we expect nothing is deferred.
		bool bIsInGameThread = IsInGameThread();
		{
			// Force AssetSearchDataGathered to process these assets, skipping the PostLoadAssetRegistryTags if needed
			TGuardValue<bool> ScopedForceCompletionEventIfClassNotLoaded(bForceCompletionEvenIfClassNotLoaded, true);
			TGuardValue<bool> ScopedForceCompletionEvenIfNotOnGameThread(bForceCompletionEvenIfNotOnGameThread, true);

			AssetSearchDataGathered(Context.EventContext, CollectedDeferredAssets, DeferredAssetsForGameThread,
				TickContext.InterruptionContext, MountPointsForVerifyAfterGather,
				bIsInGameThread ? EDeferFlag::CanExecuteGameThread : EDeferFlag::BypassNow);
		}
		// All of the assets we collected should have been processed or deferred.
		ensure(CollectedDeferredAssets.Num() == 0);

		// We should not have deferred any new assets because we set the bForceCompletionEvenIf flags to true.
		ensure(DeferredAssetsForGameThread.Num() <= OriginalNumDeferredAssetsForGameThread);

		// Tick to perform any subsequent processing required for these assets beyond AssetSearchDataGathered
		Impl::FTickContext AssetTickContext(*this, Context.EventContext, Context.InheritanceContext);
		Context.Status = TickGatherer(AssetTickContext);
	}
	FoundAssetPackageNames.Sort(FNameFastLess());
	FoundAssetPackageNames.SetNum(Algo::Unique(FoundAssetPackageNames));

#if WITH_EDITOR
	LoadCalculatedDependencies(&FoundAssetPackageNames, Context.InheritanceContext,
		&PackagesNeedingDependencyCalculation, TickContext.InterruptionContext);
	LoadCalculatedDependencies(&FoundAssetPackageNames, Context.InheritanceContext,
		&PackagesNeedingDependencyCalculationOnGameThread, TickContext.InterruptionContext);
#endif

	TSet<FSoftObjectPath> OldAssetsToRemove = ExistingAssets.Difference(GatheredAssets);
	for (FSoftObjectPath& OldAssetToRemove : OldAssetsToRemove)
	{
		FAssetData* AssetDataToRemove = State.GetMutableAssetByObjectPath(OldAssetToRemove);
		if (AssetDataToRemove)
		{
			RemoveAssetData(Context.EventContext, AssetDataToRemove);
		}
	}
	
	TSet<FName> OldVerseFilesToRemove = ExistingVerseFiles.Difference(GatheredVerseFiles);
	for (FName OldVerseFileToRemove : OldVerseFilesToRemove)
	{
		RemoveVerseFile(Context.EventContext, OldVerseFileToRemove);
	}
}

namespace Utils
{

void InitializeMountPoints(TOptional<TSet<FString>>& MountPoints)
{
	if (MountPoints.IsSet())
	{
		return;
	}
	MountPoints.Emplace();
	TArray<FString> MountPointsArray;
	FPackageName::QueryRootContentPaths(MountPointsArray, /*bIncludeReadOnlyRoots=*/ true, /*bWithoutLeadingSlashes*/ false, /*WithoutTrailingSlashes=*/ true);
	MountPoints->Append(MoveTemp(MountPointsArray));
}

bool IsPathMounted(const FString& Path, const TSet<FString>& MountPointsNoTrailingSlashes, FString& StringBuffer)
{
	const int32 SecondSlash = Path.Len() > 1 ? Path.Find(TEXT("/"), ESearchCase::CaseSensitive, ESearchDir::FromStart, 1) : INDEX_NONE;
	if (SecondSlash != INDEX_NONE)
	{
		StringBuffer.Reset(SecondSlash);
		StringBuffer.Append(*Path, SecondSlash);
		if (MountPointsNoTrailingSlashes.Contains(StringBuffer))
		{
			return true;
		}
	}
	else
	{
		if (MountPointsNoTrailingSlashes.Contains(Path))
		{
			return true;
		}
	}

	return false;
}

}

#if WITH_EDITOR

FAssetData* FAssetRegistryImpl::ResolveAssetIdCollision(FAssetData& A, FAssetData& B)
{
	FAssetData* ResolvedAsset = nullptr;
	AssetCollisionEvent.Broadcast(A, B, ResolvedAsset);
	check(!ResolvedAsset || ResolvedAsset == &A || ResolvedAsset == &B);

	// We could use file age to try to guess which file is correct:
	// FPackageName::InternalDoesPackageExistEx() to get the filename, and IFileManager::GetFileAgeSeconds
	// But that would vary from machine to machine based on when the files were synced.
	// So instead just pick one using an arbitrary deterministic process: alphabetical order
	FAssetData* Keep = ResolvedAsset ? ResolvedAsset : (A.PackageName.LexicalLess(B.PackageName) ? &A : &B);
	FAssetData* Discard = Keep == &A ? &B : &A;

	FString PackageNameB = B.PackageName.ToString();
	FString FileNameA;
	FString FileNameB;
	UE_LOGF(LogAssetRegistry, Warning,
		"Invalid duplicate copies of ExternalActor %ls. Resolve by deleting the package that is invalid. Chosing alphabetically for this process."
		"\n\tDiscarding: %ls"
		"\n\tKeeping:    %ls",
		*Keep->GetObjectPathString(),
		*Discard->PackageName.ToString(),
		*Keep->PackageName.ToString());

	return Keep;
}

bool FAssetRegistryImpl::TryPostLoadAssetRegistryTags(FAssetData* AssetData, EDeferFlag DeferFlag,
	FTopLevelAssetPath& OutRequestedGameThreadClass)
{
	check(AssetData);
	OutRequestedGameThreadClass = FTopLevelAssetPath();
	if (!AssetData->TagsAndValues.Num())
	{
		return true;
	}

	// bForceCompletionEvenIfClassNotLoaded can be set in some cases even before engine startup is complete, for code
	// that needs to access the assets immediately (e.g. ScanPathsSynchronous). Silently complete those cases. But if
	// we are forcing completion because engine loading is complete, then log a warning for them.
	bool bLogMissingClassWarnings = bPreloadingComplete && IsEngineStartupModuleLoadingComplete();
	UClass* AssetClass = nullptr;
	FTopLevelAssetPath AssetClassPath;
	AssetClassPath = AssetData->AssetClassPath;
	AssetClass = FindObject<UClass>(AssetClassPath, EFindObjectFlags::ExactClass);

	int32 CycleCount = 0;
	// We want to avoid the expense of detecting the cycle for the common case.
	// CycleCountBeforeCycleDetection is Tuned for the maximum depth encountered on a large project.
	constexpr int32 CycleCountBeforeCycleDetection = 10;
	TOptional<TSet<FTopLevelAssetPath>> CycleSet;
	TOptional<TArray<FTopLevelAssetPath>> CycleArray;
	while (!AssetClass)
	{
		if (++CycleCount > CycleCountBeforeCycleDetection)
		{
			if (!CycleSet)
			{
				CycleSet.Emplace();
				CycleArray.Emplace();
			}
			bool bExists;
			CycleSet->Add(AssetClassPath, &bExists);
			CycleArray->Add(AssetClassPath);
			if (bExists)
			{
				TStringBuilder<256> CycleText;
				for (const FTopLevelAssetPath& CyclePath : *CycleArray)
				{
					CycleText << CyclePath.ToString() << TEXT(", ");
				}
				UE_LOGF(LogAssetRegistry, Warning,
					"Cycle detected in ParentClasses, ObjectRedirectors, or CoreRedirects, starting from AssetClassPath %ls. Skipping AssetRegistry normalization of assets with that class." "\n\tCycle: { %ls }",
					*AssetData->AssetClassPath.ToString(), *CycleText);

				// We have given the relevant warning, there is nothing further we need to do with this class,
				// so mark the postload completed.
				return true;
			}
		}
		// this is probably a blueprint that has not yet been loaded, try to find its native base class
		const FTopLevelAssetPath* ParentClassPath = CachedBPInheritanceMap.Find(AssetClassPath);
		if (ParentClassPath && !ParentClassPath->IsNull())
		{
			AssetClassPath = *ParentClassPath;
			AssetClass = FindObject<UClass>(AssetClassPath, EFindObjectFlags::ExactClass);
		}
		else
		{
			FTopLevelAssetPath LastAssetClassPath = AssetClassPath;
			// Maybe it's a redirector
			FSoftObjectPath RedirectedPath = GRedirectCollector.GetAssetPathRedirection(FSoftObjectPath::ConstructFromAssetPath(AssetClassPath));
			if (RedirectedPath.IsValid())
			{
				AssetClassPath = RedirectedPath.GetAssetPath();
			}
			else
			{
				FCoreRedirectObjectName NewName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, FCoreRedirectObjectName(AssetClassPath));
				if (NewName.IsValid())
				{
					AssetClassPath = NewName.ToString();
				}
			}

			if (AssetClassPath != LastAssetClassPath && !AssetClassPath.IsNull())
			{
				AssetClass = FindObject<UClass>(AssetClassPath, EFindObjectFlags::ExactClass);
			}
			else
			{
				break;
			}
		}
	}

	if (!AssetClass)
	{
		if (!bForceCompletionEvenIfClassNotLoaded)
		{
			// We need to wait for the class to load, return false until we are forced to complete.
			return false;
		}

		// Log a warning and mark the postload completed.
		if (bLogMissingClassWarnings)
		{
			FString Reason;
			if (AssetClassPath.ToString().StartsWith(TEXT("/Script/")))
			{
				Reason = TEXT("The missing class is native--perhaps a CoreRedirector is missing?");
			}
			else
			{
				if (State.GetAssetPackageData(AssetClassPath.GetPackageName()) == nullptr)
				{
					Reason = TEXT("The class is missing on disk or could not be loaded. Perhaps it has been deleted from perforce and the referencing object is broken?");
				}
			}
			//@TODO this should become a Warning once UE-209846 is finished
			UE_LOGF(LogAssetRegistry, Verbose,
				"Unable to PostLoadAssetRegistryTags for '%ls' because ancestor class '%ls' cannot be found. %ls", 
				*AssetData->GetObjectPathString(), *AssetClassPath.ToString(), *Reason);
		}
		return true;
	}

	// Now identify the most derived native class in the class hierarchy
	while (!AssetClass->HasAnyClassFlags(CLASS_Native))
	{
		AssetClass = AssetClass->GetSuperClass();
	}

	// Use the class's CDO to run the virtual functions in ThreadedPostLoadAssetRegistryTags
	UObject* ClassDefaultObject = AssetClass->GetDefaultObject(false);
	if (ClassDefaultObject)
	{
		// It is only safe to read flags from the CDO if we have a barrier between the time where the default object
		// pointer is published, and the time we read the flag. Otherwise, we could end up reading a stale flag that
		// hasn't yet been set to RF_NeedInitialization. It needs to be paired with a release barrier when the default
		// object is set.
		std::atomic_thread_fence(std::memory_order_acquire);
		TSAN_AFTER(ClassDefaultObject); // TSAN doesn't understand fence yet, help it understand what's going on

		if (ClassDefaultObject->HasAnyFlags(RF_NeedInitialization))
		{
			ClassDefaultObject = nullptr;
		}
	}

	if (!ClassDefaultObject)
	{
		if (!bForceCompletionEvenIfClassNotLoaded)
		{
			// Continue waiting until the CDO is available
			return false;
		}
		
		// Log a warning and mark the postload completed.
		if (bLogMissingClassWarnings)
		{
			ensureMsgf(false,
				TEXT("Unable to PostLoadAssetRegistryTags for '%s' because the CDO for ancestor class '%s' could not be found or was not ready."),
				*AssetData->GetObjectPathString(), *AssetClassPath.ToString());
		}

		return true;
	}

	// We are using RF_NeedInitialization to guarantee that ClassDefaultObject is fully initialized
	// potentially on another thread. For weakly ordered memory platforms, we need to 
	// ensure that our read of the vtable ptr isn't performed prior to the read of the class flags
	// otherwise we might see a stale vtable despite seeing RF_NeedInit clear.
	std::atomic_thread_fence(std::memory_order_acquire);
	TSAN_AFTER(ClassDefaultObject); // TSAN doesn't understand fence yet, help it understand what's going on

	TArray<UObject::FAssetRegistryTag> TagsToModify;
	bool bCanCreateObjects = DeferFlag == EDeferFlag::CanExecuteGameThread;
	UObject::FPostLoadAssetRegistryTagsContext Context(*AssetData, TagsToModify, bCanCreateObjects);
	ClassDefaultObject->ThreadedPostLoadAssetRegistryTags(Context);

	if (!Context.PostLoadCouldComplete())
	{
		// If we are still able to wait on a missing class, then defer this asset before checking whether we need
		// to kick it to game thread. The Gamethread might end up creating the required data for us if we
		// continue waiting.
		if (!bForceCompletionEvenIfClassNotLoaded)
		{
			// Continue waiting until the CDO is available
			return false;
		}

		if (!bCanCreateObjects)
		{
			// Assume that we could not complete because we need bCanCreateObjects=true.
			if (!bForceCompletionEvenIfNotOnGameThread && DeferFlag != EDeferFlag::BypassNow)
			{
				// Request completion on the game thread.
				OutRequestedGameThreadClass = AssetClass->GetClassPathName();
				return false;
			}

			// Log a warning and complete the PostLoad
			UE_LOGF(LogAssetRegistry, Display,
				"Unable to PostLoadAssetRegistryTags for '%ls' because class '%ls' ThreadedPostLoadAssetRegistryTags requires GameThread to load its CDO, but we need it immediately when off the game thread.",
				*AssetData->GetObjectPathString(), *AssetClass->GetPathName());
			return true;
		}
		else
		{
			// We're on the gamethread, so we can't complete because of something else. Treat it the same as CDO not ready.
			// Log a warning and complete the postload.
			if (bLogMissingClassWarnings)
			{
				UE_LOGF(LogAssetRegistry, Display,
					"Unable to PostLoadAssetRegistryTags for '%ls' because class '%ls' ThreadedPostLoadAssetRegistryTags returned !PostLoadCouldComplete.",
					*AssetData->GetObjectPathString(), *AssetClass->GetPathName());
			}

			return true;
		}
	}

	if (TagsToModify.Num())
	{
		FAssetDataTagMap TagsAndValues = AssetData->TagsAndValues.CopyMap();
		for (const UObject::FAssetRegistryTag& Tag : TagsToModify)
		{
			if (!Tag.Value.IsEmpty())
			{
				FString& Value = TagsAndValues.FindOrAdd(Tag.Name);
				Value = Tag.Value;
			}
			else
			{
				TagsAndValues.Remove(Tag.Name);
			}
		}
		AssetData->TagsAndValues = FAssetDataTagMapSharedView(MoveTemp(TagsAndValues));
	}
	return true;
}
#endif

bool FAssetRegistryImpl::ShouldSkipGatheredAsset(FAssetData& AssetData)
{
	// TODO: This pruning of invalid ExternalActors is temporary, to handle the fallout from a bug in SaveAs
	// that is keeping the old ExternalActors as duplicates of the new ones. Remove it after the data has been
	// cleaned up for all affected licensees. If we need such validation permanently, it should be decoupled
	// from the AssetRegistry by adding a delegate.
	// Extra validation for ExternalActors. If duplicate ExternalActors with the same object path exist
	// then we intermittently will fail to find the correct one and WorldPartition will break.
	// Validate that the PackageName matches what is expected from the ObjectPath.

#if WITH_EDITORONLY_DATA
	if (AssetData.GetOptionalOuterPathName().IsNone())
	{
		// If no outer path, this can't be an external asset
		return false;
	}
#endif

	FStringView ExternalActorsFolderName(FPackagePath::GetExternalActorsFolderName());
	TStringBuilder<256> PackageNameStr;
	AssetData.PackageName.ToString(PackageNameStr);
	if (UE::String::FindFirst(PackageNameStr, ExternalActorsFolderName) != INDEX_NONE)
	{
		TStringBuilder<256> ObjectPathString;
		AssetData.AppendObjectPath(ObjectPathString);
		FStringView ObjectPathPackageName = FPackageName::ObjectPathToPackageName(ObjectPathString);
		FStringView PackageNamePackageRoot;
		FStringView PackageNameRelPath;
		FStringView ObjectPathPackageRoot;
		FStringView ObjectPathRelPath;

		// /PackageRoot/__ExternalActors__/RelPathFromPackageRootToMap/#/##/#######
		// OR
		// /PackageRoot/__ExternalActors__/ContentBundle/######/RelPathFromPackageRootToMap/#/##/#######
		// OR
		// /PackageRoot/__ExternalActors__/EDL/######/ObjectPathPackageRoot/RelPathFromPackageRootToMap/#/##/#######
		// Package roots do not need to be the same; ContentBundles can be injected into /Game maps from plugins
		PackageNamePackageRoot = FPackageName::SplitPackageNameRoot(PackageNameStr, &PackageNameRelPath);
		ObjectPathPackageRoot = FPackageName::SplitPackageNameRoot(ObjectPathPackageName, &ObjectPathRelPath);

		if (!PackageNameRelPath.StartsWith(ExternalActorsFolderName) || !PackageNameRelPath.RightChop(ExternalActorsFolderName.Len()).StartsWith(TEXT("/")))
		{
			UE_LOGF(LogAssetRegistry, Verbose,
				"Invalid ExternalActor: Package %ls is an ExternalActor package but is not in the expected root path for ExternalActors /%.*ls/%.*ls. Ignoring this actor.",
				*PackageNameStr, PackageNamePackageRoot.Len(), PackageNamePackageRoot.GetData(),
				ExternalActorsFolderName.Len(), ExternalActorsFolderName.GetData());
			return true;
		}

		bool bIsEDLActor = false;
		bool bIsPluginActor = false;
		FStringView PackageNameRelPathAfterExternalActorRoot = PackageNameRelPath.RightChop(ExternalActorsFolderName.Len() + 1);
		FStringView ContentBundleDirName(TEXTVIEW("ContentBundle"));
		FStringView ExternalDataLayerDirName(TEXTVIEW("EDL"));
		if (PackageNameRelPathAfterExternalActorRoot.StartsWith(ContentBundleDirName))
		{
			PackageNameRelPathAfterExternalActorRoot.RightChopInline(ContentBundleDirName.Len());
			bIsPluginActor = true;
		}
		else if (PackageNameRelPathAfterExternalActorRoot.StartsWith(ExternalDataLayerDirName))
		{
			PackageNameRelPathAfterExternalActorRoot.RightChopInline(ExternalDataLayerDirName.Len());
			bIsEDLActor = true;
			bIsPluginActor = true;
		}

		bool bAllowValidation = true;
		if (bIsPluginActor)
		{
			bAllowValidation = false; // Don't allow validation unless we succeed in finding the new relpath
			if (PackageNameRelPathAfterExternalActorRoot.StartsWith(TEXT("/")))
			{
				PackageNameRelPathAfterExternalActorRoot.RightChopInline(1);
				int32 NextSlash;
				PackageNameRelPathAfterExternalActorRoot.FindChar('/', NextSlash);
				if (NextSlash != INDEX_NONE)
				{
					PackageNameRelPathAfterExternalActorRoot.RightChopInline(NextSlash + 1);
					// EDL path keeps ObjectPathPackageRoot
					if (bIsEDLActor)
					{
						if (PackageNameRelPathAfterExternalActorRoot.StartsWith(ObjectPathPackageRoot))
						{
							PackageNameRelPathAfterExternalActorRoot.RightChopInline(ObjectPathPackageRoot.Len());
							if (PackageNameRelPathAfterExternalActorRoot.StartsWith(TEXT("/")))
							{
								PackageNameRelPathAfterExternalActorRoot.RightChopInline(1);
								bAllowValidation = true;
							}
						}
					}
					else
					{
						bAllowValidation = true;
					}
				}
			}
		}

		if (bAllowValidation && !PackageNameRelPathAfterExternalActorRoot.StartsWith(ObjectPathRelPath))
		{
			TStringBuilder<256> ExpectedPath;
			ExpectedPath << "/" << ObjectPathPackageRoot << "/" << ExternalActorsFolderName << "/" << ObjectPathRelPath;
			UE_LOGF(LogAssetRegistry, Verbose,
				"Invalid ExternalActor: Package %ls is an ExternalActor package but its path does not match the expected path %ls created from its objectpath %ls. Ignoring this actor.",
				*PackageNameStr, *ExpectedPath, *ObjectPathString);
			return true;
		}
	}
	return false;
}

void FAssetRegistryImpl::AssetSearchDataGathered(Impl::FEventContext& EventContext,
	TMultiMap<FName, TUniquePtr<FAssetData>>& AssetResults,
	TMultiMap<FName, TUniquePtr<FAssetData>>& OutDeferredAssetResults,
	Impl::FInterruptionContext& InOutInterruptionContext,
	TOptional<TSet<FString>>& MountPointsForVerifyAfterGather,
	EDeferFlag DeferFlag)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AssetSearchDataGathered);

	// Refreshes ClassGeneratorNames if out of date due to module load
	CollectCodeGeneratorClasses();

	FString PackagePathString;
	FString PackageRoot;
	if (AssetResults.Num() > 0 && bVerifyMountPointAfterGather)
	{
		Utils::InitializeMountPoints(MountPointsForVerifyAfterGather);
	}

	TSet<FTopLevelAssetPath> MissingClasses;
	bool bInterrupted = false;
	int64 IterationCounter = 0;

	// Add the found assets
	for (TMultiMap<FName, TUniquePtr<FAssetData>>::TIterator Iter(AssetResults); Iter && !bInterrupted; ++Iter)
	{
		ON_SCOPE_EXIT
		{
			// ShouldExitEarly calls FPlatformTime::Seconds which isn't super cheap
			// Since we can spin very quickly in this loop, avoid checking every single iteration
			if ((++IterationCounter % 10) == 0)
			{
				// Check to see if we have run out of time in this tick
				bInterrupted = InOutInterruptionContext.ShouldExitEarly();
			}
		};

		// Delete or take ownership of the BackgroundResult; it was originally new'd by an FPackageReader
		TUniquePtr<FAssetData> BackgroundResult(MoveTemp(Iter.Value()));
		FName BackgroundAssetPackageName = Iter.Key();
		CA_ASSUME(BackgroundResult.Get() != nullptr);
		Iter.RemoveCurrent();

		// Skip assets that are invalid because e.g. they are externalactors that were mistakenly not deleted
		// when their map moved.
		if (ShouldSkipGatheredAsset(*BackgroundResult))
		{
			continue;
		}

		// Skip stale gather results from unmounted roots caused by mount then unmount of a path within short period.
		const FName PackagePath = BackgroundResult->PackagePath;
		if (bVerifyMountPointAfterGather)
		{
			PackagePath.ToString(PackagePathString);
			if (!Utils::IsPathMounted(PackagePathString, *MountPointsForVerifyAfterGather, PackageRoot))
			{
				UE_LOGF(LogAssetRegistry, Display,
					"AssetRegistry: An asset has been loaded with an invalid mount point: '%ls', Mount Point: '%ls'. Ignoring the asset.",
					*BackgroundResult->GetObjectPathString(), *PackagePathString);
				continue;
			}
		}

#if WITH_EDITOR
		bool bCouldPostLoad = false;
		FTopLevelAssetPath RequestedGameThreadClass;
		{
			// Take the FGCScopeGuard for as short a duration as possible to avoid blocking 
			// the game thread for long if it needs to collect garbage. The GCLock doesn't 
			// have a means to signal to our interruption context like the InterfaceLock can.
			FGCScopeGuard Guard;
			bCouldPostLoad = TryPostLoadAssetRegistryTags(BackgroundResult.Get(), DeferFlag, RequestedGameThreadClass);
		}
		if (!bCouldPostLoad)
		{
			if (RequestedGameThreadClass.IsValid() && DeferFlag == EDeferFlag::QueueToGameThread)
			{
				BackgroundResults.AssetsForGameThread.Add(BackgroundAssetPackageName, MoveTemp(BackgroundResult));
				RequestGameThreadProcessClass(RequestedGameThreadClass);
			}
			else
			{
				OutDeferredAssetResults.Add(BackgroundAssetPackageName, MoveTemp(BackgroundResult));
			}
			continue;
		}
#endif
		bProcessedAnyAssetsAfterRetryDeferred = true;

		// Look for an existing asset to check whether we need to add or update
		FCachedAssetKey Key(*BackgroundResult);
		FAssetData* ExistingAssetData = State.GetMutableAssetByObjectPath(Key);
		// The background result should not already be registered; it should be impossible since it is in TUnqiuePtr
		check(ExistingAssetData == nullptr || ExistingAssetData != BackgroundResult.Get());

#if WITH_EDITOR
		if (ExistingAssetData && ExistingAssetData->PackageName != BackgroundResult->PackageName)
		{
			// This can happen with ExternalActors, which have a Key based on their outermost map, but 
			// are in a separate package. It's invalid to have more than one of them, but can happen when
			// actors are moved between packages if the delete is not recorded.
			FAssetData* PackageToKeep = ResolveAssetIdCollision(*ExistingAssetData, *BackgroundResult);
			if (PackageToKeep == ExistingAssetData)
			{
				continue;
			}
			else
			{
				check(PackageToKeep == BackgroundResult.Get());
				RemoveAssetData(EventContext, ExistingAssetData);
				ExistingAssetData = nullptr;
			}
		}
#endif

		if (ExistingAssetData)
		{
#if WITH_EDITOR
			if (IsLoadedAndTrackedAsset(BackgroundResult->GetSoftObjectPath()))
			{
				// If the current AssetData came from a loaded asset, don't overwrite it with the new one from disk
				// The loaded asset is more authoritative because it has run the postload steps.
				// However, the loaded asset is missing the extended tags. Our contract for extended tags is to keep any 
				// that do not exist in the non-extended tags. So add on any tags from the BackgroundResult that
				// are not already on the existing asset.
				AddNonOverlappingTags(EventContext, *ExistingAssetData, *BackgroundResult);
			}
			else
#endif
			{
				// The asset exists in the cache from disk and has not yet been loaded into memory, update it with the new background data
				UpdateAssetData(EventContext, ExistingAssetData, MoveTemp(*BackgroundResult), false /* bKeepDeletedTags */);
			}
		}
		else
		{
			// The asset isn't in the cache yet, add it and notify subscribers
#if !NO_LOGGING
			if (bVerboseLogging)
			{
				int64& ClassTagSizes = TagSizeByClass.FindOrAdd(BackgroundResult->AssetClassPath);
				BackgroundResult->TagsAndValues.ForEach([&ClassTagSizes](const TPair<FName, FAssetTagValueRef>& Pair)
					{
						ClassTagSizes += Pair.Value.GetResourceSize();
					});
			}
#endif

			AddAssetData(EventContext, BackgroundResult.Release());
		}

		// Populate the path tree
		AddAssetPath(EventContext, PackagePath);
	}
}

void FAssetRegistryImpl::PathDataGathered(Impl::FEventContext& EventContext, TRingBuffer<FString>& PathResults,
	Impl::FInterruptionContext& InOutInterruptionContext, TOptional<TSet<FString>>& MountPointsForVerifyAfterGather)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PathDataGathered);

	FString PackageRoot;
	if (PathResults.Num() > 0 && bVerifyMountPointAfterGather)
	{
		Utils::InitializeMountPoints(MountPointsForVerifyAfterGather);
	}
	
	CachedPathTree.EnsureAdditionalCapacity(PathResults.Num());

	while (PathResults.Num() > 0)
	{
		FString Path = PathResults.PopFrontValue();

		// Skip stale results caused by mount then unmount of a path within short period.
		if (!bVerifyMountPointAfterGather || Utils::IsPathMounted(Path, *MountPointsForVerifyAfterGather, PackageRoot))
		{
			AddAssetPath(EventContext, FName(*Path));
		}
		else
		{
			UE_LOGF(LogAssetRegistry, Display,
				"AssetRegistry: A path has been loaded with an invalid mount point: '%ls', Mount Point: '%ls'. Ignoring the path.",
				*Path, *PackageRoot);
		}

		// Check to see if we have run out of time in this tick
		if (InOutInterruptionContext.ShouldExitEarly())
		{
			return;
		}
	}
}

void FAssetRegistryImpl::DependencyDataGathered(TMultiMap<FName, FPackageDependencyData>& DependsResults,
	TMultiMap<FName, FPackageDependencyData>& OutDeferredDependencyResults,
	TSet<FName>* OutPackagesNeedingDependencyCalculation, Impl::FInterruptionContext& InOutInterruptionContext,
	TOptional<TSet<FString>>& MountPointsForVerifyAfterGather)
{
	using namespace UE::AssetRegistry;
	using namespace UE::AssetRegistry::DependsNode;
	using namespace UE::Tasks;
	TRACE_CPUPROFILER_EVENT_SCOPE(DependencyDataGathered);

	// Don't bother registering dependencies on these packages, every package in the game will depend on them
	static TArray<FName> ScriptPackagesToSkip = TArray<FName>{
		GetScriptPackageNameCoreUObject(),
		GetScriptPackageNameEngine(),
		GetScriptPackageNameBlueprintGraph(),
		GetScriptPackageNameUnrealEd(),
	};

	// If we need to verify mounts, initialize the mounts once before iterating
	FString PackageRoot;
	FString PackageNameString;
	const bool bLocalVerifyMountPointAfterGather = bVerifyMountPointAfterGather;
	if (bLocalVerifyMountPointAfterGather)
	{
		Utils::InitializeMountPoints(MountPointsForVerifyAfterGather);
	}

	TMap<FName, FConstructData>& AssetNameToConstructDataMap = GatherDependencyDataScratchPad->AssetNameToConstructDataMap;
	TMap<FName, FName>& CachedDepToRedirect = GatherDependencyDataScratchPad->CachedDepToRedirect;
	TSet<FDependsNode*>& ModifiedDependNodes = GatherDependencyDataScratchPad->ModifiedDependNodes;
	AssetNameToConstructDataMap.Reserve(DependsResults.Num());
	CachedDepToRedirect.Reserve(DependsResults.Num());
	ModifiedDependNodes.Reserve(DependsResults.Num());
	ON_SCOPE_EXIT{ GatherDependencyDataScratchPad->Reset(); };

	State.CachedPackageData.Reserve(DependsResults.Num());
	State.CachedDependsNodes.Reserve(DependsResults.Num());

	bool bInterrupted = false;
	int32 IterationCounter = 0;
	constexpr int32 MinBatchSize = 1024;
	const bool bUseJobs = DependsResults.Num() > MinBatchSize;
	const EParallelForFlags ParallelForFlags = bUseJobs ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DependencyGatheredData_GatherPackages);
		const bool bShouldSortReferencers = ShouldSortReferencers();
		for (TMultiMap<FName, FPackageDependencyData>::TIterator Iter(DependsResults); Iter && !bInterrupted; ++Iter)
		{
			if (DeferredAssets.Contains(Iter.Key()) || DeferredAssetsForGameThread.Contains(Iter.Key()))
			{
				OutDeferredDependencyResults.Add(MoveTemp(*Iter));
				Iter.RemoveCurrent();
				continue;
			}

			ON_SCOPE_EXIT
			{
				// ShouldExitEarly calls FPlatformTime::Seconds which isn't super cheap
				// Since we can spin very quickly in this loop, avoid checking every single iteration
				if ((++IterationCounter % MinBatchSize) == 0)
				{
					// Check to see if we have run out of time in this tick
					bInterrupted = InOutInterruptionContext.ShouldExitEarly();
				}
			};

			FPackageDependencyData Result = MoveTemp(Iter.Value());
			Iter.RemoveCurrent();

			checkf(!GIsEditor || Result.bHasPackageData, TEXT("We rely on PackageData being read for every gathered Asset in the editor."));
			FName SourceNodePackageName = Result.PackageName;
			if (Result.bHasPackageData)
			{
				// Update package data
				FAssetPackageData* PackageData = State.CreateOrGetAssetPackageData(SourceNodePackageName);
				*PackageData = Result.PackageData;
			}

			// DependsResults might contain multiple values with the same SourcePackageName. 
			// We don't want to create more constructed nodes than necessary, so we create
			// or find existing FConstructData and append results as needed.
			bool bSourceNodeAlreadyExisted = false;
			FConstructData* ConstructData = nullptr;
			if (FConstructData* ExistingConstructData = AssetNameToConstructDataMap.Find(SourceNodePackageName))
			{
				ConstructData = ExistingConstructData;
			}
			else
			{
				if (bLocalVerifyMountPointAfterGather)
				{
					SourceNodePackageName.ToString(PackageNameString);
					if (!Utils::IsPathMounted(PackageNameString, *MountPointsForVerifyAfterGather, PackageRoot))
					{
						UE_LOGF(LogAssetRegistry, Display,
							"AssetRegistry: DependencyData has been loaded with an invalid mount point: '%ls', Mount Point: '%ls'. Ignoring the DependencyData.",
							*PackageNameString, *PackageRoot);
						continue;
					}
				}

				FAssetIdentifier SourceNodeIdentifier(SourceNodePackageName);
				FDependsNode* SourceNode = State.CreateOrFindDependsNode(SourceNodeIdentifier, &bSourceNodeAlreadyExisted);
				if (!bSourceNodeAlreadyExisted)
				{
					if (FPackageDependencyData* DependencyDependencyData = DependsResults.Find(SourceNodePackageName))
					{
						SourceNode->Reserve(DependencyDependencyData->DependsNodeReservations);
					}
				}
				else
				{
					// The node was created by an earlier package registering SourceNode as a dependency, and it may
					// have had a mismatching capitalization that was saved into the package with the dependency.
					// Force the capitalization of the DependsNode for SourceNodePackage to match SourceNodePackage's
					// capitalization (which comes from its filepath on disk) if they are different.
					if (SourceNode->GetPackageName().GetDisplayIndex() != SourceNodePackageName.GetDisplayIndex())
					{
						State.CachedDependsNodes.Remove(SourceNode->GetIdentifier());
						SourceNode->SetIdentifier(SourceNodeIdentifier);
						State.CachedDependsNodes.Add(SourceNode->GetIdentifier(), SourceNode);
					}
				}

				if (Result.bHasDependencyData)
				{
					SourceNode->Reserve(Result.DependsNodeReservations);
					SourceNode->SetAllowShrinking(false);
					SourceNode->SetIsReferencersSorted(bShouldSortReferencers);

					ModifiedDependNodes.Add(SourceNode);

					ConstructData = &AssetNameToConstructDataMap.Add(SourceNodePackageName);
					ConstructData->SourceNode = SourceNode;
				}
			}

			if (ConstructData)
			{
				OutPackagesNeedingDependencyCalculation->Add(SourceNodePackageName);

				FPackageDependencyMap& PackageDependencies = ConstructData->PackageDependencies;
				PackageDependencies.Reserve(Result.PackageDependencies.Num());
				for (const FPackageDependencyData::FPackageDependency& DependencyData : Result.PackageDependencies)
				{
					FName DependencyPackageName = DependencyData.PackageName;
					if (EnumHasAnyFlags(DependencyData.Property, EDependencyProperty::Hard) 
						&& ScriptPackagesToSkip.Contains(DependencyPackageName))
					{
						continue;
					}

					FName& RedirectedName = CachedDepToRedirect.FindOrAdd(DependencyPackageName, NAME_None);
					if (RedirectedName.IsNone())
					{
						RedirectedName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Package,
							FCoreRedirectObjectName(NAME_None, NAME_None, DependencyPackageName)).PackageName;
					}
					DependencyPackageName = RedirectedName;

					// Skip dependencies to self 
					if (SourceNodePackageName == DependencyPackageName)
					{
						continue;
					}

					bool bDependsNodeAlreadyExisted = false;
					FDependsNode* DependsNode = State.CreateOrFindDependsNode(DependencyPackageName, &bDependsNodeAlreadyExisted);
					ModifiedDependNodes.Add(DependsNode);

					if (!bDependsNodeAlreadyExisted)
					{
						// We created a new node so lookup its reservation
						if (FPackageDependencyData* DependencyDependencyData = DependsResults.Find(DependencyPackageName))
						{
							DependsNode->Reserve(DependencyDependencyData->DependsNodeReservations);
						}
					}

					// See if we need to read the script package Guid.
					// Script nodes are never represented as source nodes so we can use the initialization 
					// state to check if we need to set the package Guid or not
					// This node may have been created outside of this loop so we should always check
					if (DependsNode->IsScriptPath() && !DependsNode->IsScriptDependenciesInitialized())
					{
						DependsNode->SetIsScriptDependenciesInitialized(true);
						FNameBuilder DependencyPackageNameStr(DependencyPackageName);

						// Get the guid off the script package, it is updated when 
						// script is changed so we need to refresh it every run
						FGCScopeGuard Guard;
						UPackage* Package = FindPackage(nullptr, *DependencyPackageNameStr);

						if (Package)
						{
							FAssetPackageData* ScriptPackageData = State.CreateOrGetAssetPackageData(DependencyPackageName);
#if WITH_EDITORONLY_DATA
							ScriptPackageData->SetPackageSavedHash(Package->GetSavedHash());
#endif
						}
					}

					FDependsNode::FPackageFlagSet& FlagSet = PackageDependencies.FindOrAdd(DependsNode);
					FlagSet.Add(FDependsNode::PackagePropertiesToByte(DependencyData.Property));
				}

				FDependsNode::FDependsNodeList& SearchNameDependencyNodes = ConstructData->SearchableNamesNodes;
				for (const FPackageDependencyData::FSearchableNamesDependency& NamesDependency : Result.SearchableNameDependencies)
				{
					SearchNameDependencyNodes.Reserve(SearchNameDependencyNodes.Num() + NamesDependency.ValueNames.Num());
					for (const FName& ValueName : NamesDependency.ValueNames)
					{
						FAssetIdentifier AssetId(NamesDependency.PackageName, NamesDependency.ObjectName, ValueName);
						FDependsNode* DependsNode = State.CreateOrFindDependsNode(AssetId);

						FSetElementId AddId = ModifiedDependNodes.Add(DependsNode);
						SearchNameDependencyNodes.Add(DependsNode);
					}
				}
			}
		}
	}

	if (AssetNameToConstructDataMap.IsEmpty())
	{
		return;
	}

	TArray<FConstructData*, TInlineAllocator<1>> ConstructDatas;
	ConstructDatas.Reserve(AssetNameToConstructDataMap.Num());
	for (TPair<FName, FConstructData>& Pair : AssetNameToConstructDataMap)
	{
		ConstructDatas.Add(&Pair.Value);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DependencyGatheredData_RemoveReferencers);
		// Remove referencers - Single threaded due to the poor memory access patterns 
		// between nodes and their dependencies
		for (FConstructData* ConstructData : ConstructDatas)
		{
			FDependsNode* SourceNode = ConstructData->SourceNode;
			const FPackageDependencyMap& PackageDependencies = ConstructData->PackageDependencies;

			// We will populate the node dependencies below. Empty the set here in case this file was already read
			// Also remove references to all existing dependencies, those will be also repopulated below
			SourceNode->IterateOverDependencies([SourceNode, &PackageDependencies] (FDependsNode* InDependency, 
				EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
				{
					if (!bDuplicate)
					{
						// Only remove nodes that we aren't going to re-add further down
						if (!PackageDependencies.Contains(InDependency))
						{
							bool bOldAllowShrinking = InDependency->GetAllowShrinking();
							InDependency->SetAllowShrinking(false);
							InDependency->RemoveReferencer(SourceNode);
							InDependency->SetAllowShrinking(bOldAllowShrinking);
						}
					}
				});
		}
	}

	ParallelFor(TEXT("DependencyGatheredData_AddDependencies"), ConstructDatas.Num(), MinBatchSize, 
		[ShouldSortDependencyList = ShouldSortDependencies(), &ConstructDatas] (int Index)
		{
			FConstructData* ConstructData = ConstructDatas[Index];
			FDependsNode* SourceNode = ConstructData->SourceNode;
			const FPackageDependencyMap& PackageDependencies = ConstructData->PackageDependencies;
			const FDependsNode::FDependsNodeList& SearchableNameNodes = ConstructData->SearchableNamesNodes;

			SourceNode->ClearDependencies();
			SourceNode->SetIsDependencyListSorted(EDependencyCategory::All, ShouldSortDependencyList);

			// Add package dependencies
			for (const TPair<FDependsNode*, FDependsNode::FPackageFlagSet>& Pair : PackageDependencies)
			{
				SourceNode->AddPackageDependencySet(Pair.Key, Pair.Value);

			}

			// Add searchable name node dependencies
			for (FDependsNode* DependsNode : SearchableNameNodes)
			{
				SourceNode->AddDependency(DependsNode, EDependencyCategory::SearchableName, EDependencyProperty::None);
			}
		}, ParallelForFlags);

	// Add referencers - Single threaded due to the poor memory access patterns between nodes and their dependencies
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DependencyGatheredData_AddReferencers);
		for (FConstructData* ConstructData : ConstructDatas)
		{
			FDependsNode* SourceNode = ConstructData->SourceNode;
			const FPackageDependencyMap& PackageDependencies = ConstructData->PackageDependencies;
			const FDependsNode::FDependsNodeList& SearchableNameNodes = ConstructData->SearchableNamesNodes;

			// Doubly-link all of the PackageDependencies
			for (const TPair<FDependsNode*, FDependsNode::FPackageFlagSet>& Pair : PackageDependencies)
			{
				FDependsNode* DependsNode = Pair.Key;

				// Don't add sorted, we'll sort references afterwards
				DependsNode->SetIsReferencersSorted(false);
				DependsNode->AddReferencer(SourceNode);
			}

			// Add node for all name references
			for (FDependsNode* DependsNode : SearchableNameNodes)
			{
				// Don't add sorted, we'll sort references afterwards
				DependsNode->SetIsReferencersSorted(false);
				DependsNode->AddReferencer(SourceNode);
			}
		}
	}

	// Sort all referencers now that we are done inserting
	ParallelFor(TEXT("DependencyGatheredData_SortNodes"), ModifiedDependNodes.GetMaxIndex(), 1, 
		[&ModifiedDependNodes, bShouldSortReferencers = ShouldSortReferencers()] (int Index)
		{
			FSetElementId Id = FSetElementId::FromInteger(Index);
			if (!ModifiedDependNodes.IsValidId(Id))
			{
				return;
			}
			FDependsNode* Node = ModifiedDependNodes.Get(Id);

			// Set AllowShrinking first so SetIsReferencersSorted will reduce our memory use if we have over allocated
			Node->SetAllowShrinking(true);
			Node->SetIsReferencersSorted(bShouldSortReferencers);
		}, ParallelForFlags);
}

void FAssetRegistryImpl::CookedPackageNamesWithoutAssetDataGathered(Impl::FEventContext& EventContext,
	TRingBuffer<FString>& CookedPackageNamesWithoutAssetDataResults, Impl::FInterruptionContext& InOutInterruptionContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CookedPackageNamesWithoutAssetDataGathered);

	struct FConfigValue
	{
		FConfigValue()
		{
			if (GConfig)
			{
				GConfig->GetBool(TEXT("AssetRegistry"), TEXT("LoadCookedPackagesWithoutAssetData"), bShouldProcess, GEngineIni);
			}
		}

		bool bShouldProcess = true;
	};
	static FConfigValue ShouldProcessCookedPackages;

	// Add the found assets
	if (ShouldProcessCookedPackages.bShouldProcess)
	{
		while (CookedPackageNamesWithoutAssetDataResults.Num() > 0)
		{
			// If this data is cooked and it we couldn't find any asset in its export table then try to load the entire package 
			// Loading the entire package will make all of its assets searchable through the in-memory scanning performed by GetAssets
			EventContext.RequiredLoads.Add(CookedPackageNamesWithoutAssetDataResults.PopFrontValue());

			// This function has a bug in multithreaded environment (ie UE-209843). But the feature of loading cooked packages seems to be never used
			// so let's try to deprecate it. Also a new way to load cooked package is on the way.
			UE_LOGF(LogAssetRegistry, Warning, "CookedPackageNamesWithoutAssetDataGathered : Deprecated in 5.6 due to poor performance, will be removed in a future version, contact Epic if you need this functionality.");
		}
		// Avoid marking the scan complete before we have loaded all the relevant assets. By interrupting here
		// we intend to ensure that the event context is processed, triggering a LoadPackage, and then a ProcessLoadedAssetsToUpdateCache,
		// and only then resume scanning from disk. However, in the current multithreaded implementation this is not guaranteed
		// as only the main thread broadcasts events but the background thread might come around for another time slice before
		// the main thread does so.  UE-209843
		if (InOutInterruptionContext.IsTimeSlicingEnabled())
		{
			InOutInterruptionContext.RequestEarlyExit();
			return;
		}
	}
	else
	{
		// Do nothing will these packages. For projects which could run entirely from cooked data, this
		// process will involve opening every single package synchronously on the game thread which will
		// kill performance. We need a better way.
		CookedPackageNamesWithoutAssetDataResults.Empty();
	}
}

void FAssetRegistryImpl::VerseFilesGathered(Impl::FEventContext& EventContext, TRingBuffer<FName>& VerseResults, 
	Impl::FInterruptionContext& InOutInterruptionContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(VerseFilesGathered);

	while (VerseResults.Num() > 0)
	{
		FName VerseFilePath = VerseResults.PopFrontValue();

		AddVerseFile(EventContext, VerseFilePath);

		// Check to see if we have run out of time in this tick
		if (InOutInterruptionContext.ShouldExitEarly())
		{
			return;
		}
	}
}

void FAssetRegistryImpl::AddEmptyPackage(FName PackageName)
{
	CachedEmptyPackages.Add(PackageName);
}

bool FAssetRegistryImpl::RemoveEmptyPackage(FName PackageName)
{
	return CachedEmptyPackages.Remove(PackageName) > 0;
}

bool FAssetRegistryImpl::AddAssetPath(Impl::FEventContext& EventContext, FName PathToAdd)
{
	return CachedPathTree.CachePath(PathToAdd, [this, &EventContext](FName AddedPath)
	{
		EventContext.PathEvents.Emplace(AddedPath.ToString(), Impl::FEventContext::EEvent::Added);
	});
}

bool FAssetRegistryImpl::RemoveAssetPath(Impl::FEventContext& EventContext, FName PathToRemove, bool bEvenIfAssetsStillExist)
{
	if (!bEvenIfAssetsStillExist)
	{
		// Check if there were assets in the specified folder. You can not remove paths that still contain assets
		bool bHasAsset = false;
		EnumerateAssetsByPathNoTags(PathToRemove, [&bHasAsset](const FAssetData&)
			{
				bHasAsset = true;
				return false;
			}, true /* bRecursive */, false /* bIncludeOnlyOnDiskAssets */);

		// If the verse file caches contain this path then keep it around
		bHasAsset |= CachedVerseFilesByPath.Contains(PathToRemove);

		if (bHasAsset)
		{
			// At least one asset still exists in the path. Fail the remove.
			return false;
		}
	}

	CachedPathTree.RemovePath(PathToRemove, [this, &EventContext](FName RemovedPath)
	{
		EventContext.PathEvents.Emplace(RemovedPath.ToString(), Impl::FEventContext::EEvent::Removed);
	});
	return true;
}

void FAssetRegistryImpl::AddAssetData(Impl::FEventContext& EventContext, FAssetData* AssetData)
{
#if WITH_EDITOR
	// Update Redirectors
	if (AssetData->IsRedirector())
	{
		FString RedirectDestinationString;
		AssetData->GetTagValue(UE::AssetRegistry::Impl::DestinationObjectFName, RedirectDestinationString);
		FSoftObjectPath RedirectDestination = RedirectDestinationString;
		if (!RedirectDestination.IsNull())
		{
			GRedirectCollector.AddAssetPathRedirection(AssetData->GetSoftObjectPath(), RedirectDestination);
		}
	}
#endif

	State.AddAssetData(AssetData);

	if (!ShouldSkipAsset(AssetData->AssetClassPath, AssetData->PackageFlags))
	{
		EventContext.AssetEvents.Emplace(*AssetData, Impl::FEventContext::EEvent::Added);
	}

	// Populate the class map if adding blueprint
	if (ClassGeneratorNames.Contains(AssetData->AssetClassPath))
	{
		const FString GeneratedClass = AssetData->GetTagValueRef<FString>(FBlueprintTags::GeneratedClassPath);
		const FString ParentClass = AssetData->GetTagValueRef<FString>(FBlueprintTags::ParentClassPath);
		if (!GeneratedClass.IsEmpty() && !ParentClass.IsEmpty() && GeneratedClass != TEXTVIEW("None") && ParentClass != TEXTVIEW("None"))
		{
			const FTopLevelAssetPath SavedGeneratedClassPathName(GeneratedClass);
			const FTopLevelAssetPath GeneratedClassPathName(AssetData->PackageName, SavedGeneratedClassPathName.GetAssetName());
			const FTopLevelAssetPath ParentClassPathName(ParentClass);
			if (ensureAlwaysMsgf(!GeneratedClassPathName.IsNull() && !ParentClassPathName.IsNull(),
				TEXT("Short class names used in AddAssetData: GeneratedClass=%s, ParentClass=%s. Short class names in these tags on the Blueprint class should have been converted to path names."),
				*GeneratedClass, *ParentClass))
			{
				AddCachedBPClassParent(GeneratedClassPathName, ParentClassPathName);

				// Invalidate caching because CachedBPInheritanceMap got modified
				TempCachedInheritanceBuffer.bDirty = true;
			}
		}
	}
}

void FAssetRegistryImpl::UpdateAssetData(Impl::FEventContext& EventContext, FAssetData* AssetData,
	FAssetData&& NewAssetData, bool bKeepDeletedTags)
{
	// Update the class map if updating a blueprint
	if (ClassGeneratorNames.Contains(AssetData->AssetClassPath))
	{
		const FString OldGeneratedClass = AssetData->GetTagValueRef<FString>(FBlueprintTags::GeneratedClassPath);
		const FString OldParentClass = AssetData->GetTagValueRef<FString>(FBlueprintTags::ParentClassPath);
		const FString NewGeneratedClass = NewAssetData.GetTagValueRef<FString>(FBlueprintTags::GeneratedClassPath);
		const FString NewParentClass = NewAssetData.GetTagValueRef<FString>(FBlueprintTags::ParentClassPath);
		if (OldGeneratedClass != NewGeneratedClass || OldParentClass != NewParentClass)
		{
			if (!OldGeneratedClass.IsEmpty() && OldGeneratedClass != TEXTVIEW("None"))
			{
				const FTopLevelAssetPath OldGeneratedClassName(OldGeneratedClass);
				if (ensureAlwaysMsgf(!OldGeneratedClassName.IsNull(),
					TEXT("Short class name used: OldGeneratedClass=%s. Short class names in tags on the Blueprint class should have been converted to path names."),
					*OldGeneratedClass))
				{
					CachedBPInheritanceMap.Remove(OldGeneratedClassName);

					// Invalidate caching because CachedBPInheritanceMap got modified
					TempCachedInheritanceBuffer.bDirty = true;
				}
			}

			if (!NewGeneratedClass.IsEmpty() && !NewParentClass.IsEmpty() && NewGeneratedClass != TEXTVIEW("None") && NewParentClass != TEXTVIEW("None"))
			{
				const FTopLevelAssetPath NewGeneratedClassName(NewGeneratedClass);
				const FTopLevelAssetPath NewParentClassName(NewParentClass);
				if (ensureAlwaysMsgf(!NewGeneratedClassName.IsNull() && !NewParentClassName.IsNull(),
					TEXT("Short class names used in AddAssetData: GeneratedClass=%s, ParentClass=%s. Short class names in these tags on the Blueprint class should have been converted to path names."),
					*NewGeneratedClass, *NewParentClass))
				{
					AddCachedBPClassParent(NewGeneratedClassName, NewParentClassName);
				}

				// Invalidate caching because CachedBPInheritanceMap got modified
				TempCachedInheritanceBuffer.bDirty = true;
			}
		}
	}

	if (bKeepDeletedTags)
	{
		TOptional<FAssetDataTagMap> UpdatedTags;
		AssetData->TagsAndValues.ForEach([&NewAssetData, &UpdatedTags](const TPair<FName, FAssetTagValueRef>& TagPair)
			{
				if (UpdatedTags)
				{
					if (!UpdatedTags->Contains(TagPair.Key))
					{
						UpdatedTags->Add(TagPair.Key, TagPair.Value.GetStorageString());
					}
				}
				else
				{
					if (!NewAssetData.TagsAndValues.Contains(TagPair.Key))
					{
						UpdatedTags.Emplace(NewAssetData.TagsAndValues.CopyMap());
						UpdatedTags->Add(TagPair.Key, TagPair.Value.GetStorageString());
					}
				}
			});
		if (UpdatedTags)
		{
			NewAssetData.TagsAndValues = FAssetDataTagMapSharedView(MoveTemp(*UpdatedTags));
		}
	}

#if WITH_EDITOR
	// Update Redirectors
	if (AssetData->IsRedirector() || NewAssetData.IsRedirector())
	{
		FSoftObjectPath OldSource = AssetData->GetSoftObjectPath();
		FSoftObjectPath NewSource = NewAssetData.GetSoftObjectPath();
		FSoftObjectPath OldTarget;
		FSoftObjectPath NewTarget;
		if (AssetData->IsRedirector())
		{
			FString TargetString;
			AssetData->GetTagValue(UE::AssetRegistry::Impl::DestinationObjectFName, TargetString);
			OldTarget = TargetString;
		}
		if (NewAssetData.IsRedirector())
		{
			FString TargetString;
			NewAssetData.GetTagValue(UE::AssetRegistry::Impl::DestinationObjectFName, TargetString);
			NewTarget = TargetString;
		}
		if (OldSource != NewSource && OldTarget.IsValid())
		{
			GRedirectCollector.RemoveAssetPathRedirection(OldSource);
		}
		if (NewTarget.IsValid())
		{
			GRedirectCollector.AddAssetPathRedirection(NewSource, NewTarget);
		}
	}
#endif

	bool bModified;
	State.UpdateAssetData(AssetData, MoveTemp(NewAssetData), &bModified);
	
	if (bModified && !ShouldSkipAsset(AssetData->AssetClassPath, AssetData->PackageFlags))
	{
		EventContext.AssetEvents.Emplace(*AssetData, Impl::FEventContext::EEvent::Updated);
	}
}

void FAssetRegistryImpl::AddNonOverlappingTags(Impl::FEventContext& EventContext, FAssetData& ExistingAssetData,
	const FAssetData& NewAssetData)
{
	TOptional<FAssetDataTagMap> ModifiedTags = Utils::AddNonOverlappingTags(ExistingAssetData, NewAssetData);
	if (ModifiedTags)
	{
		State.SetTagsOnExistingAsset(&ExistingAssetData, MoveTemp(*ModifiedTags));
		if (!ShouldSkipAsset(ExistingAssetData.AssetClassPath, ExistingAssetData.PackageFlags))
		{
			EventContext.AssetEvents.Emplace(ExistingAssetData, Impl::FEventContext::EEvent::Updated);
		}
	}
}

bool FAssetRegistryImpl::RemoveAssetData(Impl::FEventContext& EventContext, FAssetData* AssetData)
{
	if (!ensure(AssetData))
	{
		return false;
	}
	return RemoveAssetDatas(EventContext, { &AssetData, 1 });
}

bool FAssetRegistryImpl::RemoveAssetDatas(Impl::FEventContext& EventContext, TArrayView<FAssetData*> AssetDatas)
{
#if WITH_EDITOR
	TArray<FSoftObjectPath, TInlineAllocator<1>> RedirectSources;
	RedirectSources.Reserve(AssetDatas.Num());
#endif

	for (FAssetData* AssetData : AssetDatas)
	{
		if (!ShouldSkipAsset(AssetData->AssetClassPath, AssetData->PackageFlags))
		{
			EventContext.AssetEvents.Emplace(*AssetData, Impl::FEventContext::EEvent::Removed);
		}

#if WITH_EDITOR
		// Save a copy of the AssetData's SoftObjectPath if we need to remove it; we don't want to remove the
		// redirection unless the AssetRegistryState confirms the AssetData existed and was removed, but after
		// it is removed our AssetData pointer might become a dangling pointer.
		FSoftObjectPath RedirectSource = AssetData->IsRedirector() ? AssetData->GetSoftObjectPath() : FSoftObjectPath();
		RedirectSources.Add(RedirectSource);
#endif

		// Remove from the class map if removing a blueprint
		if (ClassGeneratorNames.Contains(AssetData->AssetClassPath))
		{
			const FString OldGeneratedClass = AssetData->GetTagValueRef<FString>(FBlueprintTags::GeneratedClassPath);
			if (!OldGeneratedClass.IsEmpty() && OldGeneratedClass != TEXTVIEW("None"))
			{
				const FTopLevelAssetPath OldGeneratedClassPathName(FPackageName::ExportTextPathToObjectPath(OldGeneratedClass));
				if (ensureAlwaysMsgf(!OldGeneratedClassPathName.IsNull(),
					TEXT("Short class name used: OldGeneratedClass=%s"), *OldGeneratedClass))
				{
					CachedBPInheritanceMap.Remove(OldGeneratedClassPathName);

					// Invalidate caching because CachedBPInheritanceMap got modified
					TempCachedInheritanceBuffer.bDirty = true;
				}
			}
		}
	}

	TBitArray<FDefaultBitArrayAllocator> bRemoved;
	TBitArray<FDefaultBitArrayAllocator> bRemovedDependencyData;
	bRemoved.SetNumUninitialized(AssetDatas.Num());
	bRemovedDependencyData.SetNumUninitialized(AssetDatas.Num());
	State.RemoveAssetDatas(AssetDatas, true /* bRemoveDependencyData */, bRemoved, bRemovedDependencyData);

#if WITH_EDITOR
	for (TBitArray<FDefaultBitArrayAllocator>::FConstIterator It(bRemoved); It; ++It)
	{
		if (It.GetValue())
		{
			// Update Redirectors
			FSoftObjectPath& RedirectSource = RedirectSources[It.GetIndex()];
			if (RedirectSource.IsValid())
			{
				GRedirectCollector.RemoveAssetPathRedirection(RedirectSource);
			}
		}
	}
#endif

	return bRemoved.Contains(true);
}

void FAssetRegistryImpl::RemovePackageData(Impl::FEventContext& EventContext, const FName PackageName)
{
	// Even if we could point to the array, we have to copy the array since RemoveAssetData may re-allocate it.
	TArray<FAssetData*, TInlineAllocator<1>> PackageAssets;
	State.EnumerateMutableAssetsByPackageName(PackageName, [&PackageAssets](FAssetData* AssetData)
		{
			PackageAssets.Add(AssetData);
			return true;
		});

	if (PackageAssets.Num() > 0)
	{
		FAssetIdentifier PackageAssetIdentifier(PackageName);
		// If there were any EDependencyCategory::Package referencers, re-add them to a new empty dependency node, as it would be when the referencers are loaded from disk
		// We do not have to handle SearchableName or Manage referencers, because those categories of dependencies are not created for non-existent AssetIdentifiers
		TArray<TPair<FAssetIdentifier, FDependsNode::FPackageFlagSet>> PackageReferencers;
		{
			FDependsNode** FoundPtr = State.CachedDependsNodes.Find(PackageAssetIdentifier);
			FDependsNode* DependsNode = FoundPtr ? *FoundPtr : nullptr;
			if (DependsNode)
			{
				DependsNode->GetPackageReferencers(PackageReferencers);
			}
		}

		for (FAssetData* PackageAsset : PackageAssets)
		{
			RemoveAssetData(EventContext, PackageAsset);
		}

		// Readd any referencers, creating an empty DependsNode to hold them
		if (PackageReferencers.Num())
		{
			FDependsNode* NewNode = State.CreateOrFindDependsNode(PackageAssetIdentifier);
			for (TPair<FAssetIdentifier, FDependsNode::FPackageFlagSet>& Pair : PackageReferencers)
			{
				FDependsNode* ReferencerNode = State.CreateOrFindDependsNode(Pair.Key);
				if (ReferencerNode != nullptr)
				{
					ReferencerNode->AddPackageDependencySet(NewNode, Pair.Value);
					NewNode->AddReferencer(ReferencerNode);
				}
			}
		}
	}
}

void FAssetRegistryImpl::AddVerseFile(Impl::FEventContext& EventContext, FName VerseFilePathToAdd)
{
	bool bAlreadyExists = false;
	CachedVerseFiles.Add(VerseFilePathToAdd, &bAlreadyExists);
	if (!bAlreadyExists)
	{
		FName VerseDirectoryPath(FPathViews::GetPath(WriteToString<256>(VerseFilePathToAdd)));
		
		// Ensure this path is represented in the CachedPathTree
		AddPath(EventContext, WriteToString<256>(VerseDirectoryPath));

		TArray<FName>& FilePathsArray = CachedVerseFilesByPath.FindOrAdd(VerseDirectoryPath);
		FilePathsArray.Add(VerseFilePathToAdd);
		EventContext.VerseEvents.Emplace(VerseFilePathToAdd, Impl::FEventContext::EEvent::Added);
	}
}

void FAssetRegistryImpl::RemoveVerseFile(Impl::FEventContext& EventContext, FName VerseFilePathToRemove)
{
	if (CachedVerseFiles.Remove(VerseFilePathToRemove))
	{
		FName VerseDirectoryPath(FPathViews::GetPath(WriteToString<256>(VerseFilePathToRemove)));
		TArray<FName>* FilePathsArray = CachedVerseFilesByPath.Find(VerseDirectoryPath);
		if (ensure(FilePathsArray)) // We found it in CachedVerseFiles, so we must also find it here
		{
			FilePathsArray->Remove(VerseFilePathToRemove);
			if (FilePathsArray->IsEmpty())
			{
				CachedVerseFilesByPath.Remove(VerseDirectoryPath);
				
				// Try to remove this path from the general CachedPathTree - assuming no other files are keeping it around
				RemoveAssetPath(EventContext, VerseDirectoryPath);  
			}
		}
		EventContext.VerseEvents.Emplace(VerseFilePathToRemove, Impl::FEventContext::EEvent::Removed);
	}
}

}

#if WITH_EDITOR

void UAssetRegistryImpl::OnDirectoryChanged(const TArray<FFileChangeData>& FileChanges)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAssetRegistryImpl::OnDirectoryChanged);

	double StartTime = FPlatformTime::Seconds();
	
	// Take local copy of FileChanges array as we wish to collapse pairs of 'Removed then Added' FileChangeData
	// entries into a single 'Modified' entry.
	TArray<FFileChangeData> FileChangesProcessed(FileChanges);

	for (int32 FileEntryIndex = 0; FileEntryIndex < FileChangesProcessed.Num(); FileEntryIndex++)
	{
		if (FileChangesProcessed[FileEntryIndex].Action == FFileChangeData::FCA_Added)
		{
			// Search back through previous entries to see if this Added can be paired with a previous Removed
			const FString& FilenameToCompare = FileChangesProcessed[FileEntryIndex].Filename;
			for (int32 SearchIndex = FileEntryIndex - 1; SearchIndex >= 0; SearchIndex--)
			{
				if (FileChangesProcessed[SearchIndex].Action == FFileChangeData::FCA_Removed &&
					FileChangesProcessed[SearchIndex].Filename == FilenameToCompare)
				{
					// Found a Removed which matches the Added - change the Added file entry to be a Modified...
					FileChangesProcessed[FileEntryIndex].Action = FFileChangeData::FCA_Modified;

					// ...and remove the Removed entry
					FileChangesProcessed.RemoveAt(SearchIndex);
					FileEntryIndex--;
					break;
				}
			}
		}
	}

	{
		// Check that the change is related to a directory that has actually been mounted.
		TStringBuilder<256> MountPointPackageName;
		TStringBuilder<256> MountPointFilePath;
		TStringBuilder<256> RelativePath;
		for (int32 FileEntryIndex = FileChangesProcessed.Num() - 1; FileEntryIndex >= 0; FileEntryIndex--)
		{
			FFileChangeData& Data = FileChangesProcessed[FileEntryIndex];
			if (Data.Action != FFileChangeData::FCA_RescanRequired && !FPackageName::TryGetMountPointForPath(
				Data.Filename, MountPointPackageName, MountPointFilePath, RelativePath))
			{
				FileChangesProcessed.RemoveAt(FileEntryIndex);
			}
		}
	}

	UE::AssetRegistry::Impl::FEventContext EventContext;
	bool bInitialSearchStarted;
	bool bInitialSearchCompleted;
	bool bAdditionalMountSearchInProgress;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		bInitialSearchStarted = GuardedData.IsInitialSearchStarted();
		bInitialSearchCompleted = GuardedData.IsInitialSearchCompleted();
		bAdditionalMountSearchInProgress = GuardedData.IsAdditionalMountSearchInProgress();
		UE::AssetRegistry::Impl::FClassInheritanceContext InheritanceContext;
		UE::AssetRegistry::Impl::FClassInheritanceBuffer InheritanceBuffer;
		GetInheritanceContextWithRequiredLock(InterfaceScopeLock, InheritanceContext, InheritanceBuffer);
		GuardedData.OnDirectoryChanged(InterfaceScopeLock, EventContext, InheritanceContext, FileChangesProcessed);
	}
	Broadcast(EventContext);
	
	FTelemetryRouter::Get().ProvideTelemetry<UE::Telemetry::AssetRegistry::FDirectoryWatcherUpdateTelemetry>({
		FileChanges,
		FPlatformTime::Seconds() - StartTime,
		bInitialSearchStarted,
		bInitialSearchCompleted,
		bAdditionalMountSearchInProgress,
	});
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::OnDirectoryChanged(FInterfaceWriteScopeLock& ScopeLock, Impl::FEventContext& EventContext,
	Impl::FClassInheritanceContext& InheritanceContext, TArray<FFileChangeData>& FileChangesProcessed)
{
	TArray<FString> NewDirs;
	TArray<FString> NewFiles;
	TArray<FString> ModifiedFiles;
	for (int32 FileIdx = 0; FileIdx < FileChangesProcessed.Num(); ++FileIdx)
	{
		if (FileChangesProcessed[FileIdx].Action == FFileChangeData::FCA_RescanRequired)
		{
			if (bInitialSearchStarted && !IsInitialSearchCompleted()) // Initial search only
			{
				// Ignore rescan request during initial scan as it is probably caused by the scan itself
				UE_LOGF(LogAssetRegistry, Log, "FAssetRegistry ignoring rescan request for %ls during startup", *FileChangesProcessed[FileIdx].Filename);
			}
			else
			{
				OnDirectoryRescanRequired(ScopeLock, EventContext, InheritanceContext, FileChangesProcessed[FileIdx].Filename,
					FileChangesProcessed[FileIdx].TimeStamp);
			}

			continue;
		}
		FString LongPackageName;
		const FString File = FString(FileChangesProcessed[FileIdx].Filename);
		const bool bIsPackageFile = FPackageName::IsPackageExtension(*FPaths::GetExtension(File, true));
		const bool bIsValidPackageName = FPackageName::TryConvertFilenameToLongPackageName(
			File,
			LongPackageName,
			/*OutFailureReason*/ nullptr,
			/* Verse files can be of the wildcard pattern `*.*.verse`. */
			FAssetDataGatherer::IsVerseFile(File) && !bIsPackageFile ? FPackageName::EConvertFlags::AllowDots : FPackageName::EConvertFlags::None);
		const bool bIsValidPackage = bIsPackageFile && bIsValidPackageName;

		if (bIsValidPackage)
		{
			FName LongPackageFName(*LongPackageName);

			bool bAddedOrCreated = false;
			switch (FileChangesProcessed[FileIdx].Action)
			{
			case FFileChangeData::FCA_Added:
				// This is a package file that was created on disk. Mark it to be scanned for asset data.
				NewFiles.AddUnique(File);
				bAddedOrCreated = true;
				UE_LOGF(LogAssetRegistry, Verbose, "File was added to content directory: %ls", *File);
				break;

			case FFileChangeData::FCA_Modified:
				// This is a package file that changed on disk. Mark it to be scanned immediately for new or removed asset data.
				ModifiedFiles.AddUnique(File);
				bAddedOrCreated = true;
				UE_LOGF(LogAssetRegistry, Verbose, "File changed in content directory: %ls", *File);
				break;

			case FFileChangeData::FCA_Removed:
				// This file was deleted. Remove all assets in the package from the registry.
				RemovePackageData(EventContext, LongPackageFName);
				// If the package was a package we were tracking as empty (due to e.g. a rename in editor), remove it.
				// Disk now matches editor
				RemoveEmptyPackage(LongPackageFName);
				UE_LOGF(LogAssetRegistry, Verbose, "File was removed from content directory: %ls", *File);
				break;
			}
			if (bAddedOrCreated && CachedEmptyPackages.Contains(LongPackageFName))
			{
				UE_LOGF(LogAssetRegistry, Warning, "%ls: package was marked as deleted in editor, but has been modified on disk. It will once again be returned from AssetRegistry queries.",
					*File);
				RemoveEmptyPackage(LongPackageFName);
			}
		}
		else if (bIsValidPackageName)
		{
			// Is this a Verse file?
			if (FAssetDataGatherer::IsVerseFile(File))
			{
				switch (FileChangesProcessed[FileIdx].Action)
				{
				case FFileChangeData::FCA_Added:
					// This is a Verse file that was created on disk.
					NewFiles.AddUnique(File);
					UE_LOGF(LogAssetRegistry, Verbose, "Verse file was added to content directory: %ls", *File);
					break;

				case FFileChangeData::FCA_Modified:
					// Note: Since content of Verse files is not scanned, no need to handle FCA_Modified
					break;

				case FFileChangeData::FCA_Removed:
					RemoveVerseFile(EventContext, FName(WriteToString<256>(LongPackageName, FPathViews::GetExtension(File, /*bIncludeDot*/ true))));
					UE_LOGF(LogAssetRegistry, Verbose, "Verse file was removed from content directory: %ls", *File);
					break;
				}
			}
			else
			{
				// This could be a directory or possibly a file with no extension or a wrong extension.
				// No guaranteed way to know at this point since it may have been deleted.
				switch (FileChangesProcessed[FileIdx].Action)
				{
				case FFileChangeData::FCA_Added:
				{
					if (FPaths::DirectoryExists(File))
					{
						NewDirs.Add(File);
						UE_LOGF(LogAssetRegistry, Verbose, "Directory was added to content directory: %ls", *File);
					}
					break;
				}
				case FFileChangeData::FCA_Removed:
				{
					FName Path(UE::String::RemoveFromEnd(FStringView(LongPackageName), TEXTVIEW("/")));
					RemoveAssetPath(EventContext, Path);
					UE_LOGF(LogAssetRegistry, Verbose, "Directory was removed from content directory: %ls", *File);
					break;
				}
				default:
					break;
				}
			}
		}

#if WITH_EDITOR
		if (bIsValidPackageName)
		{
			// If a package changes in a referenced directory, modify the Assets that monitor that directory
			FString ParentDirectory = UE::AssetRegistry::CreateStandardFilename(FPaths::GetPath(File));
			TArray<FName, TInlineAllocator<1>> WatcherPackageNames;
			while (!ParentDirectory.IsEmpty())
			{
				TSet<FName>* PackagesWatchingThisDirectory = PackagesWatchingDirectory.Find(ParentDirectory);
				if (PackagesWatchingThisDirectory)
				{
					for (FName PackageWatchingThisDirectory : *PackagesWatchingThisDirectory)
					{
						WatcherPackageNames.Add(PackageWatchingThisDirectory);
					}
				}
				FString NewParentDirectory = FPaths::GetPath(ParentDirectory);
				if (ParentDirectory == NewParentDirectory)
				{
					break;
				}
				ParentDirectory = MoveTemp(NewParentDirectory);
			}
			Algo::Sort(WatcherPackageNames, FNameFastLess());
			WatcherPackageNames.SetNum(Algo::Unique(WatcherPackageNames));

			for (FName WatcherPackageName : WatcherPackageNames)
			{
				// ScanModifiedAssetFiles accepts LongPackageNames as well as LocalPaths
				ModifiedFiles.AddUnique(WatcherPackageName.ToString());
			}
		}
#endif
			
	}

	if (NewFiles.Num() || NewDirs.Num())
	{
		if (GlobalGatherer.IsValid())
		{
			for (FString& NewDir : NewDirs)
			{
				GlobalGatherer->OnDirectoryCreated(NewDir);
			}
			GlobalGatherer->OnFilesCreated(NewFiles);
			if (GlobalGatherer->IsSynchronous())
			{
				Impl::FScanPathContext Context(EventContext, InheritanceContext, NewDirs, NewFiles,
					UE::AssetRegistry::EScanFlags::None, nullptr /* OutFoundAssets */);
				ScanPathsSynchronous(&ScopeLock, Context);
			}
		}
	}
	ScanModifiedAssetFiles(ScopeLock, EventContext, InheritanceContext, ModifiedFiles, UE::AssetRegistry::EScanFlags::None);
}

void FAssetRegistryImpl::OnDirectoryRescanRequired(FInterfaceWriteScopeLock& ScopeLock, Impl::FEventContext& EventContext,
	Impl::FClassInheritanceContext& InheritanceContext, FString& DirPath, int64 BeforeTimeStamp)
{
	TArray<TPair<FString,FString>> DirPathsAndPackageNames;
	FString DirPathAsPackageName;
	FString NormalizedDirPath = UE::AssetRegistry::CreateStandardFilename(DirPath);
	if (FPackageName::TryConvertFilenameToLongPackageName(NormalizedDirPath, DirPathAsPackageName))
	{
		DirPathsAndPackageNames.Emplace(DirPath, MoveTemp(DirPathAsPackageName));
	}
	else
	{
		TArray<FString> ContentRoots;
		FPackageName::QueryRootContentPaths(ContentRoots);
		TStringBuilder<64> UnusedPackageName;
		TStringBuilder<256> MountedFilePath;
		TStringBuilder<16> UnusedRelPath;
		for (FString& MountedLongPackageName : ContentRoots)
		{
			if (FPackageName::TryGetMountPointForPath(MountedLongPackageName, UnusedPackageName, MountedFilePath, UnusedRelPath))
			{
				FString NormalizeMountedFilePath = UE::AssetRegistry::CreateStandardFilename(MountedFilePath);
				if (FPaths::IsUnderDirectory(NormalizeMountedFilePath, NormalizedDirPath))
				{
					DirPathsAndPackageNames.Emplace(MoveTemp(NormalizeMountedFilePath), MoveTemp(MountedLongPackageName));
				}
			}
		}
	}
	if (DirPathsAndPackageNames.IsEmpty())
	{
		return;
	}

	struct FDirectoryResults
	{
		TArray<FString> NewFiles;
		TArray<FString> ModifiedFiles;
		TSet<FName> RemovedLongPackageNames;
	};
	int32 NumDirs = DirPathsAndPackageNames.Num();
	TArray<FDirectoryResults> Results;
	Results.SetNum(NumDirs);
	FDateTime BeforeDateTime = FDateTime::FromUnixTimestamp(BeforeTimeStamp);

	for (int32 DirIndex = 0; DirIndex < NumDirs; ++DirIndex)
	{
		FString& PackageNamePath = DirPathsAndPackageNames[DirIndex].Value;
		FDirectoryResults& Result = Results[DirIndex];
		EnumerateAssetsByPathNoTags(*PackageNamePath, [&Result](const FAssetData& AssetData)
			{
				Result.RemovedLongPackageNames.Add(AssetData.PackageName);
				return true;
			}, true /* bRecursive */, true /* bIncludeOnlyOnDiskAssets */);
	}

	ParallelFor(NumDirs, [this, &DirPathsAndPackageNames, &Results, &BeforeDateTime](int32 DirIndex)
		{
			FDirectoryResults& Result = Results[DirIndex];
			TPair<FString, FString>& Pair = DirPathsAndPackageNames[DirIndex];
			FString& LocalPath = Pair.Key;
			FString& PackageNamePath = Pair.Value;

			FPackageName::IteratePackagesInDirectory(LocalPath,
				[&LocalPath, &PackageNamePath, &BeforeDateTime, &Result]
				(const TCHAR* Filename, const FFileStatData& StatData)
				{
					// Convert Filename to a PackagePath. We know the base dir so its faster to use that than FPackageName
					// which has to scan all mount dirs
					FStringView RelPath;
					FString NormalizedFilename = UE::AssetRegistry::CreateStandardFilename(FStringView(Filename));
					if (!FPathViews::TryMakeChildPathRelativeTo(NormalizedFilename, LocalPath, RelPath))
					{
						return true;
					}
					const bool bIsPackageFile = FPackageName::IsPackageExtension(
						*FString(FPathViews::GetExtension(RelPath, true /* bIncludeDot */)));
					RelPath = FPathViews::GetBaseFilenameWithPath(RelPath);
					TStringBuilder<256> FilePackagePath;
					FilePackagePath << PackageNamePath;
					FPathViews::AppendPath(FilePackagePath, RelPath);
					for (int32 Index = 0; Index < FilePackagePath.Len(); ++Index)
					{
						TCHAR& Char = FilePackagePath.GetData()[Index];
						if (Char == '\\')
						{
							Char = '/';
						}
					}
					const bool bIsValidPackageName = FPackageName::IsValidTextForLongPackageName(FilePackagePath);
					if (!bIsPackageFile || !bIsValidPackageName)
					{
						return true;
					}

					if (StatData.CreationTime > BeforeDateTime)
					{
						Result.NewFiles.Add(NormalizedFilename);
					}
					else if (StatData.ModificationTime > BeforeDateTime)
					{
						Result.ModifiedFiles.Add(NormalizedFilename);
					}
					Result.RemovedLongPackageNames.Remove(FName(FilePackagePath.ToView()));

					return true;
				});
		});

	TArray<FName> FinalRemovedLongPackageNames;
	FDirectoryResults& FinalResult = Results[0];
	FinalRemovedLongPackageNames.Append(FinalResult.RemovedLongPackageNames.Array());
	for (int32 DirIndex = 1; DirIndex < NumDirs; ++DirIndex)
	{
		FDirectoryResults& ResultToMerge = Results[DirIndex];
		FinalResult.NewFiles.Append(MoveTemp(ResultToMerge.NewFiles));
		FinalResult.ModifiedFiles.Append(MoveTemp(ResultToMerge.ModifiedFiles));
		FinalRemovedLongPackageNames.Append(ResultToMerge.RemovedLongPackageNames.Array());
	}

	for (FName LongPackageName : FinalRemovedLongPackageNames)
	{
		// This file was deleted. Remove all assets in the package from the registry.
		RemovePackageData(EventContext, LongPackageName);
		// If the package was a package we were tracking as empty (due to e.g. a rename in editor), remove it.
		// Disk now matches editor
		RemoveEmptyPackage(LongPackageName);
	}
	if (FinalResult.NewFiles.Num())
	{
		if (GlobalGatherer.IsValid())
		{
			GlobalGatherer->OnFilesCreated(FinalResult.NewFiles);
			if (GlobalGatherer->IsSynchronous())
			{
				TArray<FString> UnusedNewDirs;
				Impl::FScanPathContext Context(EventContext, InheritanceContext, UnusedNewDirs, FinalResult.NewFiles,
					UE::AssetRegistry::EScanFlags::None, nullptr /* OutFoundAssets */);
				ScanPathsSynchronous(&ScopeLock, Context);
			}
		}
	}
	ScanModifiedAssetFiles(ScopeLock, EventContext, InheritanceContext, FinalResult.ModifiedFiles, UE::AssetRegistry::EScanFlags::None);
}

}

void UAssetRegistryImpl::OnAssetLoaded(UObject *AssetLoaded)
{
	LLM_SCOPE(ELLMTag::AssetRegistry);
	AddLoadedAssetToProcess(*AssetLoaded);
}

void UAssetRegistryImpl::AddLoadedAssetToProcess(const UObject& AssetLoaded)
{
	LoadedAssetsToProcess.Enqueue(&AssetLoaded);
}

void UAssetRegistryImpl::ProcessLoadedAssetsToUpdateCache(UE::AssetRegistry::Impl::FEventContext& EventContext,
	UE::AssetRegistry::Impl::EGatherStatus Status, UE::AssetRegistry::Impl::FInterruptionContext& InOutInterruptionContext)
{
	// Note this function can be reentered due to arbitrary code execution in construction of FAssetData
	if (!IsInGameThread())
	{
		// Calls to GetAssetRegistryTags are only allowed on the GameThread
		return;
	}

	// Early exit to save cputime if we're still processing cache data
	if (IsTickActive(Status) && InOutInterruptionContext.IsTimeSlicingEnabled())
	{
		return;
	}

	constexpr int32 BatchSize = 16;
	TArray<const UObject*> BatchObjects;
	TArray<FAssetData, TInlineAllocator<BatchSize>> BatchAssetDatas;

	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		GuardedData.GetProcessLoadedAssetsBatch(LoadedAssetsToProcess, BatchObjects, BatchSize, bUpdateDiskCacheAfterLoad);
		if (BatchObjects.Num() == 0)
		{
			return;
		}

		// Refreshes ClassGeneratorNames if out of date due to module load
		GuardedData.CollectCodeGeneratorClasses();
	}

	while (BatchObjects.Num() > 0)
	{
		bool bTimedOut = false;
		int32 CurrentBatchSize = BatchObjects.Num();
		BatchAssetDatas.Reset(CurrentBatchSize);
		int32 Index = 0;
		while (Index < CurrentBatchSize)
		{
			const UObject* LoadedObject = BatchObjects[Index++];
			if (!LoadedObject->IsAsset())
			{
				// If the object has changed and is no longer an asset, ignore it. This can happen when an Actor is modified during cooking to no longer have an external package
				continue;
			}
			BatchAssetDatas.Add(FAssetData(LoadedObject, FAssetData::ECreationFlags::AllowBlueprintClass,
				EAssetRegistryTagsCaller::AssetRegistryLoad));

			// Check to see if we have run out of time in this tick
			if (InOutInterruptionContext.ShouldExitEarly())
			{
				bTimedOut = true;
				break;
			}
		}

		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		GuardedData.PushProcessLoadedAssetsBatch(EventContext, LoadedAssetsToProcess, BatchAssetDatas,
			TArrayView<const UObject*>(BatchObjects).Slice(Index, CurrentBatchSize-Index));
		if (bTimedOut)
		{
			break;
		}
		GuardedData.GetProcessLoadedAssetsBatch(LoadedAssetsToProcess, BatchObjects, BatchSize, bUpdateDiskCacheAfterLoad);
	}
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::GetProcessLoadedAssetsBatch(TQueue<TWeakObjectPtr<const UObject>, EQueueMode::Mpsc>& LoadedAssetsToProcess, TArray<const UObject*>& OutLoadedAssets, uint32 BatchSize,
	bool bUpdateDiskCacheAfterLoad)
{
	if (!GlobalGatherer.IsValid() || !bUpdateDiskCacheAfterLoad)
	{
		OutLoadedAssets.Reset();
		return;
	}

	OutLoadedAssets.Reset(BatchSize);
	TWeakObjectPtr<const UObject> AssetToProcess;
	while (OutLoadedAssets.Num() < static_cast<int32>(BatchSize) && LoadedAssetsToProcess.Dequeue(AssetToProcess))
	{
		const UObject* LoadedAsset = AssetToProcess.Get();
		if (!LoadedAsset)
		{
			// This could be null, in which case it already got freed, ignore
			continue;
		}

		FString LocalPath;
		if (UNLIKELY(!FPackageName::TryConvertLongPackageNameToFilename(LoadedAsset->GetPackage()->GetName(), LocalPath)))
		{
			continue;
		}

		if (!GlobalGatherer->IsMonitored(LocalPath))
		{
			continue;
		}

		// Take a new snapshot of the asset's data every time it loads or saves

		UPackage* InMemoryPackage = LoadedAsset->GetOutermost();
		if (InMemoryPackage->IsDirty())
		{
			// Package is dirty, which means it has changes other than just a PostLoad
			// In editor, ignore the update of the asset; it will be updated when saved
			// In the cook commandlet, in which editoruser-created changes are impossible, do the update anyway.
			// Occurrences of IsDirty in the cook commandlet are spurious and a code bug.
			if (!IsRunningCookCommandlet())
			{
				continue;
			}
		}

		OutLoadedAssets.Add(LoadedAsset);
	}
}

void FAssetRegistryImpl::PushProcessLoadedAssetsBatch(Impl::FEventContext& EventContext,
	TQueue<TWeakObjectPtr<const UObject>, EQueueMode::Mpsc>& LoadedAssetsToProcess,
	TArrayView<FAssetData> LoadedAssetDatas, TArrayView<const UObject*> UnprocessedFromBatch)
{
	// Add or update existing for all of the AssetDatas created by the batch
	for (FAssetData& NewAssetData : LoadedAssetDatas)
	{
		if (ShouldSkipGatheredAsset(NewAssetData))
		{
			continue;
		}
		FCachedAssetKey Key(NewAssetData);
		FAssetData* DataFromGather = State.GetMutableAssetByObjectPath(Key);

		AssetDataObjectPathsUpdatedOnLoad.Add(NewAssetData.GetSoftObjectPath());

		if (!DataFromGather)
		{
			FAssetData* ClonedAssetData = new FAssetData(MoveTemp(NewAssetData));
			AddAssetData(EventContext, ClonedAssetData);
		}
		else
		{
			// When updating disk-based AssetData with the AssetData from a loaded UObject, we keep
			// existing tags from disk even if they are not returned from the
			// GetAssetRegistryTags(EAssetRegistryTagsCaller::AssetRegistryLoad) function on the loaded UObject.
			// We do this because the tags might be tags that are only calculated during
			// GetAssetRegistryTags(EAssetRegistryTagsCaller::SavePackage).
			// Modified tag values on the other hand do overwrite the old values from disk.
			// This means that the only way to delete no-longer present tags from an AssetData
			// is to resave the package, or to manually call AssetUpdateTags(EAssetRegistryTagsCaller::FullUpdate) from c++.
			UpdateAssetData(EventContext, DataFromGather, MoveTemp(NewAssetData), true /* bKeepDeletedTags */);
		}
	}

	// Push back any objects from the batch that were not processed due to timing out
	for (const UObject* Obj : UnprocessedFromBatch)
	{
		LoadedAssetsToProcess.Enqueue(Obj);
	}
}

bool FAssetRegistryImpl::IsLoadedAndTrackedAsset(const FSoftObjectPath& InObjectPath) const
{
	// Must be in our "updated on load" set
	if (!AssetDataObjectPathsUpdatedOnLoad.Contains(InObjectPath))
	{
		return false;
	}

	// Must also be loaded in memory
	if (InObjectPath.ResolveObject() == nullptr)
	{
		return false;
	}

	return true;
}

}

#endif // WITH_EDITOR

void UAssetRegistryImpl::ScanModifiedAssetFiles(const TArray<FString>& InFilePaths)
{
	ScanModifiedAssetFiles(InFilePaths, UE::AssetRegistry::EScanFlags::None);
}

void UAssetRegistryImpl::ScanModifiedAssetFiles(const TArray<FString>& InFilePaths, UE::AssetRegistry::EScanFlags ScanFlags)
{
	UE::AssetRegistry::Impl::FEventContext EventContext;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		UE::AssetRegistry::Impl::FClassInheritanceContext InheritanceContext;
		UE::AssetRegistry::Impl::FClassInheritanceBuffer InheritanceBuffer;
		GetInheritanceContextWithRequiredLock(InterfaceScopeLock, InheritanceContext, InheritanceBuffer);
		GuardedData.ScanModifiedAssetFiles(InterfaceScopeLock, EventContext, InheritanceContext, InFilePaths, ScanFlags);
	}

#if WITH_EDITOR
	// Our caller expects up to date results after calling this function,
	// but in-memory results will override the on-disk results we just scanned,
	// and our in-memory results might be out of date due to being queued but not yet processed.
	// So ProcessLoadedAssetsToUpdateCache before returning to make sure results are up to date.
	UE::AssetRegistry::Impl::FInterruptionContext InterruptionContext;
	ProcessLoadedAssetsToUpdateCache(EventContext, UE::AssetRegistry::Impl::EGatherStatus::Complete, InterruptionContext);
#endif

	Broadcast(EventContext);
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::ScanModifiedAssetFiles(FInterfaceWriteScopeLock& ScopeLock, Impl::FEventContext& EventContext,
	Impl::FClassInheritanceContext& InheritanceContext, const TArray<FString>& InFilePaths,
	UE::AssetRegistry::EScanFlags InScanFlags)
{
	if (InFilePaths.Num() > 0)
	{
		// ScanModifiedAssetFiles always does a force rescan of the given files
		InScanFlags |= UE::AssetRegistry::EScanFlags::ForceRescan;

		// Re-scan and update the asset registry with the new asset data
		TArray<FSoftObjectPath> FoundAssets;
		Impl::FScanPathContext Context(EventContext, InheritanceContext, TArray<FString>(), InFilePaths,
			InScanFlags, &FoundAssets);
		ScanPathsSynchronous(&ScopeLock, Context);

		// Convert all the filenames to package names
		TArray<FString> ModifiedPackageNames;
		ModifiedPackageNames.Reserve(InFilePaths.Num());
		for (const FString& File : InFilePaths)
		{
			ModifiedPackageNames.Add(FPackageName::FilenameToLongPackageName(File));
		}

		// Get the assets that are currently inside the package
		TArray<FSoftObjectPath> ExistingAssetDatas;
		ExistingAssetDatas.Reserve(InFilePaths.Num());
		for (const FString& PackageName : ModifiedPackageNames)
		{
			TArray<const FAssetData*, TInlineAllocator<1>> PackageAssets;
			State.EnumerateAssetsByPackageName(*PackageName, [&PackageAssets](const FAssetData* AssetData)
				{
					PackageAssets.Add(AssetData);
					return true;
				});
			if (PackageAssets.Num() > 0)
			{
				ExistingAssetDatas.Reserve(ExistingAssetDatas.Num() + PackageAssets.Num());
				for (const FAssetData* AssetData : PackageAssets)
				{
					ExistingAssetDatas.Add(AssetData->ToSoftObjectPath());
				}
			}
		}

		// Remove any assets that are no longer present in the package
		for (FSoftObjectPath& OldAssetPath : ExistingAssetDatas)
		{
			if (!FoundAssets.Contains(OldAssetPath))
			{
				FAssetData* OldAssetData = State.GetMutableAssetByObjectPath(OldAssetPath);
				if (OldAssetData)
				{
					RemoveAssetData(EventContext, OldAssetData);
				}
			}
		}

		// Send ModifiedOnDisk event for every Asset that was modified
		for (const FSoftObjectPath& FoundAsset : FoundAssets)
		{
			const FAssetData* AssetData = State.GetAssetByObjectPath(FCachedAssetKey(FoundAsset));
			if (AssetData)
			{
				EventContext.AssetEvents.Emplace(*AssetData, Impl::FEventContext::EEvent::UpdatedOnDisk);
			}
		}
	}
}

}

void UAssetRegistryImpl::OnMountPointMounted(const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint)
{
	if (SuspendMountingCounter.load(std::memory_order_acquire) > 0)
	{
		FScopeLock Guard(&MountingCriticalSection);
		ListOfMountedMountPoints.Emplace(MountPoint);
		return;
	}

	OnMultipleMountPointsMounted(TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>>(&MountPoint, 1));
}

void UAssetRegistryImpl::OnMultipleMountPointsMounted(TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> MountedMountPoints)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AssetRegistry_OnMultipleMountPointsMounted);
	LLM_SCOPE(ELLMTag::AssetRegistry);

#if WITH_EDITOR
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
	IDirectoryWatcher* DirectoryWatcher = nullptr;
	if (UE::AssetRegistry::Impl::IsDirectoryWatcherEnabled())
	{
		DirectoryWatcher = DirectoryWatcherModule.Get();
		for (const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint : MountedMountPoints)
		{
			if (DirectoryWatcher)
			{
				// Make sure the directory exits on disk so that the OS level DirectoryWatcher can be used to monitor it.
				IPlatformFile::GetPlatformPhysical().CreateDirectoryTree(*WriteToString<512>(MountPoint->GetLocalPathStandard()));
			}
		}
	}
#endif

	UE::AssetRegistry::Impl::FEventContext EventContext;
	UE::AssetRegistry::Impl::FActiveMountEvents ActiveMountEvents;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		UE::AssetRegistry::Impl::FClassInheritanceContext InheritanceContext;
		UE::AssetRegistry::Impl::FClassInheritanceBuffer InheritanceBuffer;
		GetInheritanceContextWithRequiredLock(InterfaceScopeLock, InheritanceContext, InheritanceBuffer);
		GuardedData.OnMultipleMountPointsMounted(*this, EventContext, ActiveMountEvents,
			InheritanceContext, InterfaceScopeLock, MountedMountPoints);

		// Listen for directory changes in this content path
#if WITH_EDITOR
		for (const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint : MountedMountPoints)
		{
			const FString StandardFileSystemPath = UE::AssetRegistry::CreateStandardFilename(MountPoint->GetLocalPathStandard());
			// In-game doesn't listen for directory changes
			if (DirectoryWatcher && !IsDirAlreadyWatchedByRootWatchers(StandardFileSystemPath))
			{
				const FString AssetPathWithTrailingSlash = UE::AssetRegistry::CreateStandardLongPackageNameDirectory(MountPoint->GetLongPackageName());
				if (!OnDirectoryChangedDelegateHandles.Contains(AssetPathWithTrailingSlash))
				{
					FDelegateHandle NewHandle;
					DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(
						StandardFileSystemPath,
						IDirectoryWatcher::FDirectoryChanged::CreateUObject(this, &UAssetRegistryImpl::OnDirectoryChanged),
						NewHandle,
						IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges);

					OnDirectoryChangedDelegateHandles.Add(AssetPathWithTrailingSlash, NewHandle);
				}
			}
		}
#endif // WITH_EDITOR
	}

	Broadcast(EventContext);
	ActiveMountEvents.Broadcast();
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::OnMultipleMountPointsMounted(UAssetRegistryImpl& UARI, Impl::FEventContext& EventContext,
	Impl::FActiveMountEvents& ActiveMountEvents,
	Impl::FClassInheritanceContext& InheritanceContext,
	UE::AssetRegistry::FInterfaceWriteScopeLock& ScopeLock, 
	TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> MountedMountPoints)
{
	for (const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint : MountedMountPoints)
	{
		FString FileSystemPath(MountPoint->GetLocalPathStandard());
		const FString AssetPathWithTrailingSlash = UE::AssetRegistry::CreateStandardLongPackageNameDirectory(MountPoint->GetLongPackageName());

		// Content roots always exist
		AddPath(EventContext, UE::String::RemoveFromEnd(FStringView(AssetPathWithTrailingSlash), TEXTVIEW("/")));

		if (GlobalGatherer.IsValid() && bSearchAllAssets)
		{
			if (GlobalGatherer->IsSynchronous())
			{
				Impl::FScanPathContext Context(EventContext, InheritanceContext, { FileSystemPath }, TArray<FString>());
				ScanPathsSynchronous(&ScopeLock, Context);
			}
			else
			{
				if (!IsGathering())
				{
					CSV_TRACE_REGION_BEGIN(TEXT("Additional Mount Search"), TEXT("AssetRegistry")); // Matching CSV_TRACE_REGION_END in OnAdditionalMountSearchCompleted
					AdditionalMountSearchStartTime = FPlatformTime::Seconds();
					bAdditionalMountSearchInProgress.store(true, std::memory_order_relaxed);
					GlobalGatherer->SetIsAdditionalMountSearchInProgress(true);
#if WITH_EDITOR
					bGatherNeedsApplyObjectRedirects = true;
#endif
				}

				GlobalGatherer->AddMountPoint(MountPoint->GetLocalPathStandard(), MountPoint->GetLongPackageName());
				GlobalGatherer->SetIsOnAllowList(MountPoint->GetLocalPathStandard(), true);
			}
		}
	}

	UARI.ActiveMountsLoader.OnMountPointsMounted(UARI, ActiveMountEvents, MountedMountPoints);
}

}

void UAssetRegistryImpl::OnMountPointDismounted(const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint)
{
	if (SuspendDismountingCounter.load(std::memory_order_acquire) > 0)
	{
		FScopeLock Guard(&MountingCriticalSection);
		ListOfDismountedMountPoints.Emplace(MountPoint);
		return;
	}

	OnMultipleMountPointsDismounted(TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>>(&MountPoint, 1));
}

void UAssetRegistryImpl::OnMultipleMountPointsDismounted(TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> DismountedMountPoints)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AssetRegistry_OnMultipleMountPointsDismounted);
#if WITH_EDITOR
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
	IDirectoryWatcher* DirectoryWatcher = nullptr;
	if (UE::AssetRegistry::Impl::IsDirectoryWatcherEnabled())
	{
		DirectoryWatcher = DirectoryWatcherModule.Get();
	}
#endif

	UE::AssetRegistry::Impl::FEventContext EventContext;
	UE::AssetRegistry::Impl::FActiveMountEvents ActiveMountEvents;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		GuardedData.OnMultipleMountPointsDismounted(*this, EventContext, ActiveMountEvents,
			InterfaceScopeLock, DismountedMountPoints);

		// Stop listening for directory changes in this content path
#if WITH_EDITOR
		for (const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint : DismountedMountPoints)
		{
			const FString StandardFileSystemPath = UE::AssetRegistry::CreateStandardFilename(MountPoint->GetLocalPathStandard());
			if (DirectoryWatcher && !IsDirAlreadyWatchedByRootWatchers(StandardFileSystemPath))
			{
				// Make sure OnDirectoryChangedDelegateHandles key is symmetrical with the one used in OnMountPointMounted
				FString AssetPathWithTrailingSlash = UE::AssetRegistry::CreateStandardLongPackageNameDirectory(MountPoint->GetLongPackageName());
				FDelegateHandle DirectoryChangedHandle;
				if (ensure(OnDirectoryChangedDelegateHandles.RemoveAndCopyValue(AssetPathWithTrailingSlash, DirectoryChangedHandle)))
				{
					DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(StandardFileSystemPath, DirectoryChangedHandle);
				}
			}
		}
#endif // WITH_EDITOR
	}
	Broadcast(EventContext);
	ActiveMountEvents.Broadcast();
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::OnMultipleMountPointsDismounted(UAssetRegistryImpl& UARI,
	Impl::FEventContext& EventContext, Impl::FActiveMountEvents& ActiveMountEvents,
	UE::AssetRegistry::FInterfaceWriteScopeLock& ProofOfWriteLock,
	TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> DismountedMountPoints)
{
	UARI.ActiveMountsLoader.OnMountPointsDismounted(UARI, ActiveMountEvents, ProofOfWriteLock, DismountedMountPoints);

	// For the GlobalGatherer we remove all input MountPoints; we don't need to keep information about their gather.
	// But for the FAssetData stored inside our AssetRegistryState, for some mount points
	// we are configured to keep those assets in memory even when dismounted.
	TArray<TRefCountPtr<UE::PackageName::IMountPoint>> MountPointsToRemoveAssetsFrom;
	MountPointsToRemoveAssetsFrom.Reserve(DismountedMountPoints.Num());
	for (const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint : DismountedMountPoints)
	{
		if (GlobalGatherer.IsValid())
		{
			GlobalGatherer->RemoveMountPoint(MountPoint->GetLocalPathStandard());
		}

		// The input AssetPaths of the unmounted MountPoints are unnormalized and might have a trailing slash,
		// but we are comparing them by exact string inside our containers 
		// and need to normalize them to not have a trailing slash.
		FName AssetPathNoTrailingSlashFName = UE::AssetRegistry::CreateStandardLongPackageFNameNoTrailingSlash(
			MountPoint->GetLongPackageName());
		if (PersistentMountPoints.Contains(AssetPathNoTrailingSlashFName))
		{
			// This path is marked to never remove its AssetDatas.
			continue;
		}

		MountPointsToRemoveAssetsFrom.Add(MountPoint);
	}
	UnloadMountPoints(EventContext, MountPointsToRemoveAssetsFrom);
}

void FAssetRegistryImpl::UnloadMountPoints(Impl::FEventContext& EventContext,
	TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> MountPoints)
{
	// Collect the list of paths from given list of roots
	TSet<FName> AssetPathsToRemoveAssetsFrom;
	TArray<FName> MountPointRootNoTrailingSlashFNames;

	const bool bRecurse = true;
	MountPointRootNoTrailingSlashFNames.Reserve(MountPoints.Num());
	for (const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint : MountPoints)
	{
		FName AssetPathNoTrailingSlashFName = UE::AssetRegistry::CreateStandardLongPackageFNameNoTrailingSlash(
			MountPoint->GetLongPackageName());
		CachedPathTree.GetSubPaths(AssetPathNoTrailingSlashFName, AssetPathsToRemoveAssetsFrom, bRecurse);
		AssetPathsToRemoveAssetsFrom.Add(AssetPathNoTrailingSlashFName);
		MountPointRootNoTrailingSlashFNames.Add(AssetPathNoTrailingSlashFName);
	}

	// Remove all cached assets and Verse files found in the mountpoints.
	{
		TArray<FAssetData*> AllAssetDataToRemove;
		for (FName PathName : AssetPathsToRemoveAssetsFrom)
		{
			// Gather assets
			State.EnumerateMutableAssetsByPackagePath(PathName, [&AllAssetDataToRemove](FAssetData* AssetData)
				{
					AllAssetDataToRemove.Add(AssetData);
					return true;
				});

			// Forget Verse files
			const TArray<FName>* VerseFilesInPath = CachedVerseFilesByPath.Find(PathName);
			if (VerseFilesInPath)
			{
				for (FName FilePath : *VerseFilesInPath)
				{
					CachedVerseFiles.Remove(FilePath);
				}
				CachedVerseFilesByPath.Remove(PathName);
			}
		}

		RemoveAssetDatas(EventContext, AllAssetDataToRemove);
	}

	// Remove the root paths of the mountpoints.
	for (FName AssetPathNoTrailingSlashFName : MountPointRootNoTrailingSlashFNames)
	{
		const bool bEvenIfAssetsStillExist = true;
		RemoveAssetPath(EventContext, AssetPathNoTrailingSlashFName, bEvenIfAssetsStillExist);
	}
}

void FAssetRegistryImpl::UpdatePersistentMountPoints()
{
	State.EnumerateAllPaths([this](FName Path)
		{
			TStringBuilder<256> PathString(InPlace, Path);
			bool bHadClassesPrefix;
			FStringView MountPoint = FPathViews::GetMountPointNameFromPath(PathString, &bHadClassesPrefix, false /* bInWithoutSlashes*/);
			if (!MountPoint.IsEmpty() && !bHadClassesPrefix)
			{
				// Format returned by GetMountPointNameFromPath is e.g. /Engine, which is the format we need:
				// LongPackageName with no trailing slash
				PersistentMountPoints.Add(FName(MountPoint));
			}
		});
}

}

void UAssetRegistryImpl::SetTemporaryCachingMode(bool bEnable)
{
	checkf(IsInGameThread(), TEXT("Changing Caching mode is only available on the game thread because it affects behavior on all threads"));
	LLM_SCOPE(ELLMTag::AssetRegistry);
	UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
	GuardedData.SetTemporaryCachingMode(bEnable);
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::SetTemporaryCachingMode(bool bEnable)
{
	if (bIsTempCachingAlwaysEnabled || bEnable == bIsTempCachingEnabled)
	{
		return;
	}

	bIsTempCachingEnabled = bEnable;
	TempCachedInheritanceBuffer.bDirty = true;
	if (!bEnable)
	{
		TempCachedInheritanceBuffer.Clear();
	}
}

}

void UAssetRegistryImpl::SetTemporaryCachingModeInvalidated()
{
	checkf(IsInGameThread(), TEXT("Invalidating temporary cache is only available on the game thread because it affects behavior on all threads"));
	LLM_SCOPE(ELLMTag::AssetRegistry);
	UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
	GuardedData.SetTemporaryCachingModeInvalidated();
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::SetTemporaryCachingModeInvalidated()
{
	TempCachedInheritanceBuffer.bDirty = true;
}

}

bool UAssetRegistryImpl::GetTemporaryCachingMode() const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	return GuardedData.IsTempCachingEnabled();
}

namespace UE::AssetRegistry
{

void FAssetRegistryImpl::AddCachedBPClassParent(const FTopLevelAssetPath& ClassPath, const FTopLevelAssetPath& NotYetRedirectedParentPath)
{
	// We do not check for CoreRedirects for ClassPath, because this function is only called on behalf of ClassPath being loaded,
	// and the code author would have changed the package containing ClassPath to match the redirect they added.
	// But we do need to check for CoreRedirects in the ParentPath, because when a parent class is renamed, we do not resave
	// all packages containing subclasses to update their FBlueprintTags::ParentClassPath AssetData tags.
	FTopLevelAssetPath ParentPath = NotYetRedirectedParentPath;
#if WITH_EDITOR
	FCoreRedirectObjectName RedirectedParentObjectName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class,
		FCoreRedirectObjectName(NotYetRedirectedParentPath.GetAssetName(), NAME_None, NotYetRedirectedParentPath.GetPackageName()));
	if (!RedirectedParentObjectName.OuterName.IsNone())
	{
		UE_LOGF(LogAssetRegistry, Error,
			"Class redirect exists from %ls -> %ls, which is invalid because ClassNames must be TopLevelAssetPaths. " "Redirect will be ignored in AssetRegistry queries.",
			*NotYetRedirectedParentPath.ToString(), *RedirectedParentObjectName.ToString());
	}
	else
	{
		ParentPath = FTopLevelAssetPath(RedirectedParentObjectName.PackageName, RedirectedParentObjectName.ObjectName);
	}
#endif
	CachedBPInheritanceMap.Add(ClassPath, ParentPath);
}

void FAssetRegistryImpl::UpdateInheritanceBuffer(Impl::FClassInheritanceBuffer& OutBuffer) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UAssetRegistryImpl::UpdateTemporaryCaches)
	LLM_SCOPE(ELLMTag::AssetRegistry);
	UE_TRACK_REFERENCING_OPNAME_SCOPED(PackageAccessTrackingOps::NAME_ResetContext);

	TMap<UClass*, TSet<UClass*>> NativeSubclasses = GetAllDerivedClasses();

	uint32 NumNativeClasses = 1; // UObject has no superclass
	for (const TPair<UClass*, TSet<UClass*>>& Pair : NativeSubclasses)
	{
		NumNativeClasses += Pair.Value.Num();
	}
	OutBuffer.InheritanceMap.Reserve(NumNativeClasses + CachedBPInheritanceMap.Num());
	OutBuffer.InheritanceMap = CachedBPInheritanceMap;
	OutBuffer.InheritanceMap.Add(UE::AssetRegistry::GetClassPathObject(), FTopLevelAssetPath());

	for (TPair<FTopLevelAssetPath, TArray<FTopLevelAssetPath>>& Pair : OutBuffer.ReverseInheritanceMap)
	{
		Pair.Value.Reset();
	}
	OutBuffer.ReverseInheritanceMap.Reserve(NativeSubclasses.Num());

	for (const TPair<UClass*, TSet<UClass*>>& Pair : NativeSubclasses)
	{
		FTopLevelAssetPath SuperclassName = Pair.Key->GetClassPathName();

		TArray<FTopLevelAssetPath>* OutputSubclasses = &OutBuffer.ReverseInheritanceMap.FindOrAdd(SuperclassName);
		OutputSubclasses->Reserve(Pair.Value.Num());
		for (UClass* Subclass : Pair.Value)
		{
			if (!Subclass->HasAnyClassFlags(CLASS_Deprecated | CLASS_NewerVersionExists))
			{
				FTopLevelAssetPath SubclassName = Subclass->GetClassPathName();
				OutputSubclasses->Add(SubclassName);
				OutBuffer.InheritanceMap.Add(SubclassName, SuperclassName);

				if (Subclass->Interfaces.Num())
				{
					// Add any implemented interfaces to the reverse inheritance map, but not to the forward map
					for (const FImplementedInterface& Interface : Subclass->Interfaces)
					{
						if (UClass* InterfaceClass = Interface.Class) // could be nulled out by ForceDelete of a blueprint interface
						{
							TArray<FTopLevelAssetPath>& Implementations = OutBuffer.ReverseInheritanceMap.FindOrAdd(InterfaceClass->GetClassPathName());
							Implementations.Add(SubclassName);
						}
					}

					// Refetch OutputSubClasses from ReverseInheritanceMap because we just modified the ReverseInheritanceMap and may have resized
					OutputSubclasses = OutBuffer.ReverseInheritanceMap.Find(SuperclassName);
					check(OutputSubclasses); // It was added above
				}
			}
		}
	}

	// Add non-native classes to reverse map
	for (const TPair<FTopLevelAssetPath, FTopLevelAssetPath>& Kvp : CachedBPInheritanceMap)
	{
		const FTopLevelAssetPath& ParentClassName = Kvp.Value;
		if (!ParentClassName.IsNull())
		{
			TArray<FTopLevelAssetPath>& ChildClasses = OutBuffer.ReverseInheritanceMap.FindOrAdd(ParentClassName);
			ChildClasses.Add(Kvp.Key);
		}

	}

	OutBuffer.SavedAllClassesVersionNumber = GetCurrentAllClassesVersionNumber();
	OutBuffer.bDirty = false;
}

}

void UAssetRegistryImpl::GetInheritanceContextWithRequiredLock(UE::AssetRegistry::FInterfaceRWScopeLock& InOutScopeLock,
	UE::AssetRegistry::Impl::FClassInheritanceContext& InheritanceContext,
	UE::AssetRegistry::Impl::FClassInheritanceBuffer& StackBuffer)
{
	using namespace UE::AssetRegistry;

	uint64 CurrentGeneratorClassesVersionNumber = FAssetRegistryImpl::GetCurrentGeneratorClassesVersionNumber();
	uint64 CurrentAllClassesVersionNumber = FAssetRegistryImpl::GetCurrentAllClassesVersionNumber();
	bool bNeedsWriteLock = false;
	if (GuardedData.GetSavedGeneratorClassesVersionNumber() != CurrentGeneratorClassesVersionNumber)
	{
		// ConditionalUpdate writes to protected data in CollectCodeGeneratorClasses, so we cannot proceed under a read lock
		bNeedsWriteLock = true;
	}
	if (GuardedData.IsTempCachingEnabled() &&
		!GuardedData.GetTempCachedInheritanceBuffer().IsUpToDate(CurrentAllClassesVersionNumber))
	{
		// Temp caching is enabled, so we will be reading the protected data in TempCachedInheritanceBuffer
		// It's out of date, so we need to write to it first, so we cannot proceed under a read lock
		bNeedsWriteLock = true;
	}
	if (bNeedsWriteLock)
	{
		InOutScopeLock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();
	}

	// Note that we have to reread all data since we may have dropped the lock
	InheritanceContext.StackBuffer = &StackBuffer;
	InheritanceContext.SelectBuffer(GuardedData, CurrentGeneratorClassesVersionNumber, CurrentAllClassesVersionNumber);
}

void UAssetRegistryImpl::GetInheritanceContextWithRequiredLock(UE::AssetRegistry::FInterfaceWriteScopeLock& InOutScopeLock,
	UE::AssetRegistry::Impl::FClassInheritanceContext& InheritanceContext,
	UE::AssetRegistry::Impl::FClassInheritanceBuffer& StackBuffer)
{
	using namespace UE::AssetRegistry;

	uint64 CurrentGeneratorClassesVersionNumber = FAssetRegistryImpl::GetCurrentGeneratorClassesVersionNumber();
	uint64 CurrentAllClassesVersionNumber = FAssetRegistryImpl::GetCurrentAllClassesVersionNumber();
	InheritanceContext.StackBuffer = &StackBuffer;
	InheritanceContext.SelectBuffer(GuardedData, CurrentGeneratorClassesVersionNumber, CurrentAllClassesVersionNumber);
}

#if WITH_EDITOR
void UAssetRegistryImpl::OnGetExtraObjectTags(FAssetRegistryTagsContext Context)
{
	if (bAddMetaDataTagsToOnGetExtraObjectTags)
	{
		// Adding metadata tags from disk is only necessary for cooked assets; uncooked assets still have the metadata and add them elsewhere
		// in UObject::GetAssetRegistryTags. Adding the tags from disk into uncooked assets would make the tags impossible to remove when
		// the uncooked assets are resaved.
		if (Context.GetObject()->GetPackage()->HasAnyPackageFlags(PKG_Cooked))
		{
			// It is critical that bIncludeOnlyOnDiskAssets=true otherwise this will cause an infinite loop
			const FAssetData AssetData = GetAssetByObjectPath(FSoftObjectPath::ConstructFromObject(Context.GetObject()), /*bIncludeOnlyOnDiskAssets=*/true);
			TSet<FName>& MetaDataTags = UObject::GetMetaDataTagsForAssetRegistry();
			for (const FName MetaDataTag : MetaDataTags)
			{
				auto OutTagsContainsTagPredicate = [MetaDataTag](const UObject::FAssetRegistryTag& Tag) { return Tag.Name == MetaDataTag; };
				if (!Context.ContainsTag(MetaDataTag))
				{
					FAssetTagValueRef TagValue = AssetData.TagsAndValues.FindTag(MetaDataTag);
					if (TagValue.IsSet())
					{
						Context.AddTag(UObject::FAssetRegistryTag(MetaDataTag, TagValue.AsString(), UObject::FAssetRegistryTag::TT_Alphabetical));
					}
				}
			}
		}
	}
}

bool UAssetRegistryImpl::IsDirAlreadyWatchedByRootWatchers(const FString& Directory) const
{
	return DirectoryWatchRoots.ContainsByPredicate([&Directory](const FString& WatchRoot) -> bool {
		return FPaths::IsUnderDirectory(Directory, WatchRoot);
	});
}

#endif

void UAssetRegistryImpl::SuspendPathsMounting()
{
	SuspendMountingCounter.fetch_add(1, std::memory_order_acq_rel);
}

void UAssetRegistryImpl::ResumePathsMounting()
{
	if (SuspendMountingCounter.fetch_sub(1, std::memory_order_acq_rel) == 1)
	{
		TArray<TRefCountPtr<UE::PackageName::IMountPoint>> LocalList;
		{
			FScopeLock Guard(&MountingCriticalSection);
			LocalList = MoveTemp(ListOfMountedMountPoints);
		}
		if (!LocalList.IsEmpty())
		{
			OnMultipleMountPointsMounted(LocalList);
		}
	}
}

void UAssetRegistryImpl::SuspendPathsDismounting()
{
	SuspendDismountingCounter.fetch_add(1, std::memory_order_acq_rel);
}

void UAssetRegistryImpl::ResumePathsDismounting()
{
	if (SuspendDismountingCounter.fetch_sub(1, std::memory_order_acq_rel) == 1)
	{
		TArray<TRefCountPtr<UE::PackageName::IMountPoint>> LocalList;
		{
			FScopeLock Guard(&MountingCriticalSection);
			LocalList = MoveTemp(ListOfDismountedMountPoints);
		}
		if (!LocalList.IsEmpty())
		{
			OnMultipleMountPointsDismounted(LocalList);
		}
	}
}

void UAssetRegistryImpl::RequestPauseBackgroundProcessing()
{
#if WITH_EDITOR
	GuardedData.RequestPauseBackgroundProcessing();
#endif
}

void UAssetRegistryImpl::RequestResumeBackgroundProcessing()
{
#if WITH_EDITOR
	GuardedData.RequestResumeBackgroundProcessing();
#endif
}

namespace UE::AssetRegistry
{

namespace Impl
{

void FClassInheritanceBuffer::Clear()
{
	InheritanceMap.Empty();
	ReverseInheritanceMap.Empty();
}

bool FClassInheritanceBuffer::IsUpToDate(uint64 CurrentAllClassesVersionNumber) const
{
	return !bDirty && SavedAllClassesVersionNumber == CurrentAllClassesVersionNumber;
}

SIZE_T FClassInheritanceBuffer::GetAllocatedSize() const
{
	return InheritanceMap.GetAllocatedSize() + ReverseInheritanceMap.GetAllocatedSize();
}

void FClassInheritanceContext::BindToBuffer(FClassInheritanceBuffer& InBuffer, FAssetRegistryImpl& InAssetRegistryImpl, 
	bool bInInheritanceMapUpToDate, bool bInCodeGeneratorClassesUpToDate)
{
	AssetRegistryImpl = &InAssetRegistryImpl;
	Buffer = &InBuffer;
	bInheritanceMapUpToDate = bInInheritanceMapUpToDate;
	bCodeGeneratorClassesUpToDate = bInCodeGeneratorClassesUpToDate;
}

void FClassInheritanceContext::ConditionalUpdate()
{
	check(Buffer != nullptr); // It is not valid to call ConditionalUpdate with an empty FClassInheritanceContext
	if (bInheritanceMapUpToDate)
	{
		return;
	}

	if (!bCodeGeneratorClassesUpToDate)
	{
		AssetRegistryImpl->CollectCodeGeneratorClasses();
		bCodeGeneratorClassesUpToDate = true;
	}
	AssetRegistryImpl->UpdateInheritanceBuffer(*Buffer);
	bInheritanceMapUpToDate = true;
}

void FClassInheritanceContext::SelectBuffer(FAssetRegistryImpl& GuardedData, uint64 CurrentGeneratorClassesVersionNumber, uint64 CurrentAllClassesVersionNumber)
{
	// If bIsTempCachingAlwaysEnabled, then we are guaranteed that bIsTempCachingEnabled=true.
	// We rely on this to simplify logic and only check bIsTempCachingEnabled
	check(!GuardedData.IsTempCachingAlwaysEnabled() || GuardedData.IsTempCachingEnabled());

	bool bLocalCodeGeneratorClassesUpToDate = GuardedData.GetSavedGeneratorClassesVersionNumber() == CurrentGeneratorClassesVersionNumber;
	if (GuardedData.IsTempCachingEnabled())
	{
		// Use the persistent buffer
		UE::AssetRegistry::Impl::FClassInheritanceBuffer& TempCachedInheritanceBuffer = GuardedData.GetTempCachedInheritanceBuffer();
		bool bLocalInheritanceMapUpToDate = TempCachedInheritanceBuffer.IsUpToDate(CurrentAllClassesVersionNumber);
		BindToBuffer(TempCachedInheritanceBuffer, GuardedData, bLocalInheritanceMapUpToDate, bLocalCodeGeneratorClassesUpToDate);
	}
	else
	{
		// Use the StackBuffer for the duration of the caller
		check(StackBuffer);
		BindToBuffer(*StackBuffer, GuardedData, false /* bInInheritanceMapUpToDate */, bLocalCodeGeneratorClassesUpToDate);
	}
}

void FClassInheritanceContext::OnLockReentered(FAssetRegistryImpl& GuardedData)
{
	uint64 CurrentGeneratorClassesVersionNumber = FAssetRegistryImpl::GetCurrentGeneratorClassesVersionNumber();
	uint64 CurrentAllClassesVersionNumber = FAssetRegistryImpl::GetCurrentAllClassesVersionNumber();
	SelectBuffer(GuardedData, CurrentGeneratorClassesVersionNumber, CurrentAllClassesVersionNumber);
}
}

void FAssetRegistryImpl::GetSubClasses(Impl::FClassInheritanceContext& InheritanceContext,
	const TArray<FTopLevelAssetPath>& InClassNames, const TSet<FTopLevelAssetPath>& ExcludedClassNames, TSet<FTopLevelAssetPath>& SubClassNames) const
{
	InheritanceContext.ConditionalUpdate();

	TSet<FTopLevelAssetPath> ProcessedClassNames;
	for (const FTopLevelAssetPath& ClassName : InClassNames)
	{
		// Now find all subclass names
		GetSubClasses_Recursive(InheritanceContext, ClassName, SubClassNames, ProcessedClassNames, ExcludedClassNames);
	}
}

void FAssetRegistryImpl::GetSubClasses_Recursive(Impl::FClassInheritanceContext& InheritanceContext, FTopLevelAssetPath InClassName,
	TSet<FTopLevelAssetPath>& SubClassNames, TSet<FTopLevelAssetPath>& ProcessedClassNames, const TSet<FTopLevelAssetPath>& ExcludedClassNames) const
{
	if (ExcludedClassNames.Contains(InClassName))
	{
		// This class is in the exclusion list. Exclude it.
	}
	else if (ProcessedClassNames.Contains(InClassName))
	{
		// This class has already been processed. Ignore it.
	}
	else
	{
		SubClassNames.Add(InClassName);
		ProcessedClassNames.Add(InClassName);

		auto AddSubClasses = [this, &InheritanceContext, &SubClassNames, &ProcessedClassNames, &ExcludedClassNames]
		(FTopLevelAssetPath ParentClassName)
		{
			const TArray<FTopLevelAssetPath>* FoundSubClassNames = InheritanceContext.Buffer->ReverseInheritanceMap.Find(ParentClassName);
			if (FoundSubClassNames)
			{
				for (FTopLevelAssetPath ClassName : (*FoundSubClassNames))
				{
					GetSubClasses_Recursive(InheritanceContext, ClassName, SubClassNames, ProcessedClassNames,
						ExcludedClassNames);
				}
			}
		};

		// Add Subclasses of the given classname
		AddSubClasses(InClassName);
	}
}


#if WITH_EDITOR
void FAssetRegistryImpl::RequestPauseBackgroundProcessing()
{
	if (GlobalGatherer.IsValid())
	{
		GlobalGatherer->PauseProcessing();
	}
}

void FAssetRegistryImpl::RequestResumeBackgroundProcessing()
{
	if (GlobalGatherer.IsValid())
	{
		GlobalGatherer->ResumeProcessing();
	}
}

bool FAssetRegistryImpl::IsBackgroundProcessingPaused() const
{
	if (GlobalGatherer.IsValid())
	{
		return GlobalGatherer->IsProcessingPauseRequested();
	}
	return true;
}
#endif

}

#if WITH_EDITOR
static FString GAssetRegistryManagementPathsPackageDebugName;
static FAutoConsoleVariableRef CVarAssetRegistryManagementPathsPackageDebugName(
	TEXT("AssetRegistry.ManagementPathsPackageDebugName"),
	GAssetRegistryManagementPathsPackageDebugName,
	TEXT("If set, when manage references are set, the chain of references that caused this package to become managed will be printed to the log"));
#endif

namespace UE::AssetRegistry
{

/**
 * In SetManagerReferences, we execute a graph search within the graph of PackageDependencies for Assets to decide
 * which Assets should be marked as having Managed properties. We use a graph search painting algorithm to
 * apply multiple properties within a single graph search. A painting algorithm keeps track of the current list of
 * properties to apply to each visited vertex, and filters that list of properties depending on the edge properties
 * when traversing an edge. The list of Managed properties we paint during SetManageReferences is given by the values
 * of EManageSearchColor.
 */
enum class EManageSearchColor : uint8
{
	CookRule = 0,
	Chunking = 1,
	Num,
};
constexpr int32 ManageSearchColorsNum = static_cast<int32>(EManageSearchColor::Num);
static_assert(ManageSearchColorsNum <= 8 * sizeof(uint32), "We store lists of SearchColors as a uint32 bitfield.");
enum class EManageSearchColorBit : uint32
{
	None = 0,
	CookRule = (1 << static_cast<int32>(EManageSearchColor::CookRule)),
	Chunking = (1 << static_cast<int32>(EManageSearchColor::Chunking)),
	All = CookRule | Chunking,
};
ENUM_CLASS_FLAGS(EManageSearchColorBit);

/**
 * Data held for each Asset (aka node in our graph search) across the multiple graph searches conducted within
 * SetManageReferences, and across multiple calls to SetManageReferences
 * 
 * Each call to SetManageReferences is called a round. The rounds occur in descending priority order of managers;
 * each round does a graph search for the nodes reported as managed by managers in that round. All the managers in
 * a given round have the same priority.
 *
 */
struct FSetManageReferencesNodeData
{
	// Values preserved across multiple rounds
	/**
	 * ManagedInEarlierRound bit (1 << N) is set at the end of a round for assets painted with ((EManageSearchColor)N) 
	 * in the round, and later rounds can use that information in ShouldSetManager calls to e.g. ignore the management
	 * of a node by a lower-priority manager.
	 */
	EManageSearchColorBit ManagedInEarlierRound = EManageSearchColorBit::None;

	// Values reset for each round.
	/**
	 * Whether the asset was modified by the round (and therefore needs to have transient properties restored at the
	 * end of the round.
	 */
	bool bModifiedByRound = false;
	/**
	 * ManagedInThisRound bit (1 << N) is set during a round's graph search for the asset painted with
	 * ((EManageSearchColor)N) in the round.
	 */
	EManageSearchColorBit ManagedInThisRound = EManageSearchColorBit::None;
	/**
	 * Whether the asset has a direct manager in the current round, as described by the caller of the round in
	 * FSetManageReferencesContext.ManagerMap. Direct Managers apply to all EManageSearchColors.
	 */
	bool bDirectManagedInThisRound = false;

	/**
	 * For a manager node in the current round, the list of directly managed assets. Directly managed assets paint
	 * with all EManageSearchColors.
	 */
	TArray<FDependsNode*> DirectManagedAssetsThisRound;

	/**
	 * DebugInstigator[N] is the source node (aka asset) that caused this node to be painted with
	 * ((EManageSearchColor)N) in the current round. This information is used to provide instigator chains for what
	 * caused an asset to be added to the cook or to a chunk.
	 */
	FDependsNode* DebugInstigator[ManageSearchColorsNum] = {};

	// Values reset for the graph search from each manager with a round.
	/**
	 * Whether the asset was modified by the manager's search (and therefore needs to have transient properties
	 * restored at the end of the search).
	 */
	bool bModifiedByCurrentManager = false;
	/**
	 * VisitedByCurrentManager bit (1 <<N) records whether the asset was found to be painted with
	 * ((EManageSearchColor)N) by the manager.
	 */
	EManageSearchColorBit VisitedByCurrentManager = EManageSearchColorBit::None;
	/**
	 * The AssetRegistry dependency properties that should be assigned to the targetnode for the Management edge we
	 * create for the node from the current manager; this accumulates the properties we set for each painted color,
	 * as well as the special EDependencyProperty::Direct property.
	 */
	EDependencyProperty CurrentManagerProperties = EDependencyProperty::None;
};

using FSetManageReferencesNodeDataMap = TMap<FDependsNode*, TUniquePtr<FSetManageReferencesNodeData>>;
struct FSetManageReferencesScratch
{
	FSetManageReferencesNodeDataMap NodeData;
};

#if WITH_EDITOR
void PrintAssetRegistryManagementPathsPackageDebugInfo(FDependsNode* Node, UE::AssetRegistry::FSetManageReferencesNodeDataMap* NodeData)
{
	if (Node)
	{
		UE_LOGF(LogAssetRegistry, Display, "SetManageReferences is printing out the reference chain that caused '%ls' to be managed", *GAssetRegistryManagementPathsPackageDebugName);
		TSet<FDependsNode*> AllVisitedNodes;
		for (;;)
		{
			TUniquePtr<FSetManageReferencesNodeData>* CurrentNodeData = NodeData->Find(Node);
			if (!CurrentNodeData)
			{
				break;
			}
			FDependsNode* ReferencingNode = (*CurrentNodeData)->DebugInstigator[(int32)EManageSearchColor::Chunking];
			if (!ReferencingNode)
			{
				break;
			}

			UE_LOGF(LogAssetRegistry, Display, "  %ls", *ReferencingNode->GetIdentifier().ToString());
			if (AllVisitedNodes.Contains(ReferencingNode))
			{
				UE_LOGF(LogAssetRegistry, Display, "  ... (Circular reference back to %ls)", *ReferencingNode->GetPackageName().ToString());
				break;
			}

			AllVisitedNodes.Add(ReferencingNode);
			Node = ReferencingNode;
		}
	}
	else
	{
		UE_LOGF(LogAssetRegistry, Warning, "Node with AssetRegistryManagementPathsPackageDebugName '%ls' was not found", *GAssetRegistryManagementPathsPackageDebugName);
	}
}
#endif // WITH_EDITOR

} // namespace UE::AssetRegistry

void UAssetRegistryImpl::SetManageReferences(const TMultiMap<FAssetIdentifier, FAssetIdentifier>& ManagerMap,
	bool bClearExisting, UE::AssetRegistry::EDependencyCategory RecurseType, TSet<FDependsNode*>& ExistingManagedNodes,
	ShouldSetManagerPredicate ShouldSetManager)
{
	if (!ShouldSetManager)
	{
		ShouldSetManager = [](const FAssetIdentifier& Manager, const FAssetIdentifier& Source, const FAssetIdentifier& Target,
			UE::AssetRegistry::EDependencyCategory Category, UE::AssetRegistry::EDependencyProperty Properties,
			EAssetSetManagerFlags::Type Flags)
			{
				return EAssetSetManagerResult::SetButDoNotRecurse;
			};
	}
	auto ShouldSetManagerByContext = [&ShouldSetManager](UE::AssetRegistry::FShouldSetManagerContext& ShouldSetContext)
		{
			return ShouldSetManager(ShouldSetContext.Manager, ShouldSetContext.Source, ShouldSetContext.Target,
				ShouldSetContext.EdgeARCategory, ShouldSetContext.EdgeARProperties, ShouldSetContext.EdgeFlags);
		};

	UE::AssetRegistry::FSetManageReferencesContext Context;
	Context.ManagerMap = &ManagerMap;
	Context.bClearExisting = bClearExisting;
	Context.RecurseType = RecurseType;
	Context.ShouldSetManager = ShouldSetManagerByContext;
	SetManageReferences(Context);
}

void UAssetRegistryImpl::SetManageReferences(UE::AssetRegistry::FSetManageReferencesContext& Context)
{
	// For performance reasons we call the ShouldSetManager callback when inside the lock. Licensee UAssetManagers
	// are responsible for not calling AssetRegistry functions from ShouldSetManager as that would create a deadlock
	LLM_SCOPE(ELLMTag::AssetRegistry);
	UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
	GuardedData.SetManageReferences(Context);
}

namespace UE::AssetRegistry
{

static EAssetSetManagerAssetFlags ConstructAssetManagerAssetFlags(const FDependsNode& DependsNode)
{
	return DependsNode.IsScriptPath() ? EAssetSetManagerAssetFlags::ScriptPackage
		: EAssetSetManagerAssetFlags::None;
};

void FAssetRegistryImpl::SetManageReferences(FSetManageReferencesContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAssetRegistryImpl::SetManageReferences);
	if (!Context.ManagerMap)
	{
		UE_LOGF(LogAssetRegistry, Error, "SetManageReferences: Context.ManagerMap must not be null.");
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
		return;
	}
	if (!Context.ShouldSetManager.IsSet())
	{
		UE_LOGF(LogAssetRegistry, Error, "SetManageReferences: Context.ShouldSetManager must be a bound pointer.");
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
		return;
	}

	FSetManageReferencesScratch* Scratch = nullptr;
	FSetManageReferencesNodeDataMap* NodeData = nullptr;
	// TODO: Invalidate Scratch and give an error if dependsnodes are written in between calls; Scratch has pointers
	// and cached data into the AssetRegistry's list of DependsNodes.

	TArray<FDependsNode*> CurrentRoundModifiedNodes;

	if (!Context.Scratch)
	{
		Context.Scratch = MakePimpl<FSetManageReferencesScratch>();
		Context.Scratch->NodeData.Reserve(State.CachedDependsNodes.Num());
	}
	Scratch = Context.Scratch.Get();
	NodeData = &Scratch->NodeData;

	// We use a TMap of TUniquePtr rather than direct c++ member data so that we can have persistent pointers to
	// NodeDatas throughout the graph search, rather than possibly have our data pointers be invalidated by a
	// reallocation when a new node is added. This has a cputime cost cost - an extra memory access per read.
	// It also has a complexity cost, because instead of calling FindOrAdd to create if non-existing, we also have
	// to check whether it was added and allocate the TUniquePtr if so. This FindOrAddNodeData handles that
	// complexity of creation.
	auto FindOrAddNodeData = [NodeData](FDependsNode* Node) -> FSetManageReferencesNodeData&
		{
			TUniquePtr<FSetManageReferencesNodeData>& Existing = NodeData->FindOrAdd(Node);
			if (!Existing)
			{
				Existing.Reset(new FSetManageReferencesNodeData());
			}
			return *Existing;
		};

	if (Context.bClearExisting)
	{
		// Clear information from Scratch about Earlier rounds
		for (TPair<FDependsNode*, TUniquePtr<FSetManageReferencesNodeData>>& NodePair : *NodeData)
		{
			NodePair.Value->ManagedInEarlierRound = EManageSearchColorBit::None;
		}

		// Clear all outgoing manage dependencies, and update the referencers of any nodes with incoming
		// manage dependencies
		TSet<FDependsNode*> NodesToRefreshReferencers;
		for (const TPair<FAssetIdentifier, FDependsNode*>& Pair : State.CachedDependsNodes)
		{
			FDependsNode* SourceNode = Pair.Value;
			SourceNode->IterateOverDependencies([&NodesToRefreshReferencers]
			(FDependsNode* TargetNode, EDependencyCategory Category, EDependencyProperty Property, bool bUnique)
				{
					NodesToRefreshReferencers.Add(TargetNode);
				}, EDependencyCategory::Manage);
			SourceNode->ClearDependencies(EDependencyCategory::Manage);
		}

		for (FDependsNode* NodeToClear : NodesToRefreshReferencers)
		{
			NodeToClear->SetIsReferencersSorted(false);
			NodeToClear->RefreshReferencers();

			CurrentRoundModifiedNodes.Add(NodeToClear);
			FSetManageReferencesNodeData& ModifiedData = FindOrAddNodeData(NodeToClear);
			ModifiedData.bModifiedByRound = true;
		}
	}

	TArray<FDependsNode*> CurrentRoundManagers;
	TArray<FDependsNode*> CurrentManagerModifiedNodes;
	TArray<TPair<FDependsNode*, EManageSearchColorBit>> CurrentManagerVisitQueue;

	// Convert ManageMap from edges in the FAssetIdentifier graph into edges in the FDependsNode graph.
	// Also mark each explicitly managed node; AssetManager's ShouldSetManager function might decide to skip
	// an indirect manage reference if there is a direct manage reference in the same round.
	CurrentRoundManagers.Reserve(Context.ManagerMap->Num());
	for (const TPair<FAssetIdentifier, FAssetIdentifier>& Pair : *Context.ManagerMap)
	{
		FDependsNode* ManagedNode = State.FindDependsNode(Pair.Value);
		if (!ManagedNode)
		{
			UE_LOGF(LogAssetRegistry, Warning, "Cannot set %ls to manage asset %ls because %ls does not exist!", 
				*Pair.Key.ToString(), *Pair.Value.ToString(), *Pair.Value.ToString());
			continue;
		}
		FDependsNode* ManagerNode = State.CreateOrFindDependsNode(Pair.Key);

		CurrentRoundManagers.Add(ManagerNode);
		FindOrAddNodeData(ManagedNode).bDirectManagedInThisRound = true;
		FindOrAddNodeData(ManagerNode).DirectManagedAssetsThisRound.Add(ManagedNode);
	}

	Algo::Sort(CurrentRoundManagers);
	CurrentRoundManagers.SetNum(Algo::Unique(CurrentRoundManagers));

	// For each manager, do a graph search starting from its directly managed nodes.
	for (FDependsNode* ManagerNode : CurrentRoundManagers)
	{
		FSetManageReferencesNodeData& ManagerNodeData = FindOrAddNodeData(ManagerNode);
		check(CurrentManagerModifiedNodes.IsEmpty());
		check(CurrentManagerVisitQueue.IsEmpty());

		auto VisitNode = [ManagerNode, &Context, &CurrentManagerModifiedNodes,
			&CurrentManagerVisitQueue, &FindOrAddNodeData]
			(FDependsNode* SourceNode, FDependsNode* TargetNode, EDependencyCategory DependencyType,
				EDependencyProperty PackageDependencyProperties, EManageSearchColorBit PaintColors)
			{
				// Only recurse if we haven't already visited, and this node passes recursion test
				FSetManageReferencesNodeData& TargetNodeData = FindOrAddNodeData(TargetNode);
				PaintColors &= ~TargetNodeData.VisitedByCurrentManager;
				if (PaintColors == EManageSearchColorBit::None)
				{
					return;
				}

				EManageSearchColorBit ColorsManagedInEarlierRound = TargetNodeData.ManagedInEarlierRound & PaintColors;
				EManageSearchColorBit ColorsNotManagedInEarlierRound = (~TargetNodeData.ManagedInEarlierRound) & PaintColors;

				bool bDirectSet = SourceNode == ManagerNode;
				EAssetSetManagerFlags::Type EdgeFlagsOtherThanManagedInEarlierRound = (EAssetSetManagerFlags::Type)(
					(bDirectSet ? EAssetSetManagerFlags::IsDirectSet : 0)
					| (TargetNodeData.bDirectManagedInThisRound && !bDirectSet
						? EAssetSetManagerFlags::TargetHasDirectManager : 0));
				FShouldSetManagerContext ShouldSetContext
				{
					.Manager = ManagerNode->GetIdentifier(),
					.Source = SourceNode->GetIdentifier(),
					.Target = TargetNode->GetIdentifier(),
					.ManagerAssetFlags = ConstructAssetManagerAssetFlags(*ManagerNode),
					.SourceAssetFlags = ConstructAssetManagerAssetFlags(*SourceNode),
					.TargetAssetFlags = ConstructAssetManagerAssetFlags(*TargetNode),
					.EdgeFlags = EdgeFlagsOtherThanManagedInEarlierRound,
					.EdgeARCategory = DependencyType,
					.EdgeARProperties = PackageDependencyProperties
				};

				EManageSearchColorBit ColorsToSet = EManageSearchColorBit::None;
				EManageSearchColorBit ColorsToRecurse = EManageSearchColorBit::None;

				if (ColorsNotManagedInEarlierRound != EManageSearchColorBit::None)
				{
					EAssetSetManagerResult::Type Result = (*Context.ShouldSetManager)(ShouldSetContext);
					switch (Result)
					{
					case EAssetSetManagerResult::DoNotSet:
						break;
					case EAssetSetManagerResult::SetButDoNotRecurse:
						ColorsToSet |= ColorsNotManagedInEarlierRound;
						break;
					case EAssetSetManagerResult::SetAndRecurse:
						ColorsToSet |= ColorsNotManagedInEarlierRound;
						ColorsToRecurse |= ColorsNotManagedInEarlierRound;
						break;
					default:
						checkNoEntry();
						break;
					}
				}
				if (ColorsManagedInEarlierRound != EManageSearchColorBit::None)
				{
					ShouldSetContext.EdgeFlags = (EAssetSetManagerFlags::Type)
						(ShouldSetContext.EdgeFlags | EAssetSetManagerFlags::TargetHasExistingManager);
					EAssetSetManagerResult::Type Result = (*Context.ShouldSetManager)(ShouldSetContext);
					switch (Result)
					{
					case EAssetSetManagerResult::DoNotSet:
						break;
					case EAssetSetManagerResult::SetButDoNotRecurse:
						ColorsToSet |= ColorsManagedInEarlierRound;
						break;
					case EAssetSetManagerResult::SetAndRecurse:
						ColorsToSet |= ColorsManagedInEarlierRound;
						ColorsToRecurse |= ColorsManagedInEarlierRound;
						break;
					default:
						checkNoEntry();
						break;
					}
				}

				if (ColorsToSet != EManageSearchColorBit::None)
				{
					TargetNodeData.VisitedByCurrentManager |= ColorsToSet;
					TargetNodeData.bModifiedByCurrentManager = true;
					CurrentManagerModifiedNodes.Add(TargetNode);

					TargetNodeData.CurrentManagerProperties |=
						(bDirectSet ? EDependencyProperty::Direct : EDependencyProperty::None)
						| (EnumHasAnyFlags(ColorsToSet, EManageSearchColorBit::CookRule)
							? EDependencyProperty::CookRule : EDependencyProperty::None)
						// Painting with any color, or marking it as Direct, also marks it as painted with Chunking.
						// There is not a separate flag to store this in EDependencyProperty; the existence of the
						// dependency implies it is at least a Chunking dependency.
						;
					for (int32 PaintColor = 0; PaintColor < ManageSearchColorsNum; ++PaintColor)
					{
						if (EnumHasAnyFlags(ColorsToSet, (EManageSearchColorBit)(1 << ManageSearchColorsNum))
							&& !TargetNodeData.DebugInstigator[PaintColor])
						{
							TargetNodeData.DebugInstigator[PaintColor] = SourceNode;
						}
					}
				}

				if (ColorsToRecurse != EManageSearchColorBit::None)
				{
					CurrentManagerVisitQueue.Push({ TargetNode, ColorsToRecurse });
				}
			};

		// Visit the directly managed nodes and populate the starting vertices for the graph search of indirectly
		// managed nodes
		for (FDependsNode* ManagedNode : ManagerNodeData.DirectManagedAssetsThisRound)
		{
			VisitNode(ManagerNode, ManagedNode, EDependencyCategory::Manage,
				EDependencyProperty::Game | EDependencyProperty::Build, EManageSearchColorBit::All);
		}

		// Find and Visit all indirectly managed nodes, but only if we have a recurse type
		if (Context.RecurseType != EDependencyCategory::None)
		{
			while (CurrentManagerVisitQueue.Num())
			{
				// Pull off end of array to avoid the cost of shifts; order of visitation doesn't matter
				TPair<FDependsNode*, EManageSearchColorBit> VisitPair = CurrentManagerVisitQueue.Pop();
				FDependsNode* SourceNode = VisitPair.Key;
				EManageSearchColorBit VertexPaintColors = VisitPair.Value;

				SourceNode->IterateOverDependencies([&VisitNode, SourceNode, VertexPaintColors]
				(FDependsNode* TargetNode, EDependencyCategory DependencyCategory,
					EDependencyProperty DependencyProperties, bool bDuplicate)
					{
						// Skip dependencies that are EditorOnly and non-build.
						// Propagate only through UsedInGame or Build dependencies.
						if (EnumHasAnyFlags(DependencyProperties, EDependencyProperty::Game | EDependencyProperty::Build))
						{
							// ChunkManagement propagates through either UsedGame or Build dependencies, but CookRules
							// propagate only through Game dependencies.
							EManageSearchColorBit EdgePaintColors;
							EdgePaintColors = EnumHasAnyFlags(DependencyProperties, EDependencyProperty::Game) ?
									EManageSearchColorBit::All : EManageSearchColorBit::Chunking;
							EManageSearchColorBit TargetPaintColors = VertexPaintColors & EdgePaintColors;
							VisitNode(SourceNode, TargetNode, DependencyCategory, DependencyProperties, TargetPaintColors);
						}
					}, Context.RecurseType);
			}
		}

		ManagerNode->SetIsDependencyListSorted(UE::AssetRegistry::EDependencyCategory::Manage, false);
		ManagerNodeData.bModifiedByRound = true;
		CurrentRoundModifiedNodes.Add(ManagerNode);

		for (FDependsNode* ModifiedNode : CurrentManagerModifiedNodes)
		{
			FSetManageReferencesNodeData& ModifiedData = FindOrAddNodeData(ModifiedNode);
			if (!ModifiedData.bModifiedByCurrentManager)
			{
				// A duplicate of a NodeData we already handled earlier in the list
				continue;
			}

			ModifiedNode->SetIsReferencersSorted(false);
			ModifiedNode->AddReferencer(ManagerNode);
			ManagerNode->AddDependency(ModifiedNode, EDependencyCategory::Manage,
				ModifiedData.CurrentManagerProperties);

			ModifiedData.ManagedInThisRound |= ModifiedData.VisitedByCurrentManager;
			ModifiedData.bModifiedByRound = true;
			CurrentRoundModifiedNodes.Add(ModifiedNode);

			ModifiedData.bModifiedByCurrentManager = false;
			ModifiedData.VisitedByCurrentManager = EManageSearchColorBit::None;
			ModifiedData.CurrentManagerProperties = EDependencyProperty::None;
		}
		CurrentManagerModifiedNodes.Reset();
		CurrentManagerVisitQueue.Reset();
	}

#if WITH_EDITOR
	if (!GAssetRegistryManagementPathsPackageDebugName.IsEmpty())
	{
		FDependsNode* PackageDebugInfoNode = State.FindDependsNode(FName(*GAssetRegistryManagementPathsPackageDebugName));
		PrintAssetRegistryManagementPathsPackageDebugInfo(PackageDebugInfoNode, NodeData);
	}
#endif

	bool bShouldSortDependencies = ShouldSortDependencies();
	bool bShouldSortReferencers = ShouldSortReferencers();
	for (FDependsNode* ModifiedNode : CurrentRoundModifiedNodes)
	{
		FSetManageReferencesNodeData& ModifiedData = FindOrAddNodeData(ModifiedNode);
		if (!ModifiedData.bModifiedByRound)
		{
			// A duplicate of a NodeData we already handled earlier in the list
			continue;
		}

		// Restore all modified nodes to manage dependencies sorted and references sorted, so we can efficiently read
		// them in future operations
		ModifiedData.ManagedInEarlierRound |= ModifiedData.ManagedInThisRound;
		ModifiedNode->SetIsDependencyListSorted(EDependencyCategory::Manage, bShouldSortDependencies);
		ModifiedNode->SetIsReferencersSorted(bShouldSortReferencers);

		ModifiedData.bModifiedByRound = false;
		ModifiedData.ManagedInThisRound = EManageSearchColorBit::None;
		ModifiedData.bDirectManagedInThisRound = false;
		ModifiedData.DirectManagedAssetsThisRound.Empty();
		for (FDependsNode*& DebugInstigator : ModifiedData.DebugInstigator)
		{
			DebugInstigator = nullptr;
		}
	}
}

}

bool UAssetRegistryImpl::SetPrimaryAssetIdForObjectPath(const FSoftObjectPath& ObjectPath, FPrimaryAssetId PrimaryAssetId)
{
	UE::AssetRegistry::Impl::FEventContext EventContext;
	bool bResult;
	{
		LLM_SCOPE(ELLMTag::AssetRegistry);
		UE::AssetRegistry::FInterfaceWriteScopeLock InterfaceScopeLock(InterfaceLock);
		bResult = GuardedData.SetPrimaryAssetIdForObjectPath(EventContext, ObjectPath, PrimaryAssetId);
	}
	Broadcast(EventContext);
	return bResult;
}

namespace UE::AssetRegistry
{

bool FAssetRegistryImpl::SetPrimaryAssetIdForObjectPath(Impl::FEventContext& EventContext, const FSoftObjectPath& ObjectPath, FPrimaryAssetId PrimaryAssetId)
{
	FAssetData* AssetData = State.GetMutableAssetByObjectPath(ObjectPath);

	if (!AssetData)
	{
		return false;
	}

	FAssetDataTagMap TagsAndValues = AssetData->TagsAndValues.CopyMap();
	TagsAndValues.Add(FPrimaryAssetId::PrimaryAssetTypeTag, PrimaryAssetId.PrimaryAssetType.ToString());
	TagsAndValues.Add(FPrimaryAssetId::PrimaryAssetNameTag, PrimaryAssetId.PrimaryAssetName.ToString());

	FAssetData NewAssetData(*AssetData);
	NewAssetData.TagsAndValues = FAssetDataTagMapSharedView(MoveTemp(TagsAndValues));
	UpdateAssetData(EventContext, AssetData, MoveTemp(NewAssetData), false /* bKeepDeletedTags */);

	return true;
}

}

bool FAssetRegistryDependencyOptions::GetPackageQuery(UE::AssetRegistry::FDependencyQuery& Flags) const
{
	Flags = UE::AssetRegistry::FDependencyQuery();

	bool bHasFilter = false;
	if (bIncludeSoftPackageReferences || bIncludeHardPackageReferences)
	{
		if (!bIncludeSoftPackageReferences) Flags.Required |= UE::AssetRegistry::EDependencyProperty::Hard;
		if (!bIncludeHardPackageReferences) Flags.Excluded |= UE::AssetRegistry::EDependencyProperty::Hard;
		bHasFilter = true;
	}
	if (bIncludeGamePackageReferences || bIncludeEditorOnlyPackageReferences)
	{
		if (!bIncludeGamePackageReferences) Flags.Excluded |= UE::AssetRegistry::EDependencyProperty::Game;
		if (!bIncludeEditorOnlyPackageReferences) Flags.Required |= UE::AssetRegistry::EDependencyProperty::Game;
		bHasFilter = true;
	}
	return bHasFilter;
}

bool FAssetRegistryDependencyOptions::GetSearchableNameQuery(UE::AssetRegistry::FDependencyQuery& Flags) const
{
	Flags = UE::AssetRegistry::FDependencyQuery();
	return bIncludeSearchableNames;
}

bool FAssetRegistryDependencyOptions::GetManageQuery(UE::AssetRegistry::FDependencyQuery& Flags) const
{
	Flags = UE::AssetRegistry::FDependencyQuery();
	if (bIncludeSoftManagementReferences || bIncludeHardManagementReferences)
	{
		if (!bIncludeSoftManagementReferences) Flags.Required |= UE::AssetRegistry::EDependencyProperty::Direct;
		if (!bIncludeHardManagementReferences) Flags.Excluded |= UE::AssetRegistry::EDependencyProperty::Direct;
		return true;
	}
	return false;
}

void FAssetDependency::WriteCompactBinary(FCbWriter& Writer) const
{
	Writer.BeginArray();
	Writer << AssetId;
	static_assert(sizeof(uint8) >= sizeof(Category));
	Writer.AddInteger((uint8)Category);
	static_assert(sizeof(uint8) >= sizeof(Properties));
	Writer.AddInteger((uint8)Properties);
	Writer.EndArray();
}

bool LoadFromCompactBinary(FCbFieldView Field, FAssetDependency& Dependency)
{
	FCbArrayView ArrayField = Field.AsArrayView();
	if (ArrayField.Num() < 3)
	{
		Dependency = FAssetDependency();
		return false;
	}
	FCbFieldViewIterator Iter = ArrayField.CreateViewIterator();
	if (!LoadFromCompactBinary(Iter++, Dependency.AssetId))
	{
		Dependency = FAssetDependency();
		return false;
	}
	uint8 Value;
	if (LoadFromCompactBinary(Iter++, Value))
	{
		Dependency.Category = (UE::AssetRegistry::EDependencyCategory)Value;
	}
	else
	{
		Dependency = FAssetDependency();
		return false;
	}
	if (LoadFromCompactBinary(Iter++, Value))
	{
		Dependency.Properties = (UE::AssetRegistry::EDependencyProperty)Value;
	}
	else
	{
		Dependency = FAssetDependency();
		return false;
	}
	return true;
}

namespace UE::AssetRegistry
{

const FAssetRegistryState& FAssetRegistryImpl::GetState() const
{
	return State;
}

const FPathTree& FAssetRegistryImpl::GetCachedPathTree() const
{
	return CachedPathTree;
}

const TSet<FName>& FAssetRegistryImpl::GetCachedEmptyPackages() const
{
	return CachedEmptyPackages;
}

bool FAssetRegistryImpl::ShouldSkipAsset(FTopLevelAssetPath AssetClass, uint32 PackageFlags) const
{
#if WITH_ENGINE && WITH_EDITOR
	UE::TReadScopeLock ReadLock(SkipClassesLock);
	return Utils::ShouldSkipAsset(AssetClass, PackageFlags, SkipUncookedClasses, SkipCookedClasses);
#else
	return false;
#endif
}

bool FAssetRegistryImpl::ShouldSkipAsset(const UObject* InAsset) const
{
#if WITH_ENGINE && WITH_EDITOR
	UE::TReadScopeLock ReadLock(SkipClassesLock);
	return Utils::ShouldSkipAsset(InAsset, SkipUncookedClasses, SkipCookedClasses);
#else
	return false;
#endif
}

#if WITH_EDITOR
void FAssetRegistryImpl::PruneAndCoalescePackagesRequiringDependencyCalculation(TSet<FName>& BackgroundPackages,
	TSet<FName>& GameThreadPackages, Impl::FInterruptionContext& InOutInterruptionContext)
{
	RebuildAssetDependencyGathererMapIfNeeded();

	FReadScopeLock GathererClassScopeLock(RegisteredDependencyGathererClassesLock);

	// In many cases, this loop will be tight. If so, we don't want to spend a bunch of time checking whether we've
	// run out of processing time. So only check every N iterations.
	uint64 IterationCounter = 0;
	auto ProcessSet = [&InOutInterruptionContext, &IterationCounter, this](TSet<FName>& SourceSet,
		TSet<FName>* OptDestinationSet)->void
		{
			for (auto Iter = SourceSet.CreateIterator(); Iter; ++Iter)
			{
				bool HasAnyRegisteredDependencyGatherers = false;
				State.EnumerateAssetsByPackageName(*Iter,
					[this, &HasAnyRegisteredDependencyGatherers](const FAssetData* AssetData)
					{
						if (RegisteredDependencyGathererClasses.Contains(AssetData->AssetClassPath))
						{
							HasAnyRegisteredDependencyGatherers = true;
							return false; // stop iterating
						}
						return true; // Keep iterating
					});

				// If we need to process this asset and we have a destination set, move it there
				if ((OptDestinationSet != nullptr) && HasAnyRegisteredDependencyGatherers)
				{
					OptDestinationSet->Add(*Iter);
					Iter.RemoveCurrent();
				}
				else if (!HasAnyRegisteredDependencyGatherers)
				{
					// If we don't have to process this asset, remove it from whichever list it is in
					Iter.RemoveCurrent();
				}

				if ((++IterationCounter % 50) == 0)
				{
					if (InOutInterruptionContext.ShouldExitEarly())
					{
						return;
					}
				}
			}
		};

	ProcessSet(GameThreadPackages, nullptr);
	if (InOutInterruptionContext.ShouldExitEarly())
	{
		return;
	}
	ProcessSet(BackgroundPackages, &GameThreadPackages);
}
#endif 

namespace Impl
{

void FEventContext::Clear()
{
	bScanStartedEventBroadcast = false;
	bFileLoadedEventBroadcast = false;
	bKnownGathersCompleteEventBroadcast = false;
	bHasSentFileLoadedEventBroadcast = false;
	ProgressUpdateData.Reset();
	PathEvents.Empty();
	AssetEvents.Empty();
	RequiredLoads.Empty();
	BlockedFiles.Empty();
}

bool FEventContext::IsEmpty() const
{
	return !bScanStartedEventBroadcast &&
		!bFileLoadedEventBroadcast &&
		!bKnownGathersCompleteEventBroadcast &&
		!ProgressUpdateData.IsSet() &&
		PathEvents.Num() == 0 &&
		AssetEvents.Num() == 0 &&
		RequiredLoads.Num() == 0 &&
		BlockedFiles.Num() == 0;
}

void FEventContext::Append(FEventContext&& Other)
{
	if (&Other == this)
	{
		return;
	}
	bScanStartedEventBroadcast |= Other.bScanStartedEventBroadcast;
	Other.bScanStartedEventBroadcast = false;
	bFileLoadedEventBroadcast |= Other.bFileLoadedEventBroadcast;
	Other.bFileLoadedEventBroadcast = false;
	bKnownGathersCompleteEventBroadcast |= Other.bKnownGathersCompleteEventBroadcast;
	Other.bKnownGathersCompleteEventBroadcast = false;
	if (Other.ProgressUpdateData.IsSet())
	{
		ProgressUpdateData = MoveTemp(Other.ProgressUpdateData);
	}
	PathEvents.Append(MoveTemp(Other.PathEvents));
	AssetEvents.Append(MoveTemp(Other.AssetEvents));
	RequiredLoads.Append(MoveTemp(Other.RequiredLoads));
	BlockedFiles.Append(MoveTemp(Other.BlockedFiles));
}

}

}

void UAssetRegistryImpl::ReadLockEnumerateAllTagToAssetDatas(TFunctionRef<bool(FName TagName, IAssetRegistry::FEnumerateAssetDatasFunc EnumerateAssets)> Callback) const
{
	UE::AssetRegistry::FInterfaceReadScopeLock InterfaceScopeLock(InterfaceLock);
	GuardedData.GetState().EnumerateTagToAssetDatas(Callback);
}

bool UAssetRegistryImpl::CanBroadcastEvents() const
{
	return IsInGameThread() && !FUObjectThreadContext::Get().IsRoutingPostLoad;
}

void UAssetRegistryImpl::Broadcast(UE::AssetRegistry::Impl::FEventContext& EventContext, bool bAllowFileLoadedEvent)
{
	using namespace UE::AssetRegistry::Impl;
	if (!CanBroadcastEvents())
	{
		// By contract events (and PackageLoads) can only be sent on the game thread; some legacy systems depend on 
		// this and are not threadsafe. If we're not in the game thread, defer all events in the EventContext
		// instead of broadcasting them on this thread
		if (EventContext.IsEmpty())
		{
			return;
		}
		// Broadcast should not be called on DeferredEvents; DeferredEvents should be moved to a separate EventContext
		// and broadcast called on that separate EventContext outside of the lock.
		FScopeLock DeferredEventsLock(&DeferredEventsCriticalSection);
		check(&EventContext != &DeferredEvents);
		DeferredEvents.Append(MoveTemp(EventContext));
		RequestTick();
		EventContext.Clear();
		return;
	}

	if (EventContext.bScanStartedEventBroadcast)
	{
		// Raise event when the scan is started
		ScanStartedEvent.Broadcast();
		EventContext.bScanStartedEventBroadcast = false;
	}

	if (EventContext.PathEvents.Num())
	{
		if (PathsAddedEvent.IsBound() || PathsRemovedEvent.IsBound()
			|| PathAddedEvent.IsBound() || PathRemovedEvent.IsBound())
		{
			// Batch add/remove events 
			TArray<FStringView> Params;
			// Ensure loop batch condition is always false first iteration
			bool bCurrentBatchIsAdd = EventContext.PathEvents[0].Get<1>() == FEventContext::EEvent::Added;
			for (const TPair<FString, FEventContext::EEvent>& PathEvent : EventContext.PathEvents)
			{
				const FString& Path = PathEvent.Get<0>();
				bool bEventIsAdd = PathEvent.Get<1>() == FEventContext::EEvent::Added;
				if (bEventIsAdd != bCurrentBatchIsAdd)
				{
					(bCurrentBatchIsAdd ? PathsAddedEvent : PathsRemovedEvent).Broadcast(MakeArrayView(Params));
					Params.Reset();
					bCurrentBatchIsAdd = bEventIsAdd;
				}
				Params.Add(FStringView(Path));
			}
			if (Params.Num() != 0)
			{
				(bCurrentBatchIsAdd ? PathsAddedEvent : PathsRemovedEvent).Broadcast(MakeArrayView(Params));
			}

			// Legacy single events 
			if (PathAddedEvent.IsBound() || PathRemovedEvent.IsBound())
			{
				for (const TPair<FString, FEventContext::EEvent>& PathEvent : EventContext.PathEvents)
				{
					const FString& Path = PathEvent.Get<0>();
					switch (PathEvent.Get<1>())
					{
					case FEventContext::EEvent::Added:
						PathAddedEvent.Broadcast(Path);
						break;
					case FEventContext::EEvent::Removed:
						PathRemovedEvent.Broadcast(Path);
						break;
					}
				}
			}
		}
		EventContext.PathEvents.Empty();
	}

	if (EventContext.AssetEvents.Num())
	{
		// Batch events so that if adds/updates are interspersed with removes, relative ordering of the add/remove is maintained 
		constexpr uint32 EventTypeCount = static_cast<uint32>(FEventContext::EEvent::MAX);
		static_assert(EventTypeCount == 4, "Loop needs to be rewritten to correctly order new event types");
		bool bHasListeners = AssetAddedEvent.IsBound() || AssetRemovedEvent.IsBound()
			|| AssetUpdatedEvent.IsBound() || AssetUpdatedOnDiskEvent.IsBound();
		if (!bHasListeners)
		{
			for (int32 i = 0; i < EventTypeCount; ++i)
			{
				if (BatchedAssetEvents[i].IsBound())
				{
					bHasListeners = true;
					break;
				}
			}
		}

		if (bHasListeners)
		{
			TArray<FAssetData> EventBatches[EventTypeCount];

			// Pre-allocate memory for event batches
			uint32 EventBatchCounters[EventTypeCount] = { 0 };
			for (const TPair<FAssetData, FEventContext::EEvent>& AssetEvent : EventContext.AssetEvents)
			{
				FEventContext::EEvent Event = AssetEvent.Get<1>();
				EventBatchCounters[static_cast<int32>(Event)]++;
			}

			for (int32 i = 0; i < UE_ARRAY_COUNT(EventBatches); ++i)
			{
				EventBatches[i].Reserve(EventBatchCounters[i]);
			}

			// Dispatch events
			FEventContext::EEvent LastEvent = EventContext.AssetEvents[0].Get<1>();
			auto FlushBatchedEvents = [&EventBatches, &Events = BatchedAssetEvents]()
				{
					for (int32 i = 0; i < UE_ARRAY_COUNT(EventBatches); ++i)
					{
						if (EventBatches[i].Num())
						{
							Events[i].Broadcast(EventBatches[i]);
							EventBatches[i].Reset();
						}
					}
				};

			for (const TPair<FAssetData, FEventContext::EEvent>& AssetEvent : EventContext.AssetEvents)
			{
				const FAssetData& AssetData = AssetEvent.Get<0>();
				FEventContext::EEvent Event = AssetEvent.Get<1>();

				// Flush events when switching between removed and non-removed events 
				if ((Event == FEventContext::EEvent::Removed) != (LastEvent == FEventContext::EEvent::Removed))
				{
					FlushBatchedEvents();
				}
				EventBatches[static_cast<int32>(Event)].Add(AssetData);
				LastEvent = Event;
			}
			// Flush last batch of events 
			FlushBatchedEvents();

			// Single events
			for (const TPair<FAssetData, FEventContext::EEvent>& AssetEvent : EventContext.AssetEvents)
			{
				const FAssetData& AssetData = AssetEvent.Get<0>();
				switch (AssetEvent.Get<1>())
				{
				case FEventContext::EEvent::Added:
					AssetAddedEvent.Broadcast(AssetData);
					break;
				case FEventContext::EEvent::Removed:
					AssetRemovedEvent.Broadcast(AssetData);
					break;
				case FEventContext::EEvent::Updated:
					AssetUpdatedEvent.Broadcast(AssetData);
					break;
				case FEventContext::EEvent::UpdatedOnDisk:
					AssetUpdatedOnDiskEvent.Broadcast(AssetData);
					break;
				default:
					checkNoEntry();
					break;
				}
			}
		}
		EventContext.AssetEvents.Empty();
	}
	if (EventContext.VerseEvents.Num())
	{
		if (VerseAddedEvent.IsBound() || VerseRemovedEvent.IsBound())
		{
			for (const TPair<FName, FEventContext::EEvent>& VerseEvent : EventContext.VerseEvents)
			{
				const FName& VerseFilepath = VerseEvent.Get<0>();
				switch (VerseEvent.Get<1>())
				{
				case FEventContext::EEvent::Added:
					VerseAddedEvent.Broadcast(VerseFilepath);
					break;
				case FEventContext::EEvent::Removed:
					VerseRemovedEvent.Broadcast(VerseFilepath);
					break;
				// (jcotton) We are not yet broadcasting Verse updating events as the only use case for VerseEvent broadcasts currently is to trigger a Verse-build
				// and triggering a build on every change would be far too expensive.
				case FEventContext::EEvent::Updated:
					[[fallthrough]];
				case FEventContext::EEvent::UpdatedOnDisk:
					[[fallthrough]];
				default:
					break;
				}
			}
		}
		EventContext.VerseEvents.Empty();
	}
	if (EventContext.RequiredLoads.Num())
	{
		TArray<int32> RequestIds;
		for (const FString& RequiredLoad : EventContext.RequiredLoads)
		{
			RequestIds.Add(LoadPackageAsync(*RequiredLoad));
		}
		if (!RequestIds.IsEmpty())
		{
			FlushAsyncLoading(RequestIds);
		}
		EventContext.RequiredLoads.Empty();
	}
	if (EventContext.BlockedFiles.Num())
	{
		FilesBlockedEvent.Broadcast(EventContext.BlockedFiles);
		EventContext.BlockedFiles.Empty();
	}

	if (EventContext.ProgressUpdateData.IsSet())
	{
		FileLoadProgressUpdatedEvent.Broadcast(*EventContext.ProgressUpdateData);
		EventContext.ProgressUpdateData.Reset();
	}

	// FileLoadedEvent needs to come after all of the AssetEvents. Some systems do more expensive work for
	// AssetEvents after receiving FileLoadedEvent, because they batched up that work for all assets in the initial load in
	// their FileLoadedEvent handler. The AssetEvents precede the FileLoadedEvent in the broadcast that is sent from
	// TickGatherer, so it is correct to make them precede it in the order in which we broadcast the events.

	if (EventContext.bFileLoadedEventBroadcast || EventContext.bKnownGathersCompleteEventBroadcast)
	{
		bool bLocalFileLoadedEventBroadcast = EventContext.bFileLoadedEventBroadcast;
		bool bLocalKnownGathersCompleteEventBroadcast = EventContext.bKnownGathersCompleteEventBroadcast;

		if (!bAllowFileLoadedEvent)
		{
			// Do not send the file loaded event yet; pass the flag on instead
			FScopeLock DeferredEventsLock(&DeferredEventsCriticalSection);
			// Broadcast should not be called on DeferredEvents; DeferredEvents should be moved to a separate EventContext
			// and broadcast called on that separate EventContext outside of the lock.
			check(&EventContext != &DeferredEvents);
			DeferredEvents.Append(MoveTemp(EventContext));
			RequestTick();
			EventContext.Clear();
			check(!EventContext.bFileLoadedEventBroadcast && !EventContext.bKnownGathersCompleteEventBroadcast); // was cleared by Append and by Clear
			check(!bLocalFileLoadedEventBroadcast || DeferredEvents.bFileLoadedEventBroadcast); // was set by Append
			check(!bLocalKnownGathersCompleteEventBroadcast || DeferredEvents.bKnownGathersCompleteEventBroadcast); // was set by Append
			return;
		}

		FEventContext CopiedDeferredEvents;
		{
			FScopeLock DeferredEventsLock(&DeferredEventsCriticalSection);
			check(&EventContext != &DeferredEvents);
			CopiedDeferredEvents = MoveTemp(DeferredEvents);
			DeferredEvents.Clear();
		}
		if (!CopiedDeferredEvents.IsEmpty())
		{
			// Recursively send all of the deferred events, except for the completion events
			// (FileLoaded, KnownGathersComplete). The completion events should not exist on DeferredEvents at this
			// point, but it's not a problem if they do; merge them into our Local variables.
			bLocalFileLoadedEventBroadcast |= CopiedDeferredEvents.bFileLoadedEventBroadcast;
			CopiedDeferredEvents.bFileLoadedEventBroadcast = false;

			bLocalKnownGathersCompleteEventBroadcast |= CopiedDeferredEvents.bKnownGathersCompleteEventBroadcast;
			CopiedDeferredEvents.bKnownGathersCompleteEventBroadcast = false;

			Broadcast(CopiedDeferredEvents, false /* bAllowFileLoadedEvent */);
		}
		// Now it is safe to broadcast the completion events. If other deferred events come in on another thread
		// after we copied from DeferredEvents, that is okay; the contract for completion events is that they are
		// guaranteed to be sent after any non-completion events sent before completion was triggered,
		// but they can be before or after non-completion events that occurred after completion was triggered.
		if (bLocalFileLoadedEventBroadcast)
		{
			double BroadcastTimeStart = FPlatformTime::Seconds();
			UE_LOGF(LogAssetRegistry, Display, "Starting OnFilesLoaded.Broadcast");
			FileLoadedEvent.Broadcast();
			ScanEndedEvent.Broadcast();
			UE_LOGF(LogAssetRegistry, Display, "Completed OnFilesLoaded.Broadcast: %.3fs",
				FPlatformTime::Seconds() - BroadcastTimeStart);
			EventContext.bHasSentFileLoadedEventBroadcast = true;
			EventContext.bFileLoadedEventBroadcast = false;
		}
		if (bLocalKnownGathersCompleteEventBroadcast)
		{
			KnownGathersCompleteEvent.Broadcast();
			EventContext.bKnownGathersCompleteEventBroadcast = false;
		}
	}
}


UAssetRegistryImpl::FFilesBlockedEvent& UAssetRegistryImpl::OnFilesBlocked()
{
	return FilesBlockedEvent;
}

UAssetRegistryImpl::FPathsEvent& UAssetRegistryImpl::OnPathsAdded()
{
	return PathsAddedEvent;
}

UAssetRegistryImpl::FPathsEvent& UAssetRegistryImpl::OnPathsRemoved()
{
	return PathsRemovedEvent;
}

UAssetRegistryImpl::FPathAddedEvent& UAssetRegistryImpl::OnPathAdded()
{
	return PathAddedEvent;
}

UAssetRegistryImpl::FPathRemovedEvent& UAssetRegistryImpl::OnPathRemoved()
{
	return PathRemovedEvent;
}

IAssetRegistry::FAssetCollisionEvent& UAssetRegistryImpl::OnAssetCollision_Private()
{
	return GuardedData.OnAssetCollision_Private();
}

UAssetRegistryImpl::FAssetAddedEvent& UAssetRegistryImpl::OnAssetAdded()
{
	return AssetAddedEvent;
}

UAssetRegistryImpl::FAssetRemovedEvent& UAssetRegistryImpl::OnAssetRemoved()
{
	return AssetRemovedEvent;
}

UAssetRegistryImpl::FAssetRenamedEvent& UAssetRegistryImpl::OnAssetRenamed()
{
	return AssetRenamedEvent;
}

UAssetRegistryImpl::FAssetUpdatedEvent& UAssetRegistryImpl::OnAssetUpdated()
{
	return AssetUpdatedEvent;
}

UAssetRegistryImpl::FAssetUpdatedEvent& UAssetRegistryImpl::OnAssetUpdatedOnDisk()
{
	return AssetUpdatedOnDiskEvent;
}

UAssetRegistryImpl::FAssetsEvent& UAssetRegistryImpl::OnAssetsAdded()
{
	return BatchedAssetEvents[static_cast<SIZE_T>(UE::AssetRegistry::Impl::FEventContext::EEvent::Added)];
}

UAssetRegistryImpl::FAssetsEvent& UAssetRegistryImpl::OnAssetsUpdated()
{
	return BatchedAssetEvents[static_cast<SIZE_T>(UE::AssetRegistry::Impl::FEventContext::EEvent::Updated)];
}

UAssetRegistryImpl::FAssetsEvent& UAssetRegistryImpl::OnAssetsUpdatedOnDisk()
{
	return BatchedAssetEvents[static_cast<SIZE_T>(UE::AssetRegistry::Impl::FEventContext::EEvent::UpdatedOnDisk)];
}

UAssetRegistryImpl::FAssetsEvent& UAssetRegistryImpl::OnAssetsRemoved()
{
	return BatchedAssetEvents[static_cast<SIZE_T>(UE::AssetRegistry::Impl::FEventContext::EEvent::Removed)];
}

UAssetRegistryImpl::FInMemoryAssetCreatedEvent& UAssetRegistryImpl::OnInMemoryAssetCreated()
{
	return InMemoryAssetCreatedEvent;
}

UAssetRegistryImpl::FInMemoryAssetDeletedEvent& UAssetRegistryImpl::OnInMemoryAssetDeleted()
{
	return InMemoryAssetDeletedEvent;
}

UAssetRegistryImpl::FVerseAddedEvent& UAssetRegistryImpl::OnVerseAdded()
{
	return VerseAddedEvent;
}

UAssetRegistryImpl::FVerseRemovedEvent& UAssetRegistryImpl::OnVerseRemoved()
{
	return VerseRemovedEvent;
}

UAssetRegistryImpl::FFilesLoadedEvent& UAssetRegistryImpl::OnFilesLoaded()
{
	return FileLoadedEvent;
}

UAssetRegistryImpl::FFileLoadProgressUpdatedEvent& UAssetRegistryImpl::OnFileLoadProgressUpdated()
{
	return FileLoadProgressUpdatedEvent;
}

UAssetRegistryImpl::FScanStartedEvent& UAssetRegistryImpl::OnScanStarted()
{
	return ScanStartedEvent;
}

UAssetRegistryImpl::FScanEndedEvent& UAssetRegistryImpl::OnScanEnded()
{
	return ScanEndedEvent;
}

UAssetRegistryImpl::FKnownGathersCompleteEvent& UAssetRegistryImpl::OnKnownGathersComplete()
{
	return KnownGathersCompleteEvent;
}

namespace UE::AssetRegistry
{
const FAssetData* GetMostImportantAsset(TConstArrayView<const FAssetData*> PackageAssetDatas, EGetMostImportantAssetFlags InFlags)
{
	if (PackageAssetDatas.Num() == 1) // common case
	{
		return PackageAssetDatas[0];
	}

	// Find a candidate asset.
	// If there's a "UAsset", then we use that as the asset.
	// If not, then we look for a "TopLevelAsset", i.e. one that shows up in the content browser.
	int32 TopLevelAssetCount = 0;

	// If we have multiple TLAs, then we pick the "least" TLA.
	// If we have NO TLAs, then we pick the "least" asset,
	// both determined by class then name:
	auto AssetDataLessThan = [](const FAssetData* LHS, const FAssetData* RHS)
	{
		int32 ClassCompare = LHS->AssetClassPath.Compare(RHS->AssetClassPath);
		if (ClassCompare == 0)
		{
			return LHS->AssetName.LexicalLess(RHS->AssetName);
		}
		return ClassCompare < 0;
	};

	const FAssetData* LeastTopLevelAsset = nullptr;
	const FAssetData* LeastAsset = nullptr;
	for (const FAssetData* Asset : PackageAssetDatas)
	{
		if (Asset->AssetName.IsNone())
		{
			continue;
		}
		if (Asset->IsUAsset())
		{
			return Asset;
		}
		// This is after IsUAsset because Blueprints can be the UAsset but also be considered skipable.
		if (!EnumHasAnyFlags(InFlags, EGetMostImportantAssetFlags::IgnoreSkipClasses))
		{
			if (FFiltering::ShouldSkipAsset(Asset->AssetClassPath, Asset->PackageFlags))
			{
				continue;
			}
		}

		if (Asset->IsTopLevelAsset())
		{
			TopLevelAssetCount++;
			if (LeastTopLevelAsset == nullptr ||
				AssetDataLessThan(Asset, LeastTopLevelAsset))
			{
				LeastTopLevelAsset = Asset;
			}
		}
		if (LeastAsset == nullptr ||
			AssetDataLessThan(Asset, LeastAsset))
		{
			LeastAsset = Asset;
		}
	}

	if (EnumHasAnyFlags(InFlags, EGetMostImportantAssetFlags::RequireOneTopLevelAsset))
	{
		if (TopLevelAssetCount == 1)
		{
			return LeastTopLevelAsset;
		}
		return nullptr;
	}

	if (TopLevelAssetCount)
	{
		return LeastTopLevelAsset;
	}
	return LeastAsset;
}


void GetAssetForPackages(TConstArrayView<FName> PackageNames, TMap<FName, FAssetData>& OutPackageToAssetData)
{
	FARFilter Filter;
	for (FName PackageName : PackageNames)
	{
		Filter.PackageNames.Add(PackageName);
	}
	
	TArray<FAssetData> AssetDataList;
	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	if (!AssetRegistry)
	{
		return;
	}
	AssetRegistry->GetAssets(Filter, AssetDataList);

	if (AssetDataList.Num() == 0)
	{
		return;
	}

	Algo::SortBy(AssetDataList, &FAssetData::PackageName, FNameFastLess());

	TArray<const FAssetData*, TInlineAllocator<1>> PackageAssetDatas;
	FName CurrentPackageName = AssetDataList[0].PackageName;
	for (const FAssetData& AssetData : AssetDataList)
	{
		if (CurrentPackageName != AssetData.PackageName)
		{
			OutPackageToAssetData.FindOrAdd(CurrentPackageName) = *GetMostImportantAsset(PackageAssetDatas);
			PackageAssetDatas.Reset();
			CurrentPackageName = AssetData.PackageName;
		}

		PackageAssetDatas.Push(&AssetData);
	}

	OutPackageToAssetData.FindOrAdd(CurrentPackageName) = *GetMostImportantAsset(PackageAssetDatas);

}

bool ShouldSearchAllAssetsAtStart()
{
	// Search at start for configurations that need the entire assetregistry and that do not load it from serialized:
	// Need it: Editor IDE, CookCommandlet, other Allowlist Commandlets
	// Possibly need it: editor running as -game or -server
	// Do not need it: Commandlets not on the Allowlist
	// Load it from serialized: Non-editor-executable
	// 
	// This behavior can be overridden with commandline option.
	// 
	// For the editor-executable configurations that do not search at start, the search will be triggered when
	// SearchAllAssets or ScanPathsSynchronous is called.

	bool bSearchAllAssetsAtStart = false;
	if (GIsEditor)
	{
		if (!IsRunningCommandlet() || IsRunningCookCommandlet())
		{
			bSearchAllAssetsAtStart = true;
		}
		else
		{
			TArray<FString> CommandletsUsingAR;
			GConfig->GetArray(TEXT("AssetRegistry"), TEXT("CommandletsUsingAR"), CommandletsUsingAR, GEngineIni);
			FString CommandlineCommandlet;
			FString CommandletToken("commandlet");
			if (!CommandletsUsingAR.IsEmpty() &&
				FParse::Value(FCommandLine::Get(), TEXT("-run="), CommandlineCommandlet))
			{
				if (CommandlineCommandlet.EndsWith(CommandletToken))
				{
					CommandlineCommandlet.LeftChopInline(CommandletToken.Len(), EAllowShrinking::No);
				}
				for (FString& CommandletUsingAR : CommandletsUsingAR)
				{
					if (CommandletUsingAR.EndsWith(CommandletToken))
					{
						CommandletUsingAR.LeftChopInline(CommandletToken.Len(), EAllowShrinking::No);
					}
					if (CommandletUsingAR == CommandlineCommandlet)
					{
						bSearchAllAssetsAtStart = true;
						break;
					}
				}
			}
		}
	}
#if WITH_EDITOR
	else
	{
		using namespace UE::AssetDataGather::Private;
		bSearchAllAssetsAtStart = GGatherSettings.GetEditorGameScanMode() != EEditorGameScanMode::None;
	}
#endif
#if WITH_EDITOR || !UE_BUILD_SHIPPING
	bool bCommandlineAllAssetsAtStart;
	if (FParse::Bool(FCommandLine::Get(), TEXT("AssetGatherAll="), bCommandlineAllAssetsAtStart))
	{
		bSearchAllAssetsAtStart = bCommandlineAllAssetsAtStart;
	}
#endif // WITH_EDITOR || !UE_BUILD_SHIPPING
	return bSearchAllAssetsAtStart;
}

namespace Impl
{

bool FInterruptionContext::ShouldExitEarly()
{
	if (EarlyExitCallback && EarlyExitCallback())
	{
		OutInterrupted = true;
	}
	else if (TickStartTime > 0 && ((FPlatformTime::Seconds() - TickStartTime) > MaxRunningTime))
	{
		OutInterrupted = true;
	}
	return OutInterrupted;
}

}

FString CreateStandardFilename(FStringView InPath)
{
	return CreateStandardFilename(FString(InPath));
}

FString CreateStandardFilename(const FString& InPath)
{
	FString Result = FPaths::CreateStandardFilename(InPath);

	// Follow most of the behavior of FPaths::CreateStandardFilename, but tweak it to also
	// replace '\' -> '/' even when the path is not convertible to an engine-relative path.
	// And add a special case tweak of that tweak, so that a windows-style network share \\
	// is left unchanged and remains understandable by the OS.
	bool bKeepLeadingSlashes = InPath.StartsWith(TEXTVIEW("\\\\"));
	Result.ReplaceInline(TEXT("\\"), TEXT("/"), ESearchCase::CaseSensitive);
	if (bKeepLeadingSlashes)
	{
		Result[0] = '\\';
		Result[1] = '\\';
	}
	return Result;
}

FString CreateStandardLongPackageNameDirectory(FStringView LongPackageName)
{
	FString Result(LongPackageName);
	if (!Result.EndsWith(TEXT("/"), ESearchCase::CaseSensitive))
	{
		Result = Result + TEXT("/");
	}
	return Result;
}

FName CreateStandardLongPackageFNameNoTrailingSlash(FStringView LongPackageName)
{
	FStringView ResultNoTrailingSlash = LongPackageName;
	if (ResultNoTrailingSlash.EndsWith(TEXT("/"), ESearchCase::CaseSensitive))
	{
		// We don't want a trailing slash here as it could interfere with RemoveAssetPath
		ResultNoTrailingSlash.LeftChopInline(1);
	}
	return FName(ResultNoTrailingSlash);
}


} // namespace AssetRegistry
