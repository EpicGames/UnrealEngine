// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARLoader.h"
#include "AssetRegistryImpl.h"
#include "Containers/Ticker.h"
#include "Misc/PackageName.h"
#include "UObject/Object.h"

#include "AssetRegistry.generated.h"

class FAssetDataGatherer;
namespace UE::AssetRegistry::Impl { class FActiveMountsLoader; }
namespace UE::AssetRegistry::Premade { struct FAsyncConsumer; }

/**
 * The AssetRegistry singleton gathers information about .uasset files in the background so things
 * like the content browser don't have to work with the filesystem
 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS

UCLASS(transient)
class UAssetRegistryImpl : public UObject, public IAssetRegistry
{
	GENERATED_BODY()
public:
	UAssetRegistryImpl(const FObjectInitializer& ObjectInitializer);
	UAssetRegistryImpl(FVTableHelper& Helper);
	virtual ~UAssetRegistryImpl();
	virtual void FinishDestroy() override;

	/** Gets the asset registry singleton for asset registry module use */
	static UAssetRegistryImpl& Get();

	// IAssetRegistry implementation
	virtual bool HasAssets(const FName PackagePath, const bool bRecursive = false) const override;
	virtual bool GetAssetsByPackageName(FName PackageName, TArray<FAssetData>& OutAssetData, bool bIncludeOnlyOnDiskAssets = false, bool bSkipARFilteredAssets=true) const override;
	virtual bool GetAssetsByPath(FName PackagePath, TArray<FAssetData>& OutAssetData, bool bRecursive = false, bool bIncludeOnlyOnDiskAssets = false) const override;
	virtual bool GetAssetsByPaths(TArray<FName> PackagePath, TArray<FAssetData>& OutAssetData, bool bRecursive = false, bool bIncludeOnlyOnDiskAssets = false) const override;
	virtual bool GetAssetsByClass(FTopLevelAssetPath ClassPathName, TArray<FAssetData>& OutAssetData, bool bSearchSubClasses = false) const override;
	virtual bool GetAssetsByTags(const TArray<FName>& AssetTags, TArray<FAssetData>& OutAssetData) const override;
	virtual bool GetAssetsByTagValues(const TMultiMap<FName, FString>& AssetTagsAndValues, TArray<FAssetData>& OutAssetData) const override;
	virtual bool GetAssets(const FARFilter& Filter, TArray<FAssetData>& OutAssetData, bool bSkipARFilteredAssets = true) const override;
	virtual bool GetAssets(const FARCompiledFilter& Filter, TArray<FAssetData>& OutAssetData, bool bSkipARFilteredAssets = true) const override;
	virtual bool GetInMemoryAssets(const FARFilter& Filter, TArray<FAssetData>& OutAssetData, bool bSkipARFilteredAssets=true) const override;
	virtual bool GetInMemoryAssets(const FARCompiledFilter& Filter, TArray<FAssetData>& OutAssetData, bool bSkipARFilteredAssets=true) const override;
	virtual bool EnumerateAssets(const FARFilter& Filter, TFunctionRef<bool(const FAssetData&)> Callback,
		UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags = UE::AssetRegistry::EEnumerateAssetsFlags::None) const override;
	virtual bool EnumerateAssets(const FARCompiledFilter& Filter, TFunctionRef<bool(const FAssetData&)> Callback,
		UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags = UE::AssetRegistry::EEnumerateAssetsFlags::None) const override;
	UE_DEPRECATED(5.1, "Asset path FNames have been deprecated, use FSoftObjectPath instead.")
	virtual FAssetData GetAssetByObjectPath( const FName ObjectPath, bool bIncludeOnlyOnDiskAssets = false ) const override;
	virtual FAssetData GetAssetByObjectPath(const FSoftObjectPath& ObjectPath, bool bIncludeOnlyOnDiskAssets = false, bool bSkipARFilteredAssets = true) const override;
	virtual UE::AssetRegistry::EExists TryGetAssetByObjectPath(const FSoftObjectPath& ObjectPath, FAssetData& OutAssetData) const override;
	virtual UE::AssetRegistry::EExists TryGetAssetPackageData(const FName PackageName, FAssetPackageData& OutAssetPackageData, bool bFailIfLockHeld = false) const override;
	virtual UE::AssetRegistry::EExists TryGetAssetPackageData(const FName PackageName, FAssetPackageData& OutAssetPackageData, FName& OutCorrectCasePackageName, bool bFailIfLockHeld = false) const override;
	virtual bool GetAllAssets(TArray<FAssetData>& OutAssetData, bool bIncludeOnlyOnDiskAssets = false) const override;
	virtual bool EnumerateAllAssets(TFunctionRef<bool(const FAssetData&)> Callback,
		UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags = UE::AssetRegistry::EEnumerateAssetsFlags::None) const override;
	virtual void GetPackagesByName(FStringView PackageName, TArray<FName>& OutPackageNames) const override;
	virtual FName GetFirstPackageByName(FStringView PackageName) const override;
	virtual bool GetDependencies(const FAssetIdentifier& AssetIdentifier, TArray<FAssetIdentifier>& OutDependencies, UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All, const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const override;
	virtual bool GetDependencies(const FAssetIdentifier& AssetIdentifier, TArray<FAssetDependency>& OutDependencies, UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All, const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const override;
	virtual bool GetDependencies(FName PackageName, TArray<FName>& OutDependencies, UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::Package, const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const override;
	virtual bool ContainsDependency(FName PackageName, FName QueryDependencyName, UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::Package, const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const override;
	virtual bool GetReferencers(const FAssetIdentifier& AssetIdentifier, TArray<FAssetIdentifier>& OutReferencers, UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All, const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const override;
	virtual bool GetReferencers(const FAssetIdentifier& AssetIdentifier, TArray<FAssetDependency>& OutReferencers, UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All, const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const override;
	virtual bool GetReferencers(FName PackageName, TArray<FName>& OutReferencers, UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::Package, const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const override;
	virtual TOptional<FAssetPackageData> GetAssetPackageDataCopy(FName PackageName) const override;
	virtual TArray<TOptional<FAssetPackageData>> GetAssetPackageDatasCopy(TArrayView<FName> PackageNames) const override;
	virtual void EnumerateAllPackages(TFunctionRef<void(FName PackageName, const FAssetPackageData& PackageData)> Callback, UE::AssetRegistry::EEnumeratePackagesFlags InEnumerateFlags) const override;
	virtual bool DoesPackageExistOnDisk(FName PackageName, FString* OutCorrectCasePackageName = nullptr, FString* OutExtension = nullptr) const override;
	virtual FSoftObjectPath GetRedirectedObjectPath(const FSoftObjectPath& ObjectPath) override;
	virtual bool GetAncestorClassNames(FTopLevelAssetPath ClassName, TArray<FTopLevelAssetPath>& OutAncestorClassNames) const override;
	virtual void GetDerivedClassNames(const TArray<FTopLevelAssetPath>& ClassNames, const TSet<FTopLevelAssetPath>& ExcludedClassNames, TSet<FTopLevelAssetPath>& OutDerivedClassNames) const override;	
	virtual void GetAllCachedPaths(TArray<FString>& OutPathList) const override;
	virtual void EnumerateAllCachedPaths(TFunctionRef<bool(FString)> Callback) const override;
	virtual void EnumerateAllCachedPaths(TFunctionRef<bool(FName)> Callback) const override;
	virtual void GetSubPaths(const FString& InBasePath, TArray<FString>& OutPathList, bool bInRecurse) const override;
	virtual void GetSubPaths(const FName& InBasePath, TArray<FName>& OutPathList, bool bInRecurse) const override;
	virtual void EnumerateSubPaths(const FString& InBasePath, TFunctionRef<bool(FString)> Callback, bool bInRecurse) const override;
	virtual void EnumerateSubPaths(const FName InBasePath, TFunctionRef<bool(FName)> Callback, bool bInRecurse) const override;
	virtual void RunAssetsThroughFilter (TArray<FAssetData>& AssetDataList, const FARFilter& Filter) const override;
	virtual void UseFilterToExcludeAssets(TArray<FAssetData>& AssetDataList, const FARFilter& Filter) const override;
	virtual void UseFilterToExcludeAssets(TArray<FAssetData>& AssetDataList, const FARCompiledFilter& CompiledFilter) const override;
	virtual bool IsAssetIncludedByFilter(const FAssetData& AssetData, const FARCompiledFilter& Filter) const override;
	virtual bool IsAssetExcludedByFilter(const FAssetData& AssetData, const FARCompiledFilter& Filter) const override;
	virtual void CompileFilter(const FARFilter& InFilter, FARCompiledFilter& OutCompiledFilter) const override;
	virtual void SetTemporaryCachingMode(bool bEnable) override;
	virtual void SetTemporaryCachingModeInvalidated() override;
	virtual bool GetTemporaryCachingMode() const override;
	virtual EAssetAvailability::Type GetAssetAvailability(const FAssetData& AssetData) const override;	
	virtual float GetAssetAvailabilityProgress(const FAssetData& AssetData, EAssetAvailabilityProgressReportingType::Type ReportType) const override;
	virtual bool GetAssetAvailabilityProgressTypeSupported(EAssetAvailabilityProgressReportingType::Type ReportType) const override;
	virtual void PrioritizeAssetInstall(const FAssetData& AssetData) const override;
	virtual bool HasVerseFiles(FName PackagePath, bool bRecursive = false) const override;
	virtual bool GetVerseFilesByPath(FName PackagePath, TArray<FName>& OutFilePaths, bool bRecursive = false) const override;
	virtual bool AddPath(const FString& PathToAdd) override;
	virtual bool RemovePath(const FString& PathToRemove) override;
	virtual bool PathExists(const FString& PathToTest) const override;
	virtual bool PathExists(const FName PathToTest) const override;
	virtual void SearchAllAssets(bool bSynchronousSearch) override;
	virtual bool IsSearchAllAssets() const override;
	virtual bool IsSearchAsync() const override;
	virtual void WaitForCompletion() override;
	virtual void WaitForPremadeAssetRegistry() override;
	virtual void ClearGathererCache() override;
	virtual void WaitForPackage(const FString& PackageName) override;
	virtual void ScanSynchronous(const TArray<FString>& InPaths, const TArray<FString>& InFilePaths, UE::AssetRegistry::EScanFlags InScanFlags = UE::AssetRegistry::EScanFlags::None) override;
	virtual void ScanPathsSynchronous(const TArray<FString>& InPaths, bool bForceRescan = false, bool bIgnoreDenyListScanFilters = false) override;
	virtual void ScanFilesSynchronous(const TArray<FString>& InFilePaths, bool bForceRescan = false) override;
	virtual void PrioritizeSearchPath(const FString& PathToPrioritize) override;
	virtual void ScanModifiedAssetFiles(const TArray<FString>& InFilePaths) override;
	virtual void ScanModifiedAssetFiles(const TArray<FString>& InFilePaths, UE::AssetRegistry::EScanFlags ScanFlags) override;
	virtual void Serialize(FArchive& Ar) override;
	virtual void AppendState(const FAssetRegistryState& InState,
		UE::AssetRegistry::EAppendMode AppendMode = UE::AssetRegistry::EAppendMode::Append) override;
	virtual bool ActiveMountsIsEnabled() override;
	virtual void ActiveMountsInitialize() override;
	virtual bool ActiveMountsRegisterAndLoadFilePathSynchronous(FStringView StateFilePath) override;
	virtual void ActiveMountsRegisterAndLoadFilePathAsync(FStringView StateFilePath,
		TFunction<void(bool bSucceeded)> OnComplete = TFunction<void(bool bSucceeded)>(),
		UE::Tasks::ETaskPriority TaskPriority = UE::Tasks::ETaskPriority::Default) override;
	virtual void ActiveMountsLoadAsync(TConstArrayView<FStringView> MountPointLongPackageNames,
		TFunction<void(bool bCanceled)> OnComplete = TFunction<void(bool bCanceled)>()) override;
	virtual void ActiveMountsReloadAllAsync(
		TFunction<void(bool bCanceled)> OnComplete = TFunction<void(bool bCanceled)>()) override;
	virtual void ActiveMountsUnloadAsync(TConstArrayView<FStringView> MountPointLongPackageNames,
		TFunction<void(bool bCanceled)> OnComplete = TFunction<void(bool bCanceled)>()) override;
	virtual void ActiveMountsUnloadAllUnmountedAsync(
		TFunction<void(bool bCanceled)> OnComplete = TFunction<void(bool bCanceled)>()) override;
	virtual bool ActiveMountsIsLoaded(FStringView MountPointLongPackageName) override;
	virtual bool ActiveMountsIsInProgress() override;
	virtual void ActiveMountsReportWhenIdle(TFunction<void()> OnIdle) override;
	virtual SIZE_T GetAllocatedSize(bool bLogDetailed = false) const override;
	virtual void LoadPackageRegistryData(FArchive& Ar, FLoadPackageRegistryData& InOutData) const override;
	virtual void LoadPackageRegistryData(const FString& PackageFilename, FLoadPackageRegistryData& InOutData) const override;
	virtual void InitializeTemporaryAssetRegistryState(FAssetRegistryState& OutState, const FAssetRegistrySerializationOptions& Options,
		bool bRefreshExisting = false, const TSet<FName>& RequiredPackages = TSet<FName>(),
		const TSet<FName>& RemovePackages = TSet<FName>()) const override;
#if ASSET_REGISTRY_STATE_DUMPING_ENABLED
	virtual void DumpState(const TArray<FString>& Arguments, TArray<FString>& OutPages, int32 LinesPerPage = 1) const override;
#endif
	virtual TSet<FName> GetCachedEmptyPackagesCopy() const override;
	virtual bool ContainsTag(FName TagName) const override;
	virtual void InitializeSerializationOptions(FAssetRegistrySerializationOptions& Options, const ITargetPlatform* TargetPlatform = nullptr, UE::AssetRegistry::ESerializationTarget Target = UE::AssetRegistry::ESerializationTarget::ForGame) const override;

	DECLARE_DERIVED_EVENT( UAssetRegistryImpl, IAssetRegistry::FFilesBlockedEvent, FFilesBlockedEvent);
	virtual FFilesBlockedEvent& OnFilesBlocked() override;

	DECLARE_DERIVED_EVENT( UAssetRegistryImpl, IAssetRegistry::FPathsEvent, FPathsEvent);
	virtual FPathsEvent& OnPathsAdded() override;
	virtual FPathsEvent& OnPathsRemoved() override;
	
	DECLARE_DERIVED_EVENT( UAssetRegistryImpl, IAssetRegistry::FPathAddedEvent, FPathAddedEvent);
	virtual FPathAddedEvent& OnPathAdded() override;

	DECLARE_DERIVED_EVENT( UAssetRegistryImpl, IAssetRegistry::FPathRemovedEvent, FPathRemovedEvent);
	virtual FPathRemovedEvent& OnPathRemoved() override;

	DECLARE_DERIVED_EVENT(UAssetRegistryImpl, IAssetRegistry::FScanStartedEvent, FScanStartedEvent);
	virtual FScanStartedEvent& OnScanStarted() override;

	DECLARE_DERIVED_EVENT(UAssetRegistryImpl, IAssetRegistry::FScanEndedEvent, FScanEndedEvent);
	virtual FScanEndedEvent& OnScanEnded() override;

	DECLARE_DERIVED_EVENT(UAssetRegistryImpl, IAssetRegistry::FKnownGathersCompleteEvent, FKnownGathersCompleteEvent);
	virtual FKnownGathersCompleteEvent& OnKnownGathersComplete() override;

	virtual void AssetCreated(UObject* NewAsset) override;
	virtual void AssetDeleted(UObject* DeletedAsset) override;
	virtual void AssetRenamed(const UObject* RenamedAsset, const FString& OldObjectPath) override;
	virtual void AssetsSaved(TArray<FAssetData>&& SavedAssets) override;
	virtual void AssetUpdateTags(UObject* Object, EAssetRegistryTagsCaller Caller) override;
	virtual void AssetTagsFinalized(const UObject& FinalizedAsset) override;

	virtual bool VerseCreated(const FString& FilePath) override;
	virtual bool VerseDeleted(const FString& FilePath) override;

	virtual void PackageDeleted(UPackage* DeletedPackage) override;

	virtual IAssetRegistry::FAssetCollisionEvent& OnAssetCollision_Private() override;

	DECLARE_DERIVED_EVENT( UAssetRegistryImpl, IAssetRegistry::FAssetAddedEvent, FAssetAddedEvent);
	virtual FAssetAddedEvent& OnAssetAdded() override;

	DECLARE_DERIVED_EVENT( UAssetRegistryImpl, IAssetRegistry::FAssetRemovedEvent, FAssetRemovedEvent);
	virtual FAssetRemovedEvent& OnAssetRemoved() override;
	
	DECLARE_DERIVED_EVENT( UAssetRegistryImpl, IAssetRegistry::FAssetRenamedEvent, FAssetRenamedEvent);
	virtual FAssetRenamedEvent& OnAssetRenamed() override;

	DECLARE_DERIVED_EVENT( UAssetRegistryImpl, IAssetRegistry::FAssetUpdatedEvent, FAssetUpdatedEvent );
	virtual FAssetUpdatedEvent& OnAssetUpdated() override;
	virtual FAssetUpdatedEvent& OnAssetUpdatedOnDisk() override;

	// Batch events
	DECLARE_DERIVED_EVENT( UAssetRegistryImpl, IAssetRegistry::FAssetsEvent, FAssetsEvent);
	virtual FAssetsEvent& OnAssetsAdded() override;
	virtual FAssetsEvent& OnAssetsRemoved() override;
	virtual FAssetsEvent& OnAssetsUpdated() override;
	virtual FAssetsEvent& OnAssetsUpdatedOnDisk() override;
	
	DECLARE_DERIVED_EVENT( UAssetRegistryImpl, IAssetRegistry::FInMemoryAssetCreatedEvent, FInMemoryAssetCreatedEvent );
	virtual FInMemoryAssetCreatedEvent& OnInMemoryAssetCreated() override;

	DECLARE_DERIVED_EVENT( UAssetRegistryImpl, IAssetRegistry::FInMemoryAssetDeletedEvent, FInMemoryAssetDeletedEvent );
	virtual FInMemoryAssetDeletedEvent& OnInMemoryAssetDeleted() override;

	DECLARE_DERIVED_EVENT( UAssetRegistryImpl, IAssetRegistry::FVerseAddedEvent, FVerseAddedEvent);
	virtual FVerseAddedEvent& OnVerseAdded() override;

	DECLARE_DERIVED_EVENT( UAssetRegistryImpl, IAssetRegistry::FVerseRemovedEvent, FVerseRemovedEvent);
	virtual FVerseRemovedEvent& OnVerseRemoved() override;

	DECLARE_DERIVED_EVENT( UAssetRegistryImpl, IAssetRegistry::FFilesLoadedEvent, FFilesLoadedEvent );
	virtual FFilesLoadedEvent& OnFilesLoaded() override;

	DECLARE_DERIVED_EVENT( UAssetRegistryImpl, IAssetRegistry::FFileLoadProgressUpdatedEvent, FFileLoadProgressUpdatedEvent );
	virtual FFileLoadProgressUpdatedEvent& OnFileLoadProgressUpdated() override;

	virtual bool IsLoadingAssets() const override;
	virtual bool IsGathering() const override;
	virtual bool HasSerializedDiscoveryCache() const override;
	virtual bool ShouldUpdateDiskCacheAfterLoad() const override
	{
#if WITH_EDITORONLY_DATA
		return bUpdateDiskCacheAfterLoad;
#else
		return false;
#endif
	}

	virtual void Tick (float DeltaTime) override;

	virtual void ReadLockEnumerateAllTagToAssetDatas(TFunctionRef<bool(FName TagName, IAssetRegistry::FEnumerateAssetDatasFunc EnumerateAssets)> Callback) const override;

	virtual bool IsPathBeautificationNeeded(const FString& InAssetPath) const override;

	UE::AssetRegistry::Impl::EGatherStatus TickOnBackgroundThread();

	DECLARE_DERIVED_EVENT(UAssetRegistryImpl, IAssetRegistry::FEnumerateAssetsEvent, FEnumerateAssetsEvent)
	virtual FEnumerateAssetsEvent& OnEnumerateAssetsEvent() override;

protected:
	virtual void SetManageReferences(const TMultiMap<FAssetIdentifier, FAssetIdentifier>& ManagerMap,
		bool bClearExisting, UE::AssetRegistry::EDependencyCategory RecurseType,
		TSet<FDependsNode*>& ExistingManagedNode, ShouldSetManagerPredicate ShouldSetManager = nullptr) override;
	virtual void SetManageReferences(UE::AssetRegistry::FSetManageReferencesContext& Context) override;
	virtual bool SetPrimaryAssetIdForObjectPath(const FSoftObjectPath& ObjectPath, FPrimaryAssetId PrimaryAssetId) override;
	virtual void CopyManageDependenciesToState(FAssetRegistryState& InOutState) const override;

private:
	void OnPreExit();
#if WITH_EDITOR
	void OnFEngineLoopInitCompleteSearchAllAssets();
	/** Called when new gatherer is registered. Requires subsequent call to RebuildAssetDependencyGathererMapIfNeeded */
	void OnAssetDependencyGathererRegistered();
#endif
	void InitializeEvents(UE::AssetRegistry::Impl::FInitializeContext& Context);
	void Broadcast(UE::AssetRegistry::Impl::FEventContext& EventContext, bool bAllowFileLoadedEvent = false);
	bool CanBroadcastEvents() const;

	bool OnResolveRedirect(const FString& InPackageName, FString& OutPackageName);

#if WITH_EDITOR
	/** Called when a file in a content directory changes on disk */
	void OnDirectoryChanged(const TArray<struct FFileChangeData>& Files);

	/** Called when an asset is loaded, it will possibly update the cache */
	void OnAssetLoaded(UObject* AssetLoaded);
#endif

	/**
	 * Called by the engine core when a new MountPoint is added dynamically at runtime.  This is wired to
	 * FPackageName's static delegate.
	 */
	void OnMountPointMounted(const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint);

	/**
	 * Called by the engine core when a MountPoint is removed dynamically at runtime.  This is wired to
	 * FPackageName's static delegate.
	 */
	void OnMountPointDismounted(const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint);

	/** Batched processing version of OnMountPointMounted. */
	void OnMultipleMountPointsMounted(TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> MountedMountPoints);

	/** Batched processing version of the OnMountPointDismounted. */
	void OnMultipleMountPointsDismounted(TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> DismountedMountPoints);

	/** Called to refresh the native classes list, called at end of engine initialization. */
	void OnPostEngineInit();

	/**
	 * Called from LaunchEngineLoop via SetEngineStartupModuleLoadingComplete after plugins are loaded, used to scan
	 * classes that were loaded by plugins, and enable some global multithreaded access.
	 */
	void OnInitialPluginLoadingComplete();

	/** Shared helper for Scan*Synchronous function */
	void ScanPathsSynchronousInternal(const TArray<FString>& InDirs, const TArray<FString>& InFiles,
		UE::AssetRegistry::EScanFlags InScanFlags);

#if WITH_EDITOR
	void AddLoadedAssetToProcess(const UObject& AssetLoaded);
	/** Create FAssetData from any loaded UObject assets and store the updated AssetData in the state */
	void ProcessLoadedAssetsToUpdateCache(UE::AssetRegistry::Impl::FEventContext& EventContext,
		UE::AssetRegistry::Impl::EGatherStatus Status, UE::AssetRegistry::Impl::FInterruptionContext& InOutInterruptionContext);
#endif
	/**
	 * Remain under the given lock and return an InheritanceContext based on the appropriate choice of the persistent
	 * caching buffer or the function-scope-only passed in StackBuffer. Mark whether the buffer needs to be updated
	 * before being used. If the buffer needs to be updated and its the persistent buffer (which is protected data),
	 * convert the given lock to a write lock if not one already.
	 */
	void GetInheritanceContextWithRequiredLock(UE::AssetRegistry::FInterfaceRWScopeLock& InOutScopeLock,
		UE::AssetRegistry::Impl::FClassInheritanceContext& InheritanceContext,
		UE::AssetRegistry::Impl::FClassInheritanceBuffer& StackBuffer);
	void GetInheritanceContextWithRequiredLock(UE::AssetRegistry::FInterfaceWriteScopeLock& InOutScopeLock,
		UE::AssetRegistry::Impl::FClassInheritanceContext& InheritanceContext,
		UE::AssetRegistry::Impl::FClassInheritanceBuffer& StackBuffer);


#if WITH_EDITOR
	/**
	 * Callback for FObject::FAssetRegistryTag::OnGetExtraObjectTags
	 * If bAddMetaDataTagsToOnGetExtraObjectTags is true, this function will add missing FMetaData tags to cooked assets
	 */
	void OnGetExtraObjectTags(FAssetRegistryTagsContext Context);

	/**
	 * Checks whether the given path is already covered by the general directory watches, or whether we need to setup a
	 * new directory watcher. The caller must ensure that the Directory parameter is in FPaths::CreateStandardFilename format.
	 */
	bool IsDirAlreadyWatchedByRootWatchers(const FString& Directory) const;
#endif

	// Implementation methods for FSuspendPathsMounting and FSuspendPathsDismounting
	virtual void SuspendPathsMounting() override;
	virtual void ResumePathsMounting() override;
	virtual void SuspendPathsDismounting() override;
	virtual void ResumePathsDismounting() override;

	/** Request to pause or resume background processing of scan results.
	 *  This can be used to allow a priority thread to perform along sequence of operations
	 *  without having to contend with the background thread for data access
	 */
	virtual void RequestPauseBackgroundProcessing() override;
	virtual void RequestResumeBackgroundProcessing() override;
	bool IsBackgroundProcessingPaused() const 
	{ 
#if WITH_EDITOR 
		return GuardedData.IsBackgroundProcessingPaused(); 
#else
		return true;
#endif
	}

	/** In engine modes that do not tick regularly, request a tick, to process deferred events. */
	void RequestTick();
	/** In engine modes that do not tick regularly, mark the tick was called and clear the request. */
	void ClearRequestTick();

private:

	UE::AssetRegistry::FAssetRegistryImpl GuardedData;
	UE::AssetRegistry::Impl::FActiveMountsLoaderWrapper ActiveMountsLoader;

	/** Lock guarding the GuardedData */
	mutable UE::AssetRegistry::Private::FRWLockWithPriority InterfaceLock;

	/** This lock doesn't strictly protect any data (the InterfaceLock does that). Instead, 
	 *	it is used to let the main thread know when the gatherer thread is doing processing work
	 *  so that the main thread does not end up blocking on the InterfaceLock in Tick(). 
	 */
	FCriticalSection GatheredDataProcessingLock;

#if WITH_EDITOR
	/** Handles to all registered OnDirectoryChanged delegates */
	TMap<FString, FDelegateHandle> OnDirectoryChangedDelegateHandles;
	TArray<FString> DirectoryWatchRoots;

	/** List of objects that need to be processed because they were loaded or saved.
	*   This object lives outside of the GuardedData as it doesn't require the InterfaceLock
	*   to be written to. Reads must be coordinated such that only one thread reads at a time.
	*/
	TQueue<TWeakObjectPtr<const UObject>, EQueueMode::Mpsc> LoadedAssetsToProcess;
#endif

#if WITH_EDITORONLY_DATA
	/** If true, the asset registry will inject missing tags from FMetaData for cooked assets only in GetAssetRegistryTags */
	bool bAddMetaDataTagsToOnGetExtraObjectTags = true;

	/** If true, the AssetRegistry updates its on-disk information for an Asset whenever that Asset loads. */
	bool bUpdateDiskCacheAfterLoad = true;
#endif

	/** The delegate to execute when one or more files have been blocked from the registry */
	FFilesBlockedEvent FilesBlockedEvent;

	/** The delegate to execute when a batch of paths are added to the registry */
	FPathsEvent PathsAddedEvent;

	/** The delegate to execute when a batch of paths are removed from the registry */
	FPathsEvent PathsRemovedEvent;
	
	/** The delegate to execute when an asset path is added to the registry */
	FPathAddedEvent PathAddedEvent;

	/** The delegate to execute when an asset path is removed from the registry */
	FPathRemovedEvent PathRemovedEvent;

	/** The delegate to execute when an asset is added to the registry */
	FAssetAddedEvent AssetAddedEvent;
	
	/** The delegate to execute when an asset is removed from the registry */
	FAssetRemovedEvent AssetRemovedEvent;

	/** The delegate to execute when an asset is renamed in the registry */
	FAssetRenamedEvent AssetRenamedEvent;

	/** The delegate to execute when an asset is updated in the registry */
	FAssetUpdatedEvent AssetUpdatedEvent;

	/** The delegate to execute when an asset is updated on disk and has been reloaded in assetregistry */
	FAssetUpdatedEvent AssetUpdatedOnDiskEvent;

	/** The delegates to execute when assets are added/removed/updated in the registry, indexed by FEventContext::EEvent or returned with public accessors */
	static constexpr SIZE_T NumBatchedEvents = static_cast<SIZE_T>(UE::AssetRegistry::Impl::FEventContext::EEvent::MAX);
	FAssetsEvent BatchedAssetEvents[NumBatchedEvents];

	/** The delegate to execute when an in-memory asset was just created */
	FInMemoryAssetCreatedEvent InMemoryAssetCreatedEvent;

	/** The delegate to execute when an in-memory asset was just deleted */
	FInMemoryAssetDeletedEvent InMemoryAssetDeletedEvent;

	/** The delegate to execute when a Verse file is added to the registry */
	FVerseAddedEvent VerseAddedEvent;

	/** The delegate to execute when a Verse file is removed from the registry */
	FVerseRemovedEvent VerseRemovedEvent;

	/** The delegate to execute when finished loading files */
	FFilesLoadedEvent FileLoadedEvent;

	/** The delegate to execute while loading files to update progress */
	FFileLoadProgressUpdatedEvent FileLoadProgressUpdatedEvent;

	/** The delegate to execute scanning has begun */
	FScanStartedEvent ScanStartedEvent;

	/** The delegate to execute scanning has ended */
	FScanEndedEvent ScanEndedEvent;

	/** The delegate to execute when each batch of searching/gathering is complete */
	FKnownGathersCompleteEvent KnownGathersCompleteEvent;

	/** The delegate to execute when a query is made. */
	FEnumerateAssetsEvent EnumerateAssetsEvent;

	/**
	 * Storage for events that will be broadcast later on the game thread. Only safe
	 *  to access under the DeferredEventsCriticalSection. 
	 */
	UE::AssetRegistry::Impl::FEventContext DeferredEvents;
	/**
	 * Handle for a currently active tick request. Runtime game only, not used in GIsEditor. Used to process DeferredEvents.
	 * Read/Write only within DeferredEventsCriticalSection.
	 */
	FTSTicker::FDelegateHandle TickRequestHandle;
	FCriticalSection DeferredEventsCriticalSection;

	// Suspension and resuming of paths mounting and dismounting. Used for batching many
	// OnMountPointMounted/OnMountPointDismounted calls together.
	std::atomic<int32> SuspendMountingCounter = 0;
	std::atomic<int32> SuspendDismountingCounter = 0;
	FCriticalSection MountingCriticalSection;
	TArray<TRefCountPtr<UE::PackageName::IMountPoint>> ListOfMountedMountPoints;
	TArray<TRefCountPtr<UE::PackageName::IMountPoint>> ListOfDismountedMountPoints;


	friend FAssetDataGatherer;
	friend UE::AssetRegistry::FAssetRegistryImpl;
	friend UE::AssetRegistry::Impl::FActiveMountsLoader;
	friend UE::AssetRegistry::Premade::FAsyncConsumer;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
