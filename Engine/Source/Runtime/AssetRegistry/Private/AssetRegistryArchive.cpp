// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistryArchive.h"

#include "Algo/Sort.h"
#include "AssetRegistryPrivate.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "Serialization/MemoryHasher.h"

#ifndef ASSET_REGISTRY_LOAD_NAMES_IN_ORDER
#define ASSET_REGISTRY_LOAD_NAMES_IN_ORDER 0
#endif

constexpr uint32 AssetRegistryNumberedNameBit = 0x80000000;

static void SaveBundleEntries(FArchive& Ar, TArray<FAssetBundleEntry*>& Entries)
{
	for (FAssetBundleEntry* EntryPtr : Entries)
	{
		FAssetBundleEntry& Entry = *EntryPtr;	
		Ar << Entry.BundleName;

		int32 Num = Entry.AssetPaths.Num();
		Ar << Num;

		TArray<FTopLevelAssetPath*> SortedPaths;
		SortedPaths.Reserve(Num);
		for (FTopLevelAssetPath& Path : Entry.AssetPaths)
		{
			SortedPaths.Add(&Path);
		}
		Algo::Sort(SortedPaths, [](FTopLevelAssetPath* A, FTopLevelAssetPath* B) { return A->Compare(*B) < 0; });
		for (FTopLevelAssetPath* Path : SortedPaths)
		{
			// Serialize using FSoftObjectPath for backwards compatibility with FRedirectCollector code.
			// We can investigate if any of this can be bypassed in future.

			FSoftObjectPath TmpPath(*Path, {});
			TmpPath.SerializePath(Ar);
		}
	}
}

static void LoadBundleEntries(FArchive& Ar, TArray<FAssetBundleEntry>& Entries)
{
	for (FAssetBundleEntry& Entry : Entries)
	{
		Ar << Entry.BundleName;

		int32 Num = 0;
		Ar << Num;
		Entry.AssetPaths.SetNum(Num);

		for (FTopLevelAssetPath& Path : Entry.AssetPaths)
		{
			FSoftObjectPath TmpPath;
			TmpPath.SerializePath(Ar);
			Path = TmpPath.GetAssetPath();
		}

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Entry.BundleAssets.Reserve(Entry.AssetPaths.Num());
		for (const FTopLevelAssetPath& Path : Entry.AssetPaths)
		{
			Entry.BundleAssets.Add(FSoftObjectPath(Path, {}));
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	}
}

static void LoadBundleEntriesOldVersion(FArchive& Ar, TArray<FAssetBundleEntry>& Entries, FAssetRegistryVersion::Type Version)
{
	for (FAssetBundleEntry& Entry : Entries)
	{
		Ar << Entry.BundleName;

		int32 Num = 0;
		Ar << Num;
		Entry.AssetPaths.SetNum(Num);

		if (Version < FAssetRegistryVersion::RemoveAssetPathFNames)
		{
			for (FTopLevelAssetPath& Path : Entry.AssetPaths)
			{
				// This change is synchronized with a change to the format of FSoftObjectPath in EUnrealEngineObjectUE5Version::FSOFTOBJECTPATH_REMOVE_ASSET_PATH_FNAMES
				// We have to manually deserialize the old format of FSoftObjectPath
				FName AssetPathName;
				Ar << AssetPathName;
				FString SubPathString;
				Ar << SubPathString;
			
				Path = FTopLevelAssetPath(AssetPathName.ToString());
			}
		}
		else
		{
			for (FTopLevelAssetPath& Path : Entry.AssetPaths)
			{
				FSoftObjectPath TmpPath;
				TmpPath.SerializePath(Ar);
				Path = TmpPath.GetAssetPath();
			}
		}
		
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Entry.BundleAssets.Reserve(Entry.AssetPaths.Num());
		for (const FTopLevelAssetPath& Path : Entry.AssetPaths)
		{
			Entry.BundleAssets.Add(FSoftObjectPath(Path, {}));
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	}
}

static void SaveBundles(FArchive& Ar, const TSharedPtr<FAssetBundleData, ESPMode::ThreadSafe>& Bundles)
{
	TArray<FAssetBundleEntry*> SortedEntries;
	if (Bundles)
	{
		TArray<FAssetBundleEntry>& Entries = Bundles->Bundles;
		SortedEntries.Reserve(Entries.Num());
		for (FAssetBundleEntry& Entry : Entries)
		{
			SortedEntries.Add(&Entry);
		}
		Algo::Sort(SortedEntries, [](FAssetBundleEntry* A, FAssetBundleEntry* B) { return A->BundleName.LexicalLess(B->BundleName); });
	}

	int32 Num = SortedEntries.Num();
	Ar << Num;

	SaveBundleEntries(Ar, SortedEntries);
}

static FORCEINLINE TSharedPtr<FAssetBundleData, ESPMode::ThreadSafe> LoadBundlesInternal(FArchive& Ar, FAssetRegistryVersion::Type Version)
{
	int32 Num;
	Ar << Num;

	if (Num > 0)
	{
		FAssetBundleData Temp;
		Temp.Bundles.SetNum(Num);
		if (Version == FAssetRegistryVersion::LatestVersion)
		{
			LoadBundleEntries(Ar, Temp.Bundles);
		}
		else
		{
			LoadBundleEntriesOldVersion(Ar, Temp.Bundles, Version);
		}

		return MakeShared<FAssetBundleData, ESPMode::ThreadSafe>(MoveTemp(Temp));
	}

	return TSharedPtr<FAssetBundleData, ESPMode::ThreadSafe>();
}

static TSharedPtr<FAssetBundleData, ESPMode::ThreadSafe> LoadBundles(FArchive& Ar)
{
	return LoadBundlesInternal(Ar, FAssetRegistryVersion::LatestVersion);
}

static TSharedPtr<FAssetBundleData, ESPMode::ThreadSafe> LoadBundlesOldVersion(FArchive& Ar, FAssetRegistryVersion::Type Version)
{
	return LoadBundlesInternal(Ar, Version);
}

void FAssetRegistryHeader::SerializeHeader(FArchive& Ar)
{
	FAssetRegistryVersion::SerializeVersion(Ar, Version);
	if (Version >= FAssetRegistryVersion::AddedHeader)
	{
		Ar << bFilterEditorOnlyData;
	}
	else if(Ar.IsLoading())
	{
		bFilterEditorOnlyData = false;		
	}
}

#if ALLOW_NAME_BATCH_SAVING

FAssetRegistryWriterOptions::FAssetRegistryWriterOptions(const FAssetRegistrySerializationOptions& Options)
: Tags({Options.CookTagsAsName, Options.CookTagsAsPath})
{}

FAssetRegistryWriter::FAssetRegistryWriter(const FAssetRegistryWriterOptions& Options, FArchive& Out)
: FArchiveProxy(MemWriter)
, Tags(Options.Tags)
, TargetAr(Out)
{
	check(!IsLoading());

	// Copy requested serialization flags to intermediate archive. Technically the flags could change after this as TargetAr is a passed-in reference
	SetArchiveState(TargetAr.GetArchiveState()); 

	// The state copy explicitly clears this flag! 
	SetFilterEditorOnly(TargetAr.IsFilterEditorOnly()); 

	// The above function in FArchiveProxy seems broken - it only modifies the inner archive state, but this archive's state is what will be returned to serialization functions
	FArchive::SetFilterEditorOnly(TargetAr.IsFilterEditorOnly()); 
}

static TArray<FDisplayNameEntryId> FlattenIndex(const TMap<FDisplayNameEntryId, uint32>& Names)
{
	TArray<FDisplayNameEntryId> Out;
	Out.SetNumZeroed(Names.Num());
	for (TPair<FDisplayNameEntryId, uint32> Pair : Names)
	{
		Out[Pair.Value] = Pair.Key;
	}
	return Out;
}

FAssetRegistryWriter::~FAssetRegistryWriter()
{
	// Save store data and collect FNames
	int64 BodySize = MemWriter.TotalSize();
	UE::AssetRegistry::AlignPosInArchive(MemWriter);	// make sure the data before store leaves it in the aligned position
	int64 StoreStart = MemWriter.TotalSize();
	SaveStore(Tags.Finalize(), *this);

	// Save in load-friendly order - names, store then body / tag maps
	SaveNameBatch(FlattenIndex(Names), TargetAr);

	// make sure the Store is aligned
	UE::AssetRegistry::AlignPosInArchive(TargetAr);
	TargetAr.Serialize(MemWriter.GetData() + StoreStart, MemWriter.TotalSize() - StoreStart);
	// Body does not need to be aligned because we copy all the data out of it rather than pointing to its memory-mapped position
	TargetAr.Serialize(MemWriter.GetData(), BodySize);
}

FArchive& FAssetRegistryWriter::operator<<(FName& Value)
{
	FDisplayNameEntryId EntryId(Value);

	uint32 Index = Names.FindOrAdd(EntryId, Names.Num());
	check((Index & AssetRegistryNumberedNameBit) == 0);

	if (Value.GetNumber() != NAME_NO_NUMBER_INTERNAL)
	{
		Index |= AssetRegistryNumberedNameBit;
		uint32 Number = Value.GetNumber();
		return *this << Index << Number;
	}

	return *this << Index;
}

void SaveTags(FAssetRegistryWriter& Writer, const FAssetDataTagMapSharedView& Map)
{
	uint64 MapHandle = Writer.Tags.AddTagMap(Map).ToInt();
	Writer << MapHandle;
}

void FAssetRegistryWriter::SerializeTagsAndBundles(const FAssetData& Out)
{
	SaveTags(*this, Out.TagsAndValues);
	SaveBundles(*this, Out.TaggedAssetBundles);
}

#endif

FAssetRegistryReader::FAssetRegistryReader(FArchive& Inner,
	TSharedPtr<UE::AssetRegistry::FMemoryMappedFile> InBackingFile,
	const FixedTagPrivate::FWeakStorePtr& PreviousLoadedStore,
	int32 NumWorkers, FAssetRegistryHeader Header)
	: FArchiveProxy(Inner)
{
	check(IsLoading());

	SetFilterEditorOnly(Header.bFilterEditorOnlyData);
	FArchive::SetFilterEditorOnly(Header.bFilterEditorOnlyData); // Workaround for bug in FArchiveProxy

	// Load older versions for compatibility
	if (UNLIKELY(Header.Version < FAssetRegistryVersion::MemoryMappedTagDataStore))
	{
		Names = LoadNameBatch(Inner, ENameBatchLoadingFlags::RespectOrder);
		Tags = FixedTagPrivate::LoadStore(*this, Header.Version, nullptr);
	}
	else
	{
		TRefCountPtr<const FixedTagPrivate::FStore> PreviousStore = PreviousLoadedStore.Pin();
		if (NumWorkers > 0 && !PreviousStore) // Multithreaded loading does not yet support PreviousStore
		{
			ENameBatchLoadingFlags LoadingFlags = ENameBatchLoadingFlags::None;
#if ASSET_REGISTRY_LOAD_NAMES_IN_ORDER
			LoadingFlags |= ENameBatchLoadingFlags::RespectOrder;
#endif
			TFunction<TArray<FDisplayNameEntryId> ()> GetFutureNames = LoadNameBatchAsync(*this, NumWorkers, LoadingFlags);

			FixedTagPrivate::FAsyncStoreLoader StoreLoader;
			StoreLoader.SetStoreBackingFile(InBackingFile);
			Task = StoreLoader.ReadInitialDataAndKickLoad(*this, NumWorkers, Header.Version);
		
			Names = GetFutureNames();
			Tags = StoreLoader.LoadFinalData(*this, Header.Version);
		}
		else
		{
			Names = LoadNameBatch(Inner, ENameBatchLoadingFlags::RespectOrder);
			if (PreviousStore)
			{
				Tags = MoveTemp(PreviousStore);
				FixedTagPrivate::SkipBytesStore(*this, Header.Version);
			}
			else
			{
				Tags = FixedTagPrivate::LoadStore(*this, Header.Version, InBackingFile);
			}
		}
	}
}

FAssetRegistryReader::~FAssetRegistryReader()
{
	WaitForTasks();
}

const TRefCountPtr<const FixedTagPrivate::FStore>& FAssetRegistryReader::GetTags() const
{
	return Tags;
}

void FAssetRegistryReader::WaitForTasks()
{
	if (Task.IsValid())
	{
		Task.Wait();
	}
}

FArchive& FAssetRegistryReader::operator<<(FName& Out)
{
	checkf(Names.Num() > 0, TEXT("Attempted to load FName before name batch loading has finished"));

	uint32 Index = 0;
	uint32 Number = NAME_NO_NUMBER_INTERNAL;
	
	*this << Index;

	if (Index & AssetRegistryNumberedNameBit)
	{
		Index -= AssetRegistryNumberedNameBit;
		*this << Number;
	}

	if (Index >= (uint32) Names.Num())
	{
		SetError();
	}
	else
	{
		Out = Names[Index].ToName(Number);
	}

	return *this;
}

FAssetDataTagMapSharedView LoadTags(FAssetRegistryReader& Reader)
{
	uint64 MapHandle;
	Reader << MapHandle;
	return FAssetDataTagMapSharedView(FixedTagPrivate::FPartialMapHandle::FromInt(MapHandle).MakeFullHandle(Reader.Tags->Index));
}

void FAssetRegistryReader::SerializeTagsAndBundles(FAssetData& Out)
{
	Out.TagsAndValues = LoadTags(*this);
	Out.TaggedAssetBundles = LoadBundles(*this);
}

void FAssetRegistryReader::SerializeTagsAndBundlesOldVersion(FAssetData& Out, FAssetRegistryVersion::Type Version)
{
	Out.TagsAndValues = LoadTags(*this);
	Out.TaggedAssetBundles = LoadBundlesOldVersion(*this, Version);
}

////////////////////////////////////////////////////////////////////////////

#if WITH_DEV_AUTOMATION_TESTS 

#include "Misc/AutomationTest.h"
#include "NameTableArchive.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAssetRegistryTagSerializationTest, "System.AssetRegistry.SerializeTagMap", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)

FAssetDataTagMapSharedView MakeLooseMap(std::initializer_list<TPairInitializer<const char*, FString>> Pairs)
{
	FAssetDataTagMap Out;
	Out.Reserve(Pairs.size());
	for (TPair<const char*, FString> Pair : Pairs)
	{
		Out.Add(FName(Pair.Key), Pair.Value);
	}
	return FAssetDataTagMapSharedView(MoveTemp(Out));
}


bool FAssetRegistryTagSerializationTest::RunTest(const FString& Parameters)
{
	TArray<FAssetDataTagMapSharedView> LooseMaps;
	LooseMaps.Add(FAssetDataTagMapSharedView());
	LooseMaps.Add(MakeLooseMap({{"Key",			"StringValue"}, 
								{"Key_0",		"StringValue_0"}}));
	LooseMaps.Add(MakeLooseMap({{"Name",		"NameValue"}, 
								{"Name_0",		"NameValue_0"}}));
	LooseMaps.Add(MakeLooseMap({{"FullPath",	"/S/P.C\'P.O\'"}, 
								{"PkgPath",		"P.O"},
								{"ObjPath",		"O"}}));
	LooseMaps.Add(MakeLooseMap({{"NumPath_0",	"/S/P.C\'P.O_0\'"}, 
								{"NumPath_1",	"/S/P.C\'P_0.O\'"},
								{"NumPath_2",	"/S/P.C_0\'P.O\'"},
								{"NumPath_3",	"/S/P.C\'P_0.O_0\'"},
								{"NumPath_4",	"/S/P.C_0\'P_0.O\'"},
								{"NumPath_5",	"/S/P.C_0\'P.O_0\'"},
								{"NumPath_6",	"/S/P.C_0\'P_0.O_0\'"}}));
	LooseMaps.Add(MakeLooseMap({{"SameSame",	"SameSame"}, 
								{"AlsoSame",	"SameSame"}}));
	LooseMaps.Add(MakeLooseMap({{"FilterKey1",	"FilterValue1"}, 
								{"FilterKey2",	"FilterValue2"}}));
	LooseMaps.Add(MakeLooseMap({{"Localized",	"NSLOCTEXT(\"\", \"5F8411BA4D1A349F6E2C56BB04A1A810\", \"Content Browser Walkthrough\")"}}));
	LooseMaps.Add(MakeLooseMap({{"Wide",		TEXT("Wide\x00DF")}}));

	TArray<uint8> Data;

#if ALLOW_NAME_BATCH_SAVING
	FAssetRegistryWriterOptions Options;
	Options.Tags.StoreAsName = {	"Name", "Name_0"};
	Options.Tags.StoreAsPath = {	"FullPath", "PkgPath", "ObjPath",
									"NumPath_0", "NumPath_1", "NumPath_2",
									"NumPath_3", "NumPath_4", "NumPath_5", "NumPath_6"};
	{
		FMemoryWriter DataWriter(Data);
		FAssetRegistryWriter RegistryWriter(Options, DataWriter);
		for (const FAssetDataTagMapSharedView& LooseMap : LooseMaps)
		{
			SaveTags(RegistryWriter, LooseMap);
		}
	}
#endif

	TArray<FAssetDataTagMapSharedView> FixedMaps;
	FixedMaps.SetNum(LooseMaps.Num());

	{
		FMemoryReader DataReader(Data);
		FAssetRegistryHeader Header;
		FAssetRegistryReader RegistryReader(DataReader, nullptr, FixedTagPrivate::FWeakStorePtr(), 0, Header);
		RegistryReader.WaitForTasks();

		for (FAssetDataTagMapSharedView& FixedMap : FixedMaps)
		{
			FixedMap = LoadTags(RegistryReader);
		}
	}

	TestTrue("SerializeTagsAndBundles round-trip", FixedMaps == LooseMaps);

	// Re-create second fixed tag store to test operator==(FMapHandle, FMapHandle)
	{
		FMemoryReader DataReader(Data);
		FAssetRegistryHeader Header;
		FAssetRegistryReader RegistryReader(DataReader, nullptr, FixedTagPrivate::FWeakStorePtr(), 0, Header);
		RegistryReader.WaitForTasks();

		for (const FAssetDataTagMapSharedView& FixedMap1 : FixedMaps)
		{
			FAssetDataTagMapSharedView FixedMap2 = LoadTags(RegistryReader);
			TestTrue("Fixed tag map equality", FixedMap1 == FixedMap2);
		}
	}

	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
