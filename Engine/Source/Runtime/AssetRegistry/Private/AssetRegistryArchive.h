// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetDataTagMapSerializationDetails.h"
#include "AssetRegistry/AssetRegistrySerializationDetails.h"
#include "AssetRegistry/AssetRegistryStateSerialization.h"
#include "Serialization/LargeMemoryWriter.h"
#include "UObject/NameBatchSerialization.h"

struct FAssetData;
struct FAssetRegistrySerializationOptions;

// Header containing versioning & data stripping information for a serialized asset registry state.
struct FAssetRegistryHeader
{
	FAssetRegistryVersion::Type Version = FAssetRegistryVersion::LatestVersion;
	bool bFilterEditorOnlyData = false; // True if editor only data has been removed/should be removed while saving 

	void SerializeHeader(FArchive& Ar);
};

/** @see FAssetRegistryWriter */
class FAssetRegistryReader : public FArchiveProxy
{
public:
	/// @param InBackingFile shared pointer to a class that allows mapping the underlying file into memory
	/// @param NumWorkers > 0 for parallel loading
	FAssetRegistryReader(FArchive& Inner, TSharedPtr<UE::AssetRegistry::FMemoryMappedFile> InBackingFile,
		const FixedTagPrivate::FWeakStorePtr& PreviousLoadedStore, int32 NumWorkers, FAssetRegistryHeader Header);
	~FAssetRegistryReader();
	const TRefCountPtr<const FixedTagPrivate::FStore>& GetTags() const;

	virtual FArchive& operator<<(FName& Value) override;

	void SerializeTagsAndBundles(FAssetData& Out);
	void SerializeTagsAndBundlesOldVersion(FAssetData& Out, FAssetRegistryVersion::Type Version);

	void WaitForTasks();

private:
	TArray<FDisplayNameEntryId> Names;
	TRefCountPtr<const FixedTagPrivate::FStore> Tags;
	TFuture<void> Task;

	friend FAssetDataTagMapSharedView LoadTags(FAssetRegistryReader& Reader);
};

#if ALLOW_NAME_BATCH_SAVING

class FAssetRegistryWriterBase
{
protected:
	FLargeMemoryWriter MemWriter;
};

struct FAssetRegistryWriterOptions
{
	FAssetRegistryWriterOptions() = default;
	explicit FAssetRegistryWriterOptions(const FAssetRegistrySerializationOptions& Options);

	FixedTagPrivate::FOptions Tags;
};

/**
 * Indexes FName and tag maps and serializes out deduplicated indices instead.
 *
 * Unlike previous FNameTableArchiveWriter:
 * - Name data stored as name batches, which is faster
 * - Name batch is written as a header instead of footer for faster seek-free loading
 * - Numberless FNames are serialized as a single 32-bit int
 * - Deduplicates all tag values, not just names
 *
 * Use in conjunction with FNameBatchReader.
 *
 * Data is written to inner archive in destructor to achieve seek-free loading.
 */
class FAssetRegistryWriter : public FAssetRegistryWriterBase, public FArchiveProxy
{
public:
	FAssetRegistryWriter(const FAssetRegistryWriterOptions& Options, FArchive& Out);
	~FAssetRegistryWriter();

	virtual FArchive& operator<<(FName& Value) override;

	void SerializeTagsAndBundles(const FAssetData& In);

private:
	TMap<FDisplayNameEntryId, uint32> Names;
	FixedTagPrivate::FStoreBuilder Tags;
	FArchive& TargetAr;

	friend void SaveTags(FAssetRegistryWriter& Writer, const FAssetDataTagMapSharedView& Map);
};

#endif
