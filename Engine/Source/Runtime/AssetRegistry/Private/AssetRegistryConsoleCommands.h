// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistryPrivate.h"
#include "HAL/IConsoleManager.h"
#include "HAL/FileManager.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "AssetRegistry"

class FAssetRegistryConsoleCommands
{
public:

	FAutoConsoleCommand GetByNameCommand;
	FAutoConsoleCommand GetByPathCommand;
	FAutoConsoleCommand GetByClassCommand;
	FAutoConsoleCommand GetByTagCommand;
	FAutoConsoleCommand GetDependenciesCommand;
	FAutoConsoleCommand GetReferencersCommand;
	FAutoConsoleCommand FindInvalidUAssetsCommand;
	FAutoConsoleCommand ScanPathCommand;
	FAutoConsoleCommand DumpAllocatedSizeCommand;
	FAutoConsoleCommand DumpStateCommand;

	FAssetRegistryConsoleCommands()
		: GetByNameCommand(
		TEXT( "AssetRegistry.GetByName" ),
		*LOCTEXT("CommandText_GetByName", "<PackageName> //Query the asset registry for assets matching the supplied package name").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw( this, &FAssetRegistryConsoleCommands::GetByName ) )
	,	GetByPathCommand(
		TEXT( "AssetRegistry.GetByPath" ),
		*LOCTEXT("CommandText_GetByPath", "<Path> //Query the asset registry for assets matching the supplied package path").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw( this, &FAssetRegistryConsoleCommands::GetByPath ) )
	,	GetByClassCommand(
		TEXT( "AssetRegistry.GetByClass" ),
		*LOCTEXT("CommandText_GetByClass", "<ClassName> //Query the asset registry for assets matching the supplied class").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw( this, &FAssetRegistryConsoleCommands::GetByClass ) )
	,	GetByTagCommand(
		TEXT( "AssetRegistry.GetByTag" ),
		*LOCTEXT("CommandText_GetByTag", "<TagName> <TagValue> //Query the asset registry for assets matching the supplied tag and value").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw( this, &FAssetRegistryConsoleCommands::GetByTag ) )
	,	GetDependenciesCommand(
		TEXT( "AssetRegistry.GetDependencies" ),
		*LOCTEXT("CommandText_GetDependencies", "<PackageName> //Query the asset registry for dependencies for the specified package").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw( this, &FAssetRegistryConsoleCommands::GetDependencies ) )
	,	GetReferencersCommand(
		TEXT( "AssetRegistry.GetReferencers" ),
		*LOCTEXT("CommandText_GetReferencers", "<ObjectPath> //Query the asset registry for referencers for the specified package").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw( this, &FAssetRegistryConsoleCommands::GetReferencers ) )
	,	FindInvalidUAssetsCommand(
		TEXT( "AssetRegistry.Debug.FindInvalidUAssets" ),
		*LOCTEXT("CommandText_FindInvalidUAssets", "Finds a list of all assets which are in UAsset files but do not share the name of the package").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw( this, &FAssetRegistryConsoleCommands::FindInvalidUAssets ) )
	, ScanPathCommand(
		TEXT("AssetRegistry.ScanPath"),
		*LOCTEXT("CommandText_ScanPath", "<PathToScan> //Scan the given filename or directoryname for package files and load them into the assetregistry. Extra string parameters: -forcerescan, -ignoreDenyLists, -asfile, -asdir").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAssetRegistryConsoleCommands::ScanPath ) )
	, DumpAllocatedSizeCommand(
		TEXT("AssetRegistry.DumpAllocatedSize"),
		*LOCTEXT("CommandText_DumpAllocatedSize", "Dump the allocations of the asset registry state to the log").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAssetRegistryConsoleCommands::DumpAllocatedSize ) )
	, DumpStateCommand(
		TEXT("AssetRegistry.DumpState"),
		*LOCTEXT("CommandText_DumpState", "Dump the state of the asset registry to a file. Pass -log to dump to the log as well. Extra string parameters: All, ObjectPath, PackageName, Path, Class, Tag, Dependencies, DependencyDetails, PackageData, AssetBundles, AssetTags").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAssetRegistryConsoleCommands::DumpState ) )
	{}

	void GetByName(const TArray<FString>& Args)
	{
		if ( Args.Num() < 1 )
		{
			UE_LOGF(LogAssetRegistry, Log, "Usage: AssetRegistry.GetByName PackageName");
			return;
		}

		TArray<FAssetData> AssetData;
		const FName AssetPackageName = FName(*Args[0]);
		IAssetRegistry::GetChecked().GetAssetsByPackageName(AssetPackageName, AssetData);
		UE_LOGF(LogAssetRegistry, Log, "GetAssetsByPackageName for %ls:", *AssetPackageName.ToString());
		for (int32 AssetIdx = 0; AssetIdx < AssetData.Num(); ++AssetIdx)
		{
			AssetData[AssetIdx].PrintAssetData();
		}
	}

	void GetByPath(const TArray<FString>& Args)
	{
		if ( Args.Num() < 1 )
		{
			UE_LOGF(LogAssetRegistry, Log, "Usage: AssetRegistry.GetByPath Path");
			return;
		}

		TArray<FAssetData> AssetData;
		const FName AssetPath = FName(*Args[0]);
		IAssetRegistry::GetChecked().GetAssetsByPath(AssetPath, AssetData);
		UE_LOGF(LogAssetRegistry, Log, "GetAssetsByPath for %ls:", *AssetPath.ToString());
		for (int32 AssetIdx = 0; AssetIdx < AssetData.Num(); ++AssetIdx)
		{
			AssetData[AssetIdx].PrintAssetData();
		}
	}

	void GetByClass(const TArray<FString>& Args)
	{
		if ( Args.Num() < 1 )
		{
			UE_LOGF(LogAssetRegistry, Log, "Usage: AssetRegistry.GetByClass ClassPathname");
			return;
		}

		TArray<FAssetData> AssetData;
		const FTopLevelAssetPath ClassPathName(Args[0]);
		if (!ClassPathName.IsNull())
		{
			IAssetRegistry::GetChecked().GetAssetsByClass(ClassPathName, AssetData);
			UE_LOGF(LogAssetRegistry, Log, "GetAssetsByClass for %ls:", *ClassPathName.ToString());
			for (int32 AssetIdx = 0; AssetIdx < AssetData.Num(); ++AssetIdx)
			{
				AssetData[AssetIdx].PrintAssetData();
			}
		}
		else
		{
			UE_LOGF(LogAssetRegistry, Error, "not a valid class path name (E.g. /Script/Engine.Actor)");
		}
	}

	void GetByTag(const TArray<FString>& Args)
	{
		if ( Args.Num() < 2 )
		{
			UE_LOGF(LogAssetRegistry, Log, "Usage: AssetRegistry.GetByTag TagName TagValue");
			return;
		}

		TMultiMap<FName, FString> TagsAndValues;
		TagsAndValues.Add(FName(*Args[0]), Args[1]);

		TArray<FAssetData> AssetData;
		IAssetRegistry::GetChecked().GetAssetsByTagValues(TagsAndValues, AssetData);
		UE_LOGF(LogAssetRegistry, Log, "GetAssetsByTagValues for Tag'%ls' and Value'%ls':", *Args[0], *Args[1]);
		for (int32 AssetIdx = 0; AssetIdx < AssetData.Num(); ++AssetIdx)
		{
			AssetData[AssetIdx].PrintAssetData();
		}
	}

	void GetDependencies(const TArray<FString>& Args)
	{
 		if ( Args.Num() < 1 )
 		{
 			UE_LOGF(LogAssetRegistry, Log, "Usage: AssetRegistry.GetDependencies PackageName");
 			return;
 		}
 
 		const FName PackageName = FName(*Args[0]);
 		TArray<FName> Dependencies;
 		
 		if ( IAssetRegistry::GetChecked().GetDependencies(PackageName, Dependencies) )
 		{
			UE_LOGF(LogAssetRegistry, Log, "Dependencies for %ls:", *PackageName.ToString());
			for ( auto DependencyIt = Dependencies.CreateConstIterator(); DependencyIt; ++DependencyIt )
			{
				UE_LOGF(LogAssetRegistry, Log, "   %ls", *(*DependencyIt).ToString());
			}
 		}
 		else
 		{
 			UE_LOGF(LogAssetRegistry, Log, "Could not find dependency data for %ls:", *PackageName.ToString());
 		}
	}

	void GetReferencers(const TArray<FString>& Args)
	{
		if ( Args.Num() < 1 )
		{
			UE_LOGF(LogAssetRegistry, Log, "Usage: AssetRegistry.GetReferencers ObjectPath");
			return;
		}

		const FName PackageName = FName(*Args[0]);
		TArray<FName> Referencers;

		if ( IAssetRegistry::GetChecked().GetReferencers(PackageName, Referencers) )
		{
			UE_LOGF(LogAssetRegistry, Log, "Referencers for %ls:", *PackageName.ToString());
			for ( auto ReferencerIt = Referencers.CreateConstIterator(); ReferencerIt; ++ReferencerIt )
			{
				UE_LOGF(LogAssetRegistry, Log, "   %ls", *(*ReferencerIt).ToString());
			}
		}
		else
		{
			UE_LOGF(LogAssetRegistry, Log, "Could not find referencer data for %ls:", *PackageName.ToString());
		}
	}

	void FindInvalidUAssets(const TArray<FString>& Args)
	{
		TArray<FAssetData> AllAssets;
		IAssetRegistry::GetChecked().GetAllAssets(AllAssets);

		UE_LOGF(LogAssetRegistry, Log, "Invalid UAssets:");

		for (int32 AssetIdx = 0; AssetIdx < AllAssets.Num(); ++AssetIdx)
		{
			const FAssetData& AssetData = AllAssets[AssetIdx];

			// Note, the 'internal' version of DoesPackageExist must be used to avoid re-entering the AssetRegistry's lock resulting in deadlock
			FPackagePath PackagePath;
			if (FPackageName::InternalDoesPackageExistEx(AssetData.PackageName.ToString(), FPackageName::EPackageLocationFilter::Any, 
				false /*bMatchCaseOnDisk*/, &PackagePath) != FPackageName::EPackageLocationFilter::None)
			{
				if (PackagePath.GetHeaderExtension() == EPackageExtension::Asset && !AssetData.IsUAsset())
				{
					// This asset was in a package with a uasset extension but did not share the name of the package
					UE_LOGF(LogAssetRegistry, Log, "%ls", *AssetData.GetObjectPathString());
				}
			}
		}
	}

	void ScanPath(const TArray<FString>& Args)
	{
		bool bForceRescan = false;
		bool bIgnoreDenyList = false;
		bool bAsFile = false;
		bool bAsDir = false;

		FString InPath;
		for (const FString& Arg : Args)
		{
			if (Arg.StartsWith(TEXT("-")))
			{
				bForceRescan = bForceRescan || Arg.Equals(TEXT("-forcerescan"), ESearchCase::IgnoreCase);
				bIgnoreDenyList = bIgnoreDenyList || Arg.Equals(TEXT("-ignoreDenyLists"), ESearchCase::IgnoreCase);
				bAsDir = bAsDir || Arg.Equals(TEXT("-asdir"), ESearchCase::IgnoreCase);
				bAsFile = bAsFile || Arg.Equals(TEXT("-asfile"), ESearchCase::IgnoreCase);
			}
			else
			{
				InPath = Arg;
			}
		}
		if (InPath.IsEmpty())
		{
			UE_LOGF(LogAssetRegistry, Log, "Usage: AssetRegistry.ScanPath [-forcerescan] [-ignoreDenyLists] [-asfile] [-asdir] FileOrDirectoryPath");
			return;
		}

		if (!bAsDir && !bAsFile)
		{
			bAsDir = true;
			if (FPackageName::IsValidLongPackageName(InPath))
			{
				FString LocalPath;
				if (FPackageName::TryConvertLongPackageNameToFilename(InPath, LocalPath))
				{
					// Note, the 'internal' version of DoesPackageExist must be used to avoid re-entering the AssetRegistry's lock resulting in deadlock
					FPackagePath PackagePath = FPackagePath::FromLocalPath(LocalPath);
					if (FPackageName::InternalDoesPackageExistEx(PackagePath, FPackageName::EPackageLocationFilter::Any, 
						false /* bMatchCaseOnDisk */, &PackagePath) != FPackageName::EPackageLocationFilter::None)
					{
						bAsFile = true;
						bAsDir = false;
					}
				}
			}
			else if (IFileManager::Get().FileExists(*InPath))
			{
				bAsFile = true;
				bAsDir = false;
			}
		}
		if (bAsDir)
		{
			IAssetRegistry::GetChecked().ScanPathsSynchronous({ InPath }, bForceRescan, bIgnoreDenyList);
		}
		else
		{
			IAssetRegistry::GetChecked().ScanFilesSynchronous({ InPath }, bForceRescan);
		}
	}

	void DumpAllocatedSize(const TArray<FString>& Args)
	{
		SIZE_T Size = IAssetRegistry::GetChecked().GetAllocatedSize(true);

		UE_LOGF(LogAssetRegistry, Log, "Total %2.f mb", double(Size) / 1024.0 / 1024.0);
	}

	void DumpState(const TArray<FString>& Args)	
	{
		const bool bLog = Args.Contains(TEXT("log"));
		const bool bDashLog = Args.Contains(TEXT("-log"));
		if (Args.Num() == 0 + int32(bLog) + int32(bDashLog))
		{
			UE_LOGF(LogAssetRegistry, Error, "No arguments for asset registry dump.");
			return;
		}

		const bool bDoLog = bLog || bDashLog || (!ALLOW_DEBUG_FILES);

#if ASSET_REGISTRY_STATE_DUMPING_ENABLED
		FString Path = FPaths::ProfilingDir() / FString::Printf(TEXT("AssetRegistryState_%s.txt"), *FDateTime::Now().ToString());
		TUniquePtr<FArchive> Ar{ IFileManager::Get().CreateDebugFileWriter(*Path) };

		TArray<FString> Pages;
		IAssetRegistry::GetChecked().DumpState(Args, Pages, 1000);
		for (const FString& Page : Pages)
		{
			if( bDoLog )
			{
				UE_LOGF(LogAssetRegistry, Log, "%ls", *Page);
			}
#if ALLOW_DEBUG_FILES
			Ar->Logf(TEXT("%s"), *Page);
#endif
		}

		UE_LOGF(LogAssetRegistry, Display, "Dumped asset registry state to %ls.", *Path);
#else
		UE_LOGF(LogAssetRegistry, Error, "Asset registry dumping is disabled by compilation flags.");
#endif
	}

};

#undef LOCTEXT_NAMESPACE
