// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/AssetRegistryHelpers.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry.h"
#include "Algo/AnyOf.h"
#include "Algo/RemoveIf.h"
#include "Blueprint/BlueprintSupport.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "UObject/LinkerLoad.h"

#if WITH_EDITOR
#include "Misc/RedirectCollector.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetRegistryHelpers)

namespace UE::AssetRegistry
{

template<typename TLessThan>
static void SortAssets(TArray<FAssetData>& Assets, TLessThan&& Predicate, EAssetRegistrySortOrder SortOrder)
{
	switch (SortOrder)
	{
	case EAssetRegistrySortOrder::Ascending: Assets.Sort(Predicate); break;
	case EAssetRegistrySortOrder::Descending:
		Assets.Sort([&Predicate](const FAssetData& Left, const FAssetData& Right)
		{
			return Predicate(Right, Left);
		});
		break;
	default: checkNoEntry(); break;
	}
}
}

TScriptInterface<IAssetRegistry> UAssetRegistryHelpers::GetAssetRegistry()
{
	return &UAssetRegistryImpl::Get();
}

FAssetData UAssetRegistryHelpers::CreateAssetData(const UObject* InAsset, bool bAllowBlueprintClass /*= false*/)
{
	if (InAsset && InAsset->IsAsset())
	{
		return FAssetData(InAsset, bAllowBlueprintClass);
	}
	else
	{
		return FAssetData();
	}
}

bool UAssetRegistryHelpers::IsValid(const FAssetData& InAssetData)
{
	return InAssetData.IsValid();
}

bool UAssetRegistryHelpers::IsUAsset(const FAssetData& InAssetData)
{
	return InAssetData.IsUAsset();
}

FString UAssetRegistryHelpers::GetFullName(const FAssetData& InAssetData)
{
	return InAssetData.GetFullName();
}

bool UAssetRegistryHelpers::IsRedirector(const FAssetData& InAssetData)
{
	return InAssetData.IsRedirector();
}

FSoftObjectPath UAssetRegistryHelpers::ToSoftObjectPath(const FAssetData& InAssetData) 
{
	return InAssetData.ToSoftObjectPath();
}

UClass* UAssetRegistryHelpers::GetClass(const FAssetData& InAssetData)
{
	return InAssetData.GetClass();
}

UObject* UAssetRegistryHelpers::GetAsset(const FAssetData& InAssetData)
{
	return InAssetData.GetAsset();
}

bool UAssetRegistryHelpers::IsAssetLoaded(const FAssetData& InAssetData)
{
	return InAssetData.IsAssetLoaded();
}

#if WITH_EDITOR
bool UAssetRegistryHelpers::IsAssetCooked(const FAssetData& InAssetData)
{
	return InAssetData.HasAnyPackageFlags(PKG_Cooked);
}

bool UAssetRegistryHelpers::AssetHasEditorOnlyData(const FAssetData& InAssetData)
{
	return !InAssetData.HasAnyPackageFlags(PKG_FilterEditorOnly);
}
#endif	// WITH_EDITOR

FString UAssetRegistryHelpers::GetExportTextName(const FAssetData& InAssetData)
{
	return InAssetData.GetExportTextName();
}

bool UAssetRegistryHelpers::GetTagValue(const FAssetData& InAssetData, const FName& InTagName, FString& OutTagValue)
{
	return InAssetData.GetTagValue(InTagName, OutTagValue);
}

FARFilter UAssetRegistryHelpers::SetFilterTagsAndValues(const FARFilter& InFilter, const TArray<FTagAndValue>& InTagsAndValues)
{
	FARFilter FilterCopy = InFilter;
	for (const FTagAndValue& TagAndValue : InTagsAndValues)
	{
		FilterCopy.TagsAndValues.Add(TagAndValue.Tag, TagAndValue.Value);
	}

	return FilterCopy;
}

UClass* UAssetRegistryHelpers::FindAssetNativeClass(const FAssetData& AssetData)
{
	UClass* AssetClass = AssetData.GetClass();
	if (AssetClass == nullptr)
	{
		const IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
		
		TArray<FTopLevelAssetPath> AncestorClasses;
		AssetRegistry.GetAncestorClassNames(AssetData.AssetClassPath, AncestorClasses);
		for (const FTopLevelAssetPath& AncestorClassPath : AncestorClasses)
		{
			AssetClass = FindObject<UClass>(AncestorClassPath);
			if (AssetClass)
			{
				break;
			}
		}
	}
	while (AssetClass && !AssetClass->HasAnyClassFlags(CLASS_Native))
	{
		AssetClass = AssetClass->GetSuperClass();
	}

	return AssetClass;
}

void UAssetRegistryHelpers::SortByPredicate(
	TArray<FAssetData>& Assets,
	FSortingPredicate SortingPredicate,
	EAssetRegistrySortOrder SortOrder
	)
{
	if (SortingPredicate.IsBound())
	{
		UE::AssetRegistry::SortAssets(
			Assets,
			[&SortingPredicate](const FAssetData& Left, const FAssetData& Right)
			{
				return SortingPredicate.Execute(Left, Right);
			}, SortOrder);
	}
}

void UAssetRegistryHelpers::SortByAssetName(TArray<FAssetData>& Assets, EAssetRegistrySortOrder SortOrder)
{
	UE::AssetRegistry::SortAssets(Assets,
		[](const FAssetData& Left, const FAssetData& Right)
		{
			// Summary: String compare needed instead of FName::LexicalLess.
			// Reason: FName::LexicalLess says e.g. FName("Scene_10") < FName("Scene_01") (while: FString::operator< says "Scene_01" < "Scene_10").
			// Explanation:
			// - "Scene_10" has ComparisionIndex of "Scene" and Number = 11,
			// - "Scene_01" has ComparisionIndex of "Scene_01" and number 0
			// - Thus, (FName("Scene_10").LexicalLess(FName("Scene_01")) internally ends up checking "Scene" < "Scene_01" , which is true.
			// - For reference, "Scene_1" has ComparisionIndex of "Scene" and number 2, which, when sorting, we'd "expect" Scene_01 to have, too.
			return Left.AssetName.ToString() < Right.AssetName.ToString();
		}, SortOrder);
}

void UAssetRegistryHelpers::FindReferencersOfAssetOfClass(
	UObject* AssetInstance,
	TConstArrayView<UClass*> InMatchClasses,
	TArray<FAssetData>& OutAssetDatas
	)
{
	FindReferencersOfAssetOfClass(AssetInstance->GetOutermost()->GetFName(), InMatchClasses, OutAssetDatas);
}

void UAssetRegistryHelpers::FindReferencersOfAssetOfClass(
	const FAssetIdentifier& InAssetIdentifier,
	TConstArrayView<UClass*> InMatchClasses,
	TArray<FAssetData>& OutAssetDatas
	)
{
	// If the asset registry is still loading assets, we cant check for referencers, so we must open the rename dialog
	const IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	TArray<FAssetIdentifier> Referencers;
	AssetRegistry.GetReferencers(InAssetIdentifier, Referencers);

	for (auto AssetIdentifier : Referencers)
	{
		TArray<FAssetData> Assets;
		AssetRegistry.GetAssetsByPackageName(AssetIdentifier.PackageName, Assets);

		for (auto AssetData : Assets)
		{
			if (InMatchClasses.Num() > 0)
			{
				if (Algo::AnyOf(InMatchClasses, [&AssetData](UClass* MatchClass){ return AssetData.IsInstanceOf(MatchClass); }))
				{
					OutAssetDatas.AddUnique(AssetData);
				}
			}
			else
			{
				OutAssetDatas.AddUnique(AssetData);
			}
		}
	}
}

void UAssetRegistryHelpers::GetBlueprintAssets(const FARFilter& InFilter, TArray<FAssetData>& OutAssetData)
{
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	FARFilter Filter(InFilter);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	UE_CLOGF(!InFilter.ClassNames.IsEmpty(), LogCore, Error,
		"ARFilter.ClassNames is not supported by UAssetRegistryHelpers::GetBlueprintAssets and will be ignored.");
	Filter.ClassNames.Empty();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;

	// Expand list of classes to include derived classes
	TArray<FTopLevelAssetPath> BlueprintParentClassPathRoots = MoveTemp(Filter.ClassPaths);
	TSet<FTopLevelAssetPath> BlueprintParentClassPaths;
	if (Filter.bRecursiveClasses)
	{
		AssetRegistry.GetDerivedClassNames(
			BlueprintParentClassPathRoots, 
			TSet<FTopLevelAssetPath>(),
			 BlueprintParentClassPaths
			 );
	}
	else
	{
		BlueprintParentClassPaths.Append(BlueprintParentClassPathRoots);
	}

	// Search for all blueprints and then check BlueprintParentClassPaths in the results
	Filter.ClassPaths.Reset(1);
	Filter.ClassPaths.Add(FTopLevelAssetPath(FName("/Script/Engine"), FName("BlueprintCore")));
	Filter.bRecursiveClasses = true;

	auto FilterLambda = [&OutAssetData, &BlueprintParentClassPaths](const FAssetData& AssetData)
	{
		// Verify blueprint class
		if (BlueprintParentClassPaths.IsEmpty() || IsAssetDataBlueprintOfClassSet(AssetData, BlueprintParentClassPaths))
		{
			OutAssetData.Add(AssetData);
		}
		return true;
	};
	AssetRegistry.EnumerateAssets(Filter, FilterLambda, UE::AssetRegistry::EEnumerateAssetsFlags::None);
}

void UAssetRegistryHelpers::GetDerivedClassAssetData(const TArray<FTopLevelAssetPath>& InBaseClasses, TArray<FAssetData>& OutDerivedClassAssetData)
{
	IAssetRegistry& AR = IAssetRegistry::GetChecked();
	TSet<FTopLevelAssetPath> DerivedClassNames;
	// For a large working set with millions of assets and thousands of entries in the 
	// hierarchy, GetDerivedClassNames has been measured to cost about 2.5ms - more than 
	// acceptable for editor performance:
	AR.GetDerivedClassNames(InBaseClasses, TSet<FTopLevelAssetPath>(), DerivedClassNames);

	// For the same working set this loop takes 4.5ms - more than acceptable for editor 
	// performance but it could be optimized. My problems in the callers are greater ATM.
	OutDerivedClassAssetData.Reserve(DerivedClassNames.Num());
	for (const FTopLevelAssetPath& Path : DerivedClassNames)
	{
		const FName PackageName = Path.GetPackageName();

		// we have no asset data for built in types
		if (PackageName.ToString().StartsWith(TEXT("/Script/")))
		{
			continue;
		}
		// SKEL classes should not even be in the asset registry.. but we can easily filter
		// them here:
		FString AssetName = Path.GetAssetName().ToString();
		if (AssetName.StartsWith(TEXT("SKEL_")))
		{
			continue;
		}

		// look for blueprint asset first, as the _C (class) object won't 
		// reliably report itself as editor only:
		if (AssetName.EndsWith(TEXT("_C")))
		{
			AssetName.RemoveFromEnd(TEXT("_C"));
		}

		const bool bIncludeOnlyOnDiskAssets = false;
		const bool bSkipARFilteredAssets = false;
		FAssetData AssetData = AR.GetAssetByObjectPath(
			FSoftObjectPath{FTopLevelAssetPath{PackageName, FName(*AssetName)}, FString()}, 
			bIncludeOnlyOnDiskAssets, 
			bSkipARFilteredAssets);
		if (AssetData.PackageName.IsNone())
		{
			// We didn't find an asset, but don't give up - look for an asset with _C - 
			// the origin of this ambiguity is blueprints, where we have  two asset 
			// objects (the generated class and the UBlueprint):
			AssetName += TEXT("_C");
			AssetData = AR.GetAssetByObjectPath(
				FSoftObjectPath{FTopLevelAssetPath{PackageName, FName(*AssetName)}, FString()}, 
				bIncludeOnlyOnDiskAssets, 
				bSkipARFilteredAssets);
		}

		// AssetData very likely is set, no known cases where we fail to find asset data
		// but callers will appreciate if we guarantee that all returned asset data
		// is valid:
		if(!AssetData.PackageName.IsNone()) 
		{
			OutDerivedClassAssetData.Emplace(MoveTemp(AssetData));
		}
	}
}

bool UAssetRegistryHelpers::IsAssetDataBlueprintOfClassSet(const FAssetData& AssetData, const TSet<FTopLevelAssetPath>& ClassNameSet)
{
	const FString ParentClassFromData = AssetData.GetTagValueRef<FString>(FBlueprintTags::ParentClassPath);
	if (!ParentClassFromData.IsEmpty())
	{
		const FTopLevelAssetPath ClassObjectPath(FPackageName::ExportTextPathToObjectPath(ParentClassFromData));
		const FName ClassName = ClassObjectPath.GetAssetName();

		TArray<FTopLevelAssetPath> ValidNames;
		ValidNames.Add(ClassObjectPath);
		// Check for redirected name
		FTopLevelAssetPath RedirectedName = FTopLevelAssetPath(FLinkerLoad::FindNewPathNameForClass(ClassObjectPath.ToString(), false));
		if (!RedirectedName.IsNull())
		{
			ValidNames.Add(RedirectedName);
		}
		for (const FTopLevelAssetPath& ValidName : ValidNames)
		{
			if (ClassNameSet.Contains(ValidName))
			{
				// Our parent class is in the class name set
				return true;
			}
		}
	}
	return false;
}

UAssetRegistryHelpers::FTemporaryCachingModeScope::FTemporaryCachingModeScope(bool InTempCachingMode)
{
	PreviousCachingMode = UAssetRegistryHelpers::GetAssetRegistry()->GetTemporaryCachingMode();
	UAssetRegistryHelpers::GetAssetRegistry()->SetTemporaryCachingMode(InTempCachingMode);
}

UAssetRegistryHelpers::FTemporaryCachingModeScope::~FTemporaryCachingModeScope()
{
	UAssetRegistryHelpers::GetAssetRegistry()->SetTemporaryCachingMode(PreviousCachingMode);
}

void UAssetRegistryHelpers::FixupRedirectedAssetPath(FSoftObjectPath& InOutSoftObjectPath)
{
	if (InOutSoftObjectPath.IsNull())
	{
		return;
	}

	FSoftObjectPath FoundRedirection;
	InOutSoftObjectPath.FixupCoreRedirects();

#if WITH_EDITOR
	FoundRedirection = GRedirectCollector.GetAssetPathRedirection(InOutSoftObjectPath);
	if (FoundRedirection.IsValid())
	{
		InOutSoftObjectPath = FoundRedirection;
		return;
	}
#endif

	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	FoundRedirection = AssetRegistry.GetRedirectedObjectPath(InOutSoftObjectPath.GetWithoutSubPath());
	InOutSoftObjectPath = FSoftObjectPath(FoundRedirection.GetAssetPath(), InOutSoftObjectPath.GetSubPathUtf8String());
}

void UAssetRegistryHelpers::FixupRedirectedAssetPath(FName& InOutAssetPath)
{
	if (InOutAssetPath.IsNone())
	{
		return;
	}

	FSoftObjectPath SoftObjectPath(InOutAssetPath.ToString());
	FixupRedirectedAssetPath(SoftObjectPath);
	InOutAssetPath = FName(*SoftObjectPath.ToString());
}

#if WITH_EDITOR
TArray<FAssetData> UAssetRegistryHelpers::GetAssetsWithOuterForPaths(TConstArrayView<FName> InPackagePaths, FName InOuterPath, bool bInRecursivePaths, bool bInIncludeOnlyOnDiskAssets, bool bInExactOuter)
{
	EAssetsWithOuterForPathsFlags Flags = EAssetsWithOuterForPathsFlags::ScanPaths;
	if (bInRecursivePaths)
	{
		Flags |= EAssetsWithOuterForPathsFlags::RecursivePaths;
	}

	if (bInIncludeOnlyOnDiskAssets)
	{
		Flags |= EAssetsWithOuterForPathsFlags::IncludeOnlyOnDiskAsset;
	}

	if (bInExactOuter)
	{
		Flags |= EAssetsWithOuterForPathsFlags::ExactOuter;
	}

	return GetAssetsWithOuterForPaths(InPackagePaths, InOuterPath, Flags);
}


TArray<FAssetData> UAssetRegistryHelpers::GetAssetsWithOuterForPaths(TConstArrayView<FName> InPackagePaths, FName InOuterPath, EAssetsWithOuterForPathsFlags InFlags)
{
	FARFilter Filter;
	Filter.PackagePaths = InPackagePaths;
	Filter.bRecursivePaths = EnumHasAnyFlags(InFlags, EAssetsWithOuterForPathsFlags::RecursivePaths);
	Filter.bIncludeOnlyOnDiskAssets = EnumHasAnyFlags(InFlags, EAssetsWithOuterForPathsFlags::IncludeOnlyOnDiskAsset);

	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	
	if (EnumHasAnyFlags(InFlags, EAssetsWithOuterForPathsFlags::ScanPaths))
	{
		TArray<FString> PathsToScan;
		PathsToScan.Reserve(Filter.PackagePaths.Num());
		Algo::Transform(Filter.PackagePaths, PathsToScan, [](FName InPackagePath) { return InPackagePath.ToString(); });

		AssetRegistry.ScanSynchronous(PathsToScan, TArray<FString>());
	}

	TArray<FAssetData> Assets;
	AssetRegistry.GetAssets(Filter, Assets);

	FNameBuilder PartialOuterRoot;
	if (!EnumHasAnyFlags(InFlags, EAssetsWithOuterForPathsFlags::ExactOuter))
	{
		InOuterPath.AppendString(PartialOuterRoot);
	}
	Assets.SetNum(
		Algo::RemoveIf(Assets, [InOuterPath, &PartialOuterRoot](const FAssetData& InAssetData)
		{
			if (InAssetData.GetOptionalOuterPathName() == InOuterPath)
			{
				return false;
			}

			if (PartialOuterRoot.Len() > 0 && FNameBuilder(InAssetData.GetOptionalOuterPathName()).ToView().StartsWith(PartialOuterRoot.ToView(), ESearchCase::IgnoreCase))
			{
				return false;
			}

			return true;
		}));
	Assets.Sort();

	return Assets;
}
#endif
