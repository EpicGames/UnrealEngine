// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARLoader.h"
#include "ARTypes.h"
#include "AssetDataGatherer.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/PathTree.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Queue.h"
#include "Containers/RingBuffer.h"
#include "Containers/Set.h"
#include "HAL/CriticalSection.h"
#include "Math/NumericLimits.h"
#include "Misc/Optional.h"
#include "ModuleDescriptor.h"
#include "PackageDependencyData.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"

#include <atomic>

class ITargetPlatform;
class FDependsNode;
struct FARFilter;
class FAssetDataGatherer;
struct FFileChangeData;
class FPackageReader;
class UAssetRegistryImpl;
namespace UE::AssetRegistry::DependsNode { struct FGatherDependencyDataScratchpad; }
namespace UE::AssetRegistry::Premade { enum class ELoadResult : uint8; }
namespace UE::AssetRegistry::Premade { struct FAsyncConsumer; }
namespace UE::AssetRegistry::Impl { class FActiveMountEvents; }
namespace UE::AssetRegistry::Impl { class FActiveMountsLoader; }
namespace UE::AssetRegistry::Impl { struct FEventContext; }
namespace UE::AssetRegistry::Impl { struct FInitializeContext; }
namespace UE::AssetRegistry::Impl { struct FScanPathContext; }
namespace UE::AssetRegistry::Impl { struct FTickContext; }
namespace UE::AssetRegistry::Impl { struct FClassInheritanceContext; }
#if WITH_EDITOR
namespace UE::AssetDependencyGatherer::Private { class FRegisteredAssetDependencyGatherer; }
#endif

namespace UE::AssetRegistry
{

namespace Impl
{
/** Container for global class inheritance data; the AssetRegistry may use a persistent buffer or a stack local buffer. */
struct FClassInheritanceBuffer
{
	/** Map from Class->SuperClass for all classes including native classes and blueprint classes. Updated on demand. */
	TMap<FTopLevelAssetPath, FTopLevelAssetPath> InheritanceMap;
	/** Map from Class->(All subclasses) for all classes including native classes and blueprint classes. Updated on demand. */
	TMap<FTopLevelAssetPath, TArray<FTopLevelAssetPath>> ReverseInheritanceMap;
	/** Snapshot of GetCurrentAllClassesVersionNumber() at the time of the last update, to invalidate on changes to classes. */
	uint64 SavedAllClassesVersionNumber = MAX_uint64;
	/** Dirty flag to invalidate on other changes requiring a recompute. */
	bool bDirty = true;

	/** Report whether the dirty flag and other checks indicate this buffer does not need to be updated. */
	bool IsUpToDate(uint64 CurrentAllClassesVersionNumber) const;
	/** Delete data and free allocations. */
	void Clear();
	/** Report size of dynamic allocations. */
	SIZE_T GetAllocatedSize() const;
};

/** Status of gathering, returned from the Tick function */
enum class EGatherStatus : uint8
{
	TickActiveGatherActive,
	TickActiveGatherIdle,
	TickGameThreadActiveGatherIdle,
	Complete,
	UnableToProgress,
	WaitingForEvents,
};
	
/** Affects how rules are applied to improve loading/runtime performance */
enum EPerformanceMode : uint8
{
	// Handling slow async load
	BulkLoading,

	// Not changing, optimize for runtime queries
	MostlyStatic,
};

/** Provides handling for time slicing during TickGatherer */
struct FInterruptionContext
{
public:
	typedef TFunction<bool(void)> ShouldExitEarlyCallbackType;

	FInterruptionContext() = default;
	explicit FInterruptionContext(double InTickStartTime, double InMaxRunningTime)
		: TickStartTime(InTickStartTime)
		, MaxRunningTime(InMaxRunningTime)
	{
	}

	FInterruptionContext(double InTickStartTime, double InMaxRunningTime, ShouldExitEarlyCallbackType& Callback)
		: TickStartTime(InTickStartTime)
		, MaxRunningTime(InMaxRunningTime)
		, EarlyExitCallback(Callback) {}

	void SetEarlyExitCallback(const ShouldExitEarlyCallbackType& InCallback)
	{
		EarlyExitCallback = InCallback;
	}
	void SetUnlimitedTickTime()
	{
		TickStartTime = -1.;
		MaxRunningTime = -1.;
	}
	void SetLimitedTickTime(double InTickStartTime, double InMaxRunningTime)
	{
		TickStartTime = InTickStartTime;
		MaxRunningTime = InMaxRunningTime;
	}
	double GetTickStartTime() const { return TickStartTime; }
	bool IsTimeSlicingEnabled() const { return TickStartTime > 0; }
	bool WasInterrupted() const { return OutInterrupted; }
	bool ShouldExitEarly();
	void RequestEarlyExit() { OutInterrupted = true; }

private:
	// A negative value disables time slicing
	double TickStartTime = -1.;
	// The maximum time should allow before interruption. If TickStartTime is negative, this is ignored
	double MaxRunningTime = -1.;
	// If provided, this is always checked
	ShouldExitEarlyCallbackType EarlyExitCallback;
	// True if we ran out of time, EarlyExitCallback returned true, or RequestEarlyExit() was called
	bool OutInterrupted = false;
};

/** Container holding information for runtime resolution of parsed filter tag rules */
struct FParsedFilterTagRules
{
	struct FFilterTagsRule
	{
		FTopLevelAssetPath ClassPath;
		const UClass* ClassPtr = nullptr;
		int32 InheritanceDepth = 0; // used as a sorting criteria to order classes from base to leaf.
		bool bInclude = false;
	};
	TArray<FFilterTagsRule> OrderedList;
	bool bContainsAnyUnresolvedRule = false;
};

struct FFilterTagRules
{
	SIZE_T GetAllocatedSize() const { return Rules.GetAllocatedSize(); }

	TMap<FName, FParsedFilterTagRules> Rules;
	uint64 ClassesFingerprint = 0;
	bool bContainsAnyUnresolvedRule = false;
};
} // namespace Impl within namespace UE::AssetRegistry

/** Options for how AssetRegistry functions should handle operations that must only be run on the gamethread. */
enum class EDeferFlag
{
	/** The function is running on the gamethread currently, it can execute the operation. */
	CanExecuteGameThread,
	/** The function should mark itself temporarily failed and the caller will try again later. */
	RetryLater,
	/** The function should send a message to the game thread to execute the operation there. */
	QueueToGameThread,
	/**
	 * The function can not execute, can not defer, cannot queue; it has to complete its task now.
	 * Skip the operation, and log warnings or errors if the skipping will cause problems.
	 */
	BypassNow,
};

} // namespace UE::AssetRegistry

/** Returns true if ANY work remains to be done. This work might require the game thread. */
FORCEINLINE bool IsTickActive(UE::AssetRegistry::Impl::EGatherStatus Status) 
{ 
	return (Status == UE::AssetRegistry::Impl::EGatherStatus::TickGameThreadActiveGatherIdle) 
		|| (Status == UE::AssetRegistry::Impl::EGatherStatus::TickActiveGatherActive)
		|| (Status == UE::AssetRegistry::Impl::EGatherStatus::TickActiveGatherIdle);
}

namespace UE::AssetRegistry
{

/**
 * Threading helper class for UAssetRegistryImpl that holds all of the data.
 * This class is not threadsafe; all const functions must be called from within a ReadLock and all non-const 
 * from within a WriteLock.
 */
class FAssetRegistryImpl
{
	using FAssetDataMap = UE::AssetRegistry::Private::FAssetDataMap;
	using FCachedAssetKey = UE::AssetRegistry::Private::FCachedAssetKey;

public:
	/** Constructor initializes as little as possible; meaningful operations are first done during Initialize. */
	FAssetRegistryImpl();
	/** Construct the AssetRegistryImpl, including initial scans if applicable. */
	void Initialize(UE::AssetRegistry::FInterfaceWriteScopeLock& ScopeLock, Impl::FInitializeContext& Context);

	/** Event for when duplicated assets are found and needs to be resolved */
	IAssetRegistry::FAssetCollisionEvent& OnAssetCollision_Private();

	// Helpers for functions of the same name from UAssetRegistryImpl


	bool HasAssets(const FName PackagePath, const bool bRecursive) const;
	/** 
	* Returns the redirected object path for the provided soft object path. 
	* This function will not attempt to scan the redirected path to ensure it has been registered with the Asset Registry. 
	* If it's unclear if the object path is known to the Asset Registry already, call GetAndScanRedirectedObjectPath instead.
	*/
	FSoftObjectPath GetRedirectedObjectPath(FInterfaceReadScopeLock& ScopeLock, const FSoftObjectPath& ObjectPath);
	/**
	* Returns the redirected object path for the provided soft object path.
	* This function will scan the redirected path to ensure it has been registered with the Asset Registry.
	*/
	FSoftObjectPath GetAndScanRedirectedObjectPath(FInterfaceWriteScopeLock& ScopeLock, const FSoftObjectPath& ObjectPath, 
		UE::AssetRegistry::Impl::FEventContext* EventContext, UE::AssetRegistry::Impl::FClassInheritanceContext* InheritanceContext);
	bool GetAncestorClassNames(Impl::FClassInheritanceContext& InheritanceContext, FTopLevelAssetPath ClassName,
		TArray<FTopLevelAssetPath>& OutAncestorClassNames) const;
	void CompileFilter(Impl::FClassInheritanceContext& InheritanceContext, const FARFilter& InFilter,
		FARCompiledFilter& OutCompiledFilter) const;
	void SetTemporaryCachingMode(bool bEnable);
	void SetTemporaryCachingModeInvalidated();
	bool AddPath(Impl::FEventContext& EventContext, FStringView PathToAdd);
	void SearchAllAssets(Impl::FEventContext& EventContext, Impl::FClassInheritanceContext& InheritanceContext,
		bool bSynchronousSearch);
	bool GetVerseFilesByPath(FName PackagePath, TArray<FName>* OutFilePaths, bool bRecursive) const;
	void ScanPathsSynchronous(FInterfaceWriteScopeLock* ScopeLock, Impl::FScanPathContext& Context);
	void PrioritizeSearchPath(const FString& PathToPrioritize);
	void ScanModifiedAssetFiles(FInterfaceWriteScopeLock& ScopeLock, Impl::FEventContext& EventContext, Impl::FClassInheritanceContext& InheritanceContext,
		const TArray<FString>& InFilePaths, UE::AssetRegistry::EScanFlags ScanFlags);
	void Serialize(FArchive& Ar, Impl::FEventContext& EventContext);
	void AppendState(Impl::FEventContext& EventContext, const FAssetRegistryState& InState,
		FAssetRegistryState::EInitializationMode Mode = FAssetRegistryState::EInitializationMode::Append, bool bEmitAssetEvents = false);
	/** Similar to AppendState, but it moves rather than copies. @see FAssetRegistryState::ConsumeExisting. */
	void ConsumeState(Impl::FEventContext& EventContext, FAssetRegistryState&& InState, bool bEmitAssetEvents = false);
	void GetAllocatedSize(bool bLogDetailed, SIZE_T& StateSize, SIZE_T& StaticSize, SIZE_T& SearchSize) const;
	bool IsLoadingAssets() const;
	bool IsGathering() const;
	bool HasSerializedDiscoveryCache() const;
	void SetManageReferences(FSetManageReferencesContext& Context);
	bool SetPrimaryAssetIdForObjectPath(Impl::FEventContext& EventContext, const FSoftObjectPath& ObjectPath, FPrimaryAssetId PrimaryAssetId);
	bool ResolveRedirect(const FString& InPackageName, FString& OutPackageName) const;
#if WITH_EDITOR
	void OnDirectoryChanged(FInterfaceWriteScopeLock& ScopeLock, Impl::FEventContext& EventContext, Impl::FClassInheritanceContext& InheritanceContext,
		TArray<FFileChangeData>& FileChangesProcessed);
	void OnDirectoryRescanRequired(FInterfaceWriteScopeLock& ScopeLock, Impl::FEventContext& EventContext,
		Impl::FClassInheritanceContext& InheritanceContext, FString& DirPath, int64 BeforeTimeStamp);
#endif
	void OnMultipleMountPointsMounted(UAssetRegistryImpl& UARI, Impl::FEventContext& EventContext,
		Impl::FActiveMountEvents& ActiveMountEvents,
		Impl::FClassInheritanceContext& InheritanceContext,
		UE::AssetRegistry::FInterfaceWriteScopeLock& ScopeLock,
		TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> MountedMountPoints);
	void OnMultipleMountPointsDismounted(UAssetRegistryImpl& UARI, Impl::FEventContext& EventContext,
		Impl::FActiveMountEvents& ActiveMountEvents, UE::AssetRegistry::FInterfaceWriteScopeLock& ProofOfWriteLock,
		TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> DismountedMountPoints);

	// Other helper functions called by UAssetRegistryImpl


	/**
	 * Update cached values about classes, and enable some global multithreaded access,
	 * when all startup plugins finish loading.
	 */
	void OnPluginLoadingComplete(bool bPhaseSuccessful);
	/** Update cached values about native classes after they have possibly changed. */
	void RefreshNativeClasses();

	/** Enumerate assets in the State, filtering by filter and package not in PackagesToSkip */
	void EnumerateDiskAssets(const FARCompiledFilter& InFilter, TSet<FName>& PackagesToSkip,
		TFunctionRef<bool(const FAssetData&)> Callback, UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags) const;
	/** Enumerate all assets in the State. PackageNamesToSkip are filtered. EEnumerateAssetsFlags provide additional filtering. */
	void EnumerateAllDiskAssets(TSet<FName>& PackageNamesToSkip, TFunctionRef<bool(const FAssetData&)> Callback, UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags = UE::AssetRegistry::EEnumerateAssetsFlags::None) const;

	/** Waits for the gatherer to be idle if it is operating synchronously. */
	void WaitForGathererIdleIfSynchronous();
	/** Waits for the gatherer to be idle. */
	void WaitForGathererIdle(float TimeoutSeconds);
	/** Consume any results from the gatherer and return its status */
	Impl::EGatherStatus TickGatherer(Impl::FTickContext& TickContext);
	/**
	 * During gamethread tick, if we find that the background thread is active in TickGatherer, process any classes
	 * that the background requested from the gamethread tick.
	 */
	void ProcessGameThreadRequestedClasses();
	/**
	 * If TickContext is on a background thread and the game thread has executed ProcessGameThreadRequestedClasses,
	 * retry all assets in AssetsForGameThread by moving them to the any-thread Assets queue.
	 */
	void ConditionalRetryGatherAssetsForGameThread(Impl::FTickContext& TickContext);
	/** Clear GameThreadRequestedClasses if AssetsForGameThread is empty. */
	void ConditionalClearGameThreadRequestedClasses(Impl::FTickContext& TickContext);
	/** Add a class to GameThreadRequestedClasses and set the sentry flag to request processing. */
	void RequestGameThreadProcessClass(const FTopLevelAssetPath& RequestedGameThreadClass);
	/**
	 * At some times during editor startup, depending on config settings, we might want to block
	 * on the gather, and not tick the rest of the editor. When we want that to happen, we set
	 * this flag. This function can be read outside of the lock (we need to do that during Tick).
	 */
	bool IsGameThreadTakeOverGatherEachTick() const;
	void SetGameThreadTakeOverGatherEachTick(bool bValue);

	/** Send a log message with the search statistics. 
	 *  StartTime is used to report wall clock search time in the case of background scan
	 */
	void LogSearchDiagnostics(double StartTime, bool bInitialScan);
	/** Look for and load a single AssetData result from the gatherer. */
	void TickGatherPackage(UE::AssetRegistry::FInterfaceWriteScopeLock& ScopeLock, Impl::FEventContext& EventContext, const FString& PackageName, const FString& LocalPath);
	void ClearGathererCache();
#if WITH_EDITOR
	void AssetsSaved(UE::AssetRegistry::Impl::FEventContext& EventContext, TArray<FAssetData>&& Assets);
	void GetProcessLoadedAssetsBatch(TQueue<TWeakObjectPtr<const UObject>, EQueueMode::Mpsc>& LoadedAssetsToProcess,
		TArray<const UObject*>& OutLoadedAssets, uint32 BatchSize, bool bUpdateDiskCacheAfterLoad);
	void PushProcessLoadedAssetsBatch(Impl::FEventContext& EventContext,
		TQueue<TWeakObjectPtr<const UObject>, EQueueMode::Mpsc>& LoadedAssetsToProcess,
		TArrayView<FAssetData> LoadedAssetDatas, TArrayView<const UObject*> UnprocessedFromBatch);
	bool IsLoadedAndTrackedAsset(const FSoftObjectPath& InObjectPath) const;
	/** Call LoadCalculatedDependencies on each Package updated after the last LoadCalculatedDependencies. */
	void LoadCalculatedDependencies(TArray<FName>* AssetPackageNamessToCalculate, Impl::FClassInheritanceContext& InheritanceContext, 
		TSet<FName>* InPackagesNeedingDependencyCalculation, Impl::FInterruptionContext& InOutInterruptionContext);
	/** For each package in BackgroundPackages, if there is a registered gatherer for its asset class, this function will 
	 *  move the package to GameThreadPackages otherwise it will be removed from the set. Any entries in GameThreadPackages that
	 *  do not have a corresponding registered gatherer will likewise be removed. The end condition for this function is that 
	 *  BackgroundPackages is empty and GameThreadPackages contains only those packages which require LoadCalculatedDependencies to
	 *  be called on them
	 */
	void PruneAndCoalescePackagesRequiringDependencyCalculation(TSet<FName>& BackgroundPackages, TSet<FName>& GameThreadPackages,
		Impl::FInterruptionContext& InOutInterruptionContext);

	/** Look for a CalculatedDependencies function registered for the asset(s) in the given package
	 *  and call that function to add calculated dependencies. Calculated dependencies are added only after
	 *  all normal dependencies gathered from the AssetRegistry data stored in the package have been loaded.
	 */
	void LoadCalculatedDependencies(FName PackageName, Impl::FClassInheritanceContext& InheritanceContext,
		bool& bOutHadActivity);
	/** Add a watch on a directory that to modify data for a package whenever packages in the directory are modified. */
	void AddDirectoryReferencer(FName PackageName, const FString& DirectoryLocalPathOrLongPackageName);
	/** Set the directories watched by PackageName. Removes previous list, call with empty array to clear. */
	void SetDirectoriesWatchedByPackage(FName PackageName, TConstArrayView<FString> Directories);

	/** Called when new gatherer is registered. Requires subsequent call to RebuildAssetDependencyGathererMapIfNeeded */
	void OnAssetDependencyGathererRegistered() { bRegisteredDependencyGathererClassesDirty = true; }

	void ApplyObjectRedirectsToGatheredAssetDatas();
#endif

	/** Adds an asset to the empty package list which contains packages that have no assets left in them */
	void AddEmptyPackage(FName PackageName);
	/** Removes an asset from the empty package list because it is no longer empty */
	bool RemoveEmptyPackage(FName PackageName);
	/** Adds a path to the cached paths tree. Returns true if the path was added and did not already exist */
	bool AddAssetPath(Impl::FEventContext& EventContext, FName PathToAdd);
	/** Removes a path to the cached paths tree. Returns true if successful. */
	bool RemoveAssetPath(Impl::FEventContext& EventContext, FName PathToRemove, bool bEvenIfAssetsStillExist = false);
	/** Removes the asset data associated with this package from the look-up maps */
	void RemovePackageData(Impl::FEventContext& EventContext, const FName PackageName);
	/** Adds the Verse file to the look up maps */
	void AddVerseFile(Impl::FEventContext& EventContext, FName VerseFilePathToAdd);
	/** Removes the Verse file from the look-up maps */
	void RemoveVerseFile(Impl::FEventContext& EventContext, FName VerseFilePathToRemove);

	/** Returns the names of all subclasses of the class whose name is ClassName */
	void GetSubClasses(Impl::FClassInheritanceContext& InheritanceContext, const TArray<FTopLevelAssetPath>& InClassNames,
		const TSet<FTopLevelAssetPath>& ExcludedClassNames, TSet<FTopLevelAssetPath>& SubClassNames) const;

	bool IsInitialSearchCompleted() const { return bInitialSearchCompleted.load(std::memory_order_relaxed); }
	bool IsTempCachingEnabled() const { return bIsTempCachingEnabled; }
	bool IsTempCachingAlwaysEnabled() const { return bIsTempCachingAlwaysEnabled; }
	bool IsInitialSearchStarted() const { return bInitialSearchStarted; }
	bool IsAdditionalMountSearchInProgress() const { return bAdditionalMountSearchInProgress.load(std::memory_order_relaxed); }
	bool IsSearchAllAssets() const { return bSearchAllAssets; }
	Impl::FClassInheritanceBuffer& GetTempCachedInheritanceBuffer() { return TempCachedInheritanceBuffer; }
	uint64 GetSavedGeneratorClassesVersionNumber() const { return SavedGeneratorClassesVersionNumber; }
	uint64 GetSavedAllClassesVersionNumber() const { return SavedAllClassesVersionNumber; }
	static uint64 GetCurrentGeneratorClassesVersionNumber();
	static uint64 GetCurrentAllClassesVersionNumber();

	/** Get a copy of the cached serialization options that were parsed from ini */
	void CopySerializationOptions(FAssetRegistrySerializationOptions& OutOptions, ESerializationTarget Target) const;
	
	/** Query the performance mode, which modifies how data structures are loaded */
	Impl::EPerformanceMode GetPerformanceMode() const { return PerformanceMode; }
	void SetPerformanceMode(Impl::EPerformanceMode NewMode);
	bool ShouldSortDependencies() const;
	bool ShouldSortReferencers() const;

	const FAssetRegistryState& GetState() const;
	const FPathTree& GetCachedPathTree() const;
	const TSet<FName>& GetCachedEmptyPackages() const;

	/** Same as UE::AssetRegistry::Filtering::ShouldSkipAsset, but can be read from any thread under readlock */
	bool ShouldSkipAsset(FTopLevelAssetPath AssetClass, uint32 PackageFlags) const;
	bool ShouldSkipAsset(const UObject* InAsset) const;

	/** Finds all class names of classes capable of generating new UClasses */
	void CollectCodeGeneratorClasses();

	/**
	 * Block until the Premade AssetRegistry finishes loading, if there is one still loading.
	 * For performance reasons, this must be called within the WriteScopeLock, and the caller 
	 * must handle the possibility that it leaves and reenters the lock.
	 */
	void ConditionalLoadPremadeAssetRegistry(UE::AssetRegistry::FInterfaceWriteScopeLock& ScopeLock,
		UE::AssetRegistry::Impl::FEventContext& EventContext);

	/** Unload specified MountPoints, helper for OnMultipleMountPointsDismounted and others. */
	void UnloadMountPoints(Impl::FEventContext& EventContext,
		TConstArrayView<TRefCountPtr<UE::PackageName::IMountPoint>> MountPoints);

#if WITH_EDITOR
	/** Request to pause or resume background processing of scan results.
	 *  This can be used to allow a priority thread to perform along sequence of operations
	 *  without having to contend with the background thread for data access
	 */
	void RequestPauseBackgroundProcessing();
	void RequestResumeBackgroundProcessing();
	bool IsBackgroundProcessingPaused() const;
	uint32& GetBackgroundTickInterruptionsCount();

#endif
private:

	/** The delegate to execute when an asset collision is detected */
	IAssetRegistry::FAssetCollisionEvent AssetCollisionEvent;

	/** Helper method for public implementations which vary based on lock type */
	template<typename TScopeLock>
	FSoftObjectPath GetRedirectedObjectPath(TScopeLock& ScopeLock, const FSoftObjectPath& ObjectPath, UE::AssetRegistry::Impl::FEventContext* EventContext,
		UE::AssetRegistry::Impl::FClassInheritanceContext* InheritanceContext);
	/**
	 * Enumerate all AssetData in memory and in the State that match the given path.
	 * The returned AssetData for objects present in memory have no AssetRegistryTags calculated
	 */
	void EnumerateAssetsByPathNoTags(FName PackagePath,
		TFunctionRef<bool(const FAssetData&)> Callback, bool bRecursive, bool bIncludeOnlyOnDiskAssets) const;
	/** Create redirectors read by this->GetRedirectedObjectPath, based on plugin settings */
	void InitRedirectors(Impl::FEventContext& EventContext, Impl::FClassInheritanceContext& InheritanceContext,
		bool& bOutRedirectorsNeedSubscribe);
	/** If collecting dependencies, create an FAssetPackageData for every script package, to make sure dependencies can find a data for them. */
	void ReadScriptPackages();
	/* Try to construct the gatherer if it does not already exist. If that is not possible (e.g., because we are shutting down)
	 * return false. If it was successfully created or already existed, return true.
	 */
	bool TryConstructGathererIfNeeded();
	/**  Called to set up timing variables and launch the on-constructor SearchAllAssets async call */
	void SearchAllAssetsInitialAsync(Impl::FEventContext& EventContext, Impl::FClassInheritanceContext& InheritanceContext);
	/**
	 * Called every tick to when data is retrieved by the background asset search.
	 * If TickStartTime is < 0, the entire list of gathered assets will be cached. Also used in sychronous searches
	 * The DeferredResults array receives assets that were not ready for processing due to required UClass's not being loaded.
	 */
	void AssetSearchDataGathered(Impl::FEventContext& EventContext,
		TMultiMap<FName, TUniquePtr<FAssetData>>& AssetResults, 
		TMultiMap<FName, TUniquePtr<FAssetData>>& OutDeferredResults,
		Impl::FInterruptionContext& InOutInterruptionContext,
		TOptional<TSet<FString>>& MountPointsForVerifyAfterGather,
		EDeferFlag DeferFlag);
	/** Validate assets gathered from disk before adding them to the AssetRegistry. */
	bool ShouldSkipGatheredAsset(FAssetData& AssetData);

	/**
	 * Called every tick when data is retrieved by the background path search.
	 * If TickStartTime is < 0, the entire list of gathered assets will be cached. Also used in sychronous searches
	 */
	void PathDataGathered(Impl::FEventContext& EventContext, TRingBuffer<FString>& PathResults,
		Impl::FInterruptionContext& InOutInterruptionContext,
		TOptional<TSet<FString>>& MountPointsForVerifyAfterGather);

	/** Called every tick when data is retrieved by the background dependency search */
	void DependencyDataGathered(TMultiMap<FName, FPackageDependencyData>& DependsResults, 
		TMultiMap<FName, FPackageDependencyData>& OutDeferredDependencyResults,
		TSet<FName>* OutPackagesNeedingDependencyCalculation, Impl::FInterruptionContext& InOutInterruptionContext,
		TOptional<TSet<FString>>& MountPointsForVerifyAfterGather);

	/** Called every tick when data is retrieved by the background search for cooked packages that do not contain asset data */
	void CookedPackageNamesWithoutAssetDataGathered(Impl::FEventContext& EventContext, 
		TRingBuffer<FString>& CookedPackageNamesWithoutAssetDataResults, Impl::FInterruptionContext& InOutInterruptionContext);

	/** Called every tick when data is retrieved by the background dependency search */
	void VerseFilesGathered(Impl::FEventContext& EventContext, TRingBuffer<FName>& VerseResults, Impl::FInterruptionContext& InOutInterruptionContext);

	/** Adds the asset data to the lookup maps */
	void AddAssetData(Impl::FEventContext& EventContext, FAssetData* AssetData);

	/** Updates an existing asset data with the new value and updates lookup maps */
	void UpdateAssetData(Impl::FEventContext& EventContext, FAssetData* AssetData, FAssetData&& NewAssetData, bool bKeepDeletedTags);

	/** Updates the tags on an existing asset data by adding the tags from NewAssetData that do not already exist. */
	void AddNonOverlappingTags(Impl::FEventContext& EventContext, FAssetData& ExistingAssetData,
		const FAssetData& NewAssetData);

	/** Removes the asset data from the lookup maps. Returns true if it was removed, false if it already did not exist. */
	bool RemoveAssetData(Impl::FEventContext& EventContext, FAssetData* AssetData);

	/** Removes a set of asset data from the lookup maps. Returns true if any were removed, false if all already did not exist. */
	bool RemoveAssetDatas(Impl::FEventContext& EventContext, TArrayView<FAssetData*> AssetDatas);

#if WITH_EDITOR
	/** Called when two assets have the same ID but different packages. Logs it and returns which one to keep. */
	FAssetData* ResolveAssetIdCollision(FAssetData& A, FAssetData& B);
	/**
	 * Calls PostLoadAssetRegistryTags on the CDO of the asset class this data represents.
	 * @param AssetData Existing asset data
	 * @param OutRequestedGameThreadClass If the postload fails because a class needs processing on the gamethread, this
	 *        variable receives the classname. Otherwise it is empty.
	 * @return Returns false if required UClass data is not yet available and we are not forced to complete.
	 */
	bool TryPostLoadAssetRegistryTags(FAssetData* AssetData, EDeferFlag DeferFlag,
		FTopLevelAssetPath& OutRequestedGameThreadClass);

	/** If new gatherers were registered, this call will refresh the mapping */
	void RebuildAssetDependencyGathererMapIfNeeded();

	/** When the scan is completed, resolve the redirections in CachedBPInheritanceMap. */
	void ResolveRedirectionsInCachedBPInheritanceMap();

#endif // WITH_EDITOR
	void GetSubClasses_Recursive(Impl::FClassInheritanceContext& InheritanceContext, FTopLevelAssetPath InClassName,
		TSet<FTopLevelAssetPath>& SubClassNames, TSet<FTopLevelAssetPath>& ProcessedClassNames, const TSet<FTopLevelAssetPath>& ExcludedClassNames) const;

	/** Internal helper which processes a given state and adds its contents to the current registry */
	void CachePathsFromState(Impl::FEventContext& EventContext, const FAssetRegistryState& InState,
		const FAssetRegistrySerializationOptions* Options = nullptr);

	/** Adds ClassPath-> ParentPath to CachedBPInheritanceMap. Applies CoreRedirects to NotYetRedirectedParentPath. */
	void AddCachedBPClassParent(const FTopLevelAssetPath& ClassPath, const FTopLevelAssetPath& NotYetRedirectedParentPath);

	/** Updates OutBuffer from loaded classes and registered-in-CachedBPInheritanceMap blueprint classes */
	void UpdateInheritanceBuffer(Impl::FClassInheritanceBuffer& OutBuffer) const;

	/** Conditionally loads the Premade AssetRegistry from disk, or queues it for asynchronous loading if not yet loaded. */
	void ConsumeOrDeferPreloadedPremade(UAssetRegistryImpl& UARI, Impl::FEventContext& EventContext,
		Impl::FActiveMountEvents& ActiveMountEvents, FInterfaceWriteScopeLock& ProofOfWriteLock);
	/** Moves a premade asset registry state into this AR */
	void LoadPremadeAssetRegistry(UAssetRegistryImpl& UARI, Impl::FEventContext& Context,
		Impl::FActiveMountEvents& ActiveMountEvents, FInterfaceWriteScopeLock& ProofOfWriteLock,
		Premade::ELoadResult LoadResult, Impl::FAssetProviderInitData&& StateInitData);
	/** Add MountPoints of all AssetDatas currently registered in this->State to the list of PersistentMountPoints. */
	void UpdatePersistentMountPoints();
	void OnInitialSearchCompleted(Impl::FEventContext& EventContext);
	void OnAdditionalMountSearchCompleted(Impl::FEventContext& EventContext);

	/** This function is not called and, currently, will always return 'true'. See the commented out call in
	 *	FAssetDataGatherer::TickInternal for further details on how it could be used. 
	 */
	friend FAssetDataGatherer;
	bool ClassRequiresGameThreadProcessing(const UClass* Class) const;

	friend UAssetRegistryImpl;
	/** This exists purely for use during shutdown to enable the UAssetRegistryImpl to
	 *  avoid waiting for the gatherer to terminate while holding the interface lock.
	 *  See UAssetRegistryImpl::OnEnginePreExit 
	 */
	TUniquePtr<FAssetDataGatherer>& AccessGlobalGatherer() { return GlobalGatherer; }

	void UpdateMaxSecondsPerFrame();

	/** Free slack memory accumulated by the asset registry. */
	void Shrink();
private:

	/** Internal state of the cached asset registry */
	FAssetRegistryState State;

	/** Database of known Verse files */
	TSet<FName> CachedVerseFiles;
	TMap<FName, TArray<FName>> CachedVerseFilesByPath;

	/** Default options used for serialization */
	FAssetRegistrySerializationOptions SerializationOptions;
	FAssetRegistrySerializationOptions DevelopmentSerializationOptions;

	/** The set of empty package names (packages which contain no assets but have not yet been saved) */
	TSet<FName> CachedEmptyPackages;

	/** The map of classes to their parents, only full for offline blueprints */
	TMap<FTopLevelAssetPath, FTopLevelAssetPath> CachedBPInheritanceMap;

	/** Helper container to resolve filter rules */
	Impl::FFilterTagRules CachedParsedFilterRules;

	/** If true, search caching is enabled */
	bool bIsTempCachingEnabled;

	/** If true, search caching is enabled permanently */
	bool bIsTempCachingAlwaysEnabled;

	/** Persistent InheritanceBuffer used when SetTemporaryCachingMode is on */
	Impl::FClassInheritanceBuffer TempCachedInheritanceBuffer;

	uint64 SavedGeneratorClassesVersionNumber;
	uint64 SavedAllClassesVersionNumber;

	/** The tree of known cached paths that assets may reside within */
	FPathTree CachedPathTree;

	/** Async task that gathers asset information from disk */
	TUniquePtr<FAssetDataGatherer> GlobalGatherer;

	/** Lists of results from the gatherer thread that are waiting to get processed */
	UE::AssetDataGather::FResults BackgroundResults;

	/** Assets and dependencies that are not ready for processing because they can't yet run PostLoadAssetRegistrytags */
	TMultiMap<FName, TUniquePtr<FAssetData>> DeferredAssets;
	TMultiMap<FName, TUniquePtr<FAssetData>> DeferredAssetsForGameThread;
	TMultiMap<FName, FPackageDependencyData> DeferredDependencies;
	TMultiMap<FName, FPackageDependencyData> DeferredDependenciesForGameThread;
	/** Classes that some of the AssetsForGameThread need processed on the gamethread before a retry. */
	TSet<FTopLevelAssetPath> GameThreadRequestedClasses;
	FCriticalSection GameThreadRequestedClassesLock;
	/** Sentry values read by gamethread and gatherthread to performantly coordinate on GameThreadRequestedClasses. */
	std::atomic<bool> bGameThreadRequestedClassesShouldRetry{ false };
	std::atomic<bool> bGameThreadRequestedClassesShouldProcess{ false };

#if !NO_LOGGING
	/** Memory profiling information: How much memory is being used by the tags for each class. */
	TMap<FTopLevelAssetPath, int64> TagSizeByClass;
#endif

	/**
	 * MountPoints, in the format of LongPackageName with no trailing slash, that should not have their
	 * AssetDatas removed even if the MountPoint is dismounted.
	 */
	TSet<FName> PersistentMountPoints;

	/** Time spent processing Gather results */
	float StoreGatherResultsTimeSeconds;
	/** The highest number of pending results observed during initial gathering */
	int32 HighestPending = 0;

	/** Time the initial async search was started */
	double InitialSearchStartTime = 0.0f;
	/** Time the additional mount async search was started */
	double AdditionalMountSearchStartTime = 0.0f;
	/** Flag to indicate if we used an initial async search */
	bool bInitialSearchStarted = false;
	/** Flag to indicate Initialize() has run to completion */
	bool bInitializationComplete = false;
	/** Flag to indicate if the initial background search has completed. All access are relaxed because
	 *  the actual search data can only be accessed under a proper lock.
	 */
	std::atomic<bool> bInitialSearchCompleted;
	/** Flag to indicate if an additional mount background search has started after the initial search.
	 * All access are relaxed because the actual search data can only be accessed under a proper lock.
	 */
	std::atomic<bool> bAdditionalMountSearchInProgress;
	/**
	 * Flag to indicate PreloadingComplete; finishing the background search is blocked until preloading complete
	 * because preloading can add assets.
	 */
	bool bPreloadingComplete = false;
	/** Status of the background search, so we can take actions when it changes to or from idle */
	Impl::EGatherStatus GatherStatus;
	/** What kind of performance mode this is in, used to optimize for initial loading vs runtime */
	Impl::EPerformanceMode PerformanceMode;

	/**
	 * Enables extra check to make sure path still mounted before adding.
	 * Removing mount point can happen between scan (background thread + multiple ticks and the add).
	 */
	bool bVerifyMountPointAfterGather;

	/** Record whether SearchAllAssets has been called; if so we will also search new mountpoints when added */
	bool bSearchAllAssets;

	bool bVerboseLogging;

	bool bForceCompletionEvenIfClassNotLoaded = false;
	bool bForceCompletionEvenIfNotOnGameThread = false;
	bool bProcessedAnyAssetsAfterRetryDeferred = true;
	float MaxSecondsPerFrame = 0.04f;

	/** List of all class names derived from Blueprint (including Blueprint itself) */
	TSet<FTopLevelAssetPath> ClassGeneratorNames;

	struct FAssetRegistryPackageRedirect
	{
	public:
		FAssetRegistryPackageRedirect(const FString& InSourcePackageName, const FString& InDestPackageName)
			: SourcePackageName(InSourcePackageName), DestPackageName(InDestPackageName)
		{
		}
		FString SourcePackageName;
		FString DestPackageName;
	};
	TArray<FAssetRegistryPackageRedirect> PackageRedirects;

#if WITH_EDITOR
	/**
	 * The set of object paths that have had their dependencies gathered since the last idle,
	 * and that need to check for calculated dependencies at the next idle.
	 */
	TSet<FName> PackagesNeedingDependencyCalculation;
	TSet<FName> PackagesNeedingDependencyCalculationOnGameThread;

	/** The set of object paths that have had their disk cache updated from the in memory version */
	TSet<FSoftObjectPath> AssetDataObjectPathsUpdatedOnLoad;

	/**
	 * A bidirectional map of which directories are watched by which packages for CalculatedDependencies. When a
	 * directory updates we need to rerun CalculatedDependencies for every package watching the directory. When a
	 * package runs CalculatedDependencies, we need to clear all of its watched files.
	 */
	TMap<FString, TSet<FName>> PackagesWatchingDirectory;
	TMap<FName, TArray<FString>> DirectoriesWatchedByPackage;

	/**
	 * Number of times during the startup scan that TickOnBackgroundThread was interrupted by a request for the
	 * AssetRegistry's lock from another thread.
	 */
	uint32 BackgroundTickInterruptionsCount = 0;
	std::atomic<bool> bGameThreadTakeOverGatherEachTick;

	/** A map of per asset class dependency gatherer called in LoadCalculatedDependencies */
	TMultiMap<FTopLevelAssetPath, UE::AssetDependencyGatherer::Private::FRegisteredAssetDependencyGatherer*> RegisteredDependencyGathererClasses;
	mutable FRWLock RegisteredDependencyGathererClassesLock;
	bool bRegisteredDependencyGathererClassesDirty;

	bool bGatherNeedsApplyObjectRedirects = false;
#endif
#if WITH_ENGINE && WITH_EDITOR
	mutable FRWLock SkipClassesLock;
	/** Class names that return true for IsAsset but which should not be treated as assets in uncooked packages */
	TSet<FTopLevelAssetPath> SkipUncookedClasses;
	/** Class names that return true for IsAsset but which should not be treated as assets in cooked packages */
	TSet<FTopLevelAssetPath> SkipCookedClasses;
#endif
	UE::AssetRegistry::Premade::FAsyncConsumer AsyncConsumer;
	friend UE::AssetRegistry::Premade::FAsyncConsumer;
	friend UE::AssetRegistry::Impl::FActiveMountsLoader;
	friend UE::AssetRegistry::Impl::FClassInheritanceContext;
	friend UE::AssetRegistry::Impl::FTickContext;

	TUniquePtr<UE::AssetRegistry::DependsNode::FGatherDependencyDataScratchpad> GatherDependencyDataScratchPad = nullptr;
};

namespace Impl
{

/**
 * Stores events that need to be broadcasted from the AssetRegistry; these events are added to 
 * by functions within a lock, and are broadcast after the lock is dropped. The broadcast is also deferred
 * until on the GameThread.
 */
struct FEventContext
{
	enum class EEvent : uint32
	{
		Added,
		Removed,
		Updated,
		UpdatedOnDisk,
		
		MAX
	};
	TOptional<IAssetRegistry::FFileLoadProgressUpdateData> ProgressUpdateData;
	TArray<TPair<FString, EEvent>> PathEvents;
	TArray<TPair<FAssetData, EEvent>> AssetEvents;
	TArray<TPair<FName, EEvent>> VerseEvents;
	TArray<FString> RequiredLoads;
	TArray<FString> BlockedFiles;
	bool bFileLoadedEventBroadcast = false;
	bool bScanStartedEventBroadcast = false;
	bool bHasSentFileLoadedEventBroadcast = false;
	bool bKnownGathersCompleteEventBroadcast = false;

	/** Remove all stored events */
	void Clear();
	/** Report whether this context has any stored events */
	bool IsEmpty() const;
	/** Add all events from other onto this context's collection of events */
	void Append(FEventContext&& Other);
};


/*
 * An accessor for the inheritance map and related data for functions that need them; the source
 * of this data is an internal detail unneeded by the functions reading it. Readers of the data call
 * ConditionalUpdate to ensure it is up to date before reading it.
 */
struct FClassInheritanceContext
{
	/** The Buffer providing the data for this context. */
	FClassInheritanceBuffer* Buffer = nullptr;
	/** The stack allocated Buffer which may be used to provide data for this context. */
	FClassInheritanceBuffer* StackBuffer = nullptr;
	/** Back pointer to the AssetRegistryImpl to update the Buffer if necessary */
	FAssetRegistryImpl* AssetRegistryImpl = nullptr;
	/** Whether ConditionalUpdate needs to update inheritance data */
	bool bInheritanceMapUpToDate = false;
	/** Whether ConditionalUpdate needs to update code generate classes before it can update inheritance data */
	bool bCodeGeneratorClassesUpToDate = false;

	/** Set the context to use the data from the given buffer */
	void BindToBuffer(FClassInheritanceBuffer& InBuffer, FAssetRegistryImpl& InAssetRegistryImpl,
		bool bInInheritanceMapUpToDate, bool bInCodeGeneratorClassesUpToDate);
	/** Update the backing buffer if it is out of date */
	void ConditionalUpdate();
	/** Selects and binds a FClassInheritanceBuffer buffer to be used by this context. */
	void SelectBuffer(FAssetRegistryImpl& GuardedData, uint64 CurrentGeneratorClassesVersionNumber, uint64 CurrentAllClassesVersionNumber);
	/** Updates the context after the InterfaceLock has been unlocked and locked while the context is in use. */
	void OnLockReentered(FAssetRegistryImpl& GuardedData);
};

/** Context to transform, collect, and pass along ScanPathsSynchronous arguments and results */
struct FScanPathContext
{
	FScanPathContext(FEventContext& InEventContext, FClassInheritanceContext& InInheritanceContext,
		const TArray<FString>& InDirs, const TArray<FString>& InFiles,
		UE::AssetRegistry::EScanFlags InScanFlags = UE::AssetRegistry::EScanFlags::None,
		TArray<FSoftObjectPath>* FoundAssets = nullptr);

	TArray<FString> PackageDirs;
	TArray<FString> LocalDirs;
	TSet<FString> PackageFiles;
	TArray<FString> LocalFiles;
	TArray<FString> LocalPaths;
	FEventContext& EventContext;
	FClassInheritanceContext& InheritanceContext;
	TArray<FSoftObjectPath>* OutFoundAssets = nullptr;
	int32 NumFoundAssets = 0;
	bool bForceRescan = false;
	bool bIgnoreDenyListScanFilters = false;
	bool bIgnoreInvalidPathWarning = false;
	EGatherStatus Status = EGatherStatus::TickActiveGatherActive;
};

/** Input and output variables for the TickGatherer function. */
struct FTickContext
{
public:
	FTickContext(FAssetRegistryImpl& InGuardedData, Impl::FEventContext& InEventContext,
		Impl::FClassInheritanceContext& InInheritanceContext);

	/** Start the timer to measure time spent in the current call to TickGatherer, if not already started. */
	void LazyStartTimer();
	/**
	 * Record the time that was spent in the current call to TickGatherer onto the metric for it on GuardedData, if
	 * LazyStartTimer has been called and RecordTimer has not already been called.
	 */
	void RecordTimer();

	/**
	 * Return progress-bar metric for how many elements remain in the various lists of work to do. A portion of this
	 * calculation is cached (the GetNumGatherFromDiskPending) and it will not get the most up to date data until
	 * SetNumGatherFromDiskPendingDirty is called.
	 */
	int32 GetNumPending();
	/**
	 * Return the portion of GetNumPending that is handled during gathering from disk; does not include the
	 * calculations that we do once the gather from disk is done. This function is cached, it does not redo the
	 * calculation until SetNumPendingDirty is called.
	 */
	int32 GetNumGatherFromDiskPending();
	/** Invalidate the cache in GetNumGatherFromDiskPending  */
	void SetNumGatherFromDiskPendingDirty();
	/** Run the expression to calculate the value used in GetNumGatherFromDiskPending. */
	int32 CalculateNumGatherFromDiskPending();

	/**
	 * Store the result of TickGatherer in the AssetRegistry's status variable and send the progress event.
	 * Returns the input Status so it can be used in a return line.
	 */
	EGatherStatus SetIntermediateStatus(EGatherStatus Status);
	/**
	 * Calculate the result of TickGatherer from the current gather data on the TickContext and call
	 * SetIntermediateStatus. Returns the input Status so it can be used in a return line.
	 */
	EGatherStatus UpdateIntermediateStatus();


	void RunAssetSearchDataGathered(TMultiMap<FName, TUniquePtr<FAssetData>>& InAssetResults,
		TMultiMap<FName, TUniquePtr<FAssetData>>& OutDeferredAssetResults);
	void RunDependencyDataGathered(TMultiMap<FName, FPackageDependencyData>& DependenciesToProcess,
		TMultiMap<FName, FPackageDependencyData>& OutDeferredDependencies,
		TSet<FName>* OutPackagesNeedingDependencyCalculation);

public:
	UE::AssetDataGather::FResultContext ResultContext;
	Impl::FInterruptionContext InterruptionContext;
	FAssetRegistryImpl& GuardedData;
	Impl::FEventContext& EventContext;
	Impl::FClassInheritanceContext& InheritanceContext;
	TOptional<TSet<FString>> MountPointsForVerifyAfterGather;
	double TimingStartTime = -1.;
	int32 NumGatherFromDiskPending = 0;
	bool bHandleCompletion = false;
	bool bHandleDeferred = false;
	bool bNumGatherFromDiskPendingDirty = true;
	bool bHadAssetsToProcess = false;
	bool bIsInGameThread = false;
};

} // namespace Impl within namespace UE::AssetRegistry

namespace Utils
{

/** Different modes for RunAssetThroughFilter and related filter functions */
enum class EFilterMode : uint8
{
	/** Include things that pass the filter; include everything if the filter is empty */
	Inclusive,
	/** Exclude things that pass the filter; exclude nothing if the filter is empty */
	Exclusive,
};

/** Report whether the external-code-created filter is in a state that is safe to use in filtering */
bool IsFilterValid(const FARCompiledFilter& Filter);
/** Report whether the given AssetData passes/fails the given filter */
bool RunAssetThroughFilter(const FAssetData& AssetData, const FARCompiledFilter& Filter, const EFilterMode FilterMode);
/** Helper for RunAssetThroughFilter and RunAssetsThroughFilter that skips validity check */
bool RunAssetThroughFilter_Unchecked(const FAssetData& AssetData, const FARCompiledFilter& Filter, const bool bPassFilterValue);
/**
 * Given an array of asset data, trim the items that fail the filter based on the inclusion/exclusion mode used.
 *  - In inclusive mode it will remove all assets that fail the filter, and in exclusive mode it will remove all assets that pass the filter.
 *  - If the filter is empty, then the array will be untouched.
 */
void RunAssetsThroughFilter(TArray<FAssetData>& AssetDataList, const FARCompiledFilter& Filter, const EFilterMode FilterMode);

/** 
 * This will always read the ini, public version may return cache.
 * if OutRules is nullptr, no tags will be filtered.
 */
void InitializeSerializationOptionsFromIni(
	FAssetRegistrySerializationOptions& Options,
	Impl::FFilterTagRules* OutRules,
	const ITargetPlatform* TargetPlatform = nullptr,
	ESerializationTarget Target = ESerializationTarget::ForGame,
	bool bSuppressWarnings = false);

/**
 * Returns the status of Engine module existance.
 * Used to suppress emitting warnings about missing Engine classes.
 */
bool HasEngineModuleLoaded();

/**
 * Updates FAssetRegistrySerializationOptions filters with the information based on runtime classes.
 * Options are parsed based on class name and need to be resolved using the class hierarchy,
 * and since the class hierarchy can change during startup we may need to reexecute the resolve.
 */
void UpdateSerializationOptions(
	FAssetRegistrySerializationOptions& InOutOptions,
	Impl::FFilterTagRules& InOutFilterRules,
	bool bSuppressWarnings);

/**
 * Parses the list of CookedTagsBlacklist or CookedTagsWhitelist gathered from the INI.
 * Returns a mapping from a tag to a list of rules. May contain unresolved rules.
 */
TMap<FName, Impl::FParsedFilterTagRules> ParseFilterTags(const TArray<FString>& FilterListItems, bool bIsFilteringDevelopmentAR, bool bUseAllowListInsteadOfDenyList);

/**
 * Resolves and expands the list of rules to list of all derived loaded classes that satisfy those rules.
 * Then transposes rules to a map with a class as a key and a list of tags as value.
 */
TMap<FTopLevelAssetPath, TSet<FName>> ResolveFilterTags(Impl::FFilterTagRules& InOutRules, bool bSuppressWarnings);

/** Gets the current availability of an asset, primarily for streaming install purposes. */
EAssetAvailability::Type GetAssetAvailability(const FAssetData& AssetData);
/** Gets an ETA or percentage complete for an asset that is still in the process of being installed. */
float GetAssetAvailabilityProgress(const FAssetData& AssetData, EAssetAvailabilityProgressReportingType::Type ReportType);
/** Returns if a given report type is supported on the current platform */
bool GetAssetAvailabilityProgressTypeSupported(EAssetAvailabilityProgressReportingType::Type ReportType);
/**  Hint the streaming installers to prioritize a specific asset for install. */
void PrioritizeAssetInstall(const FAssetData& AssetData);

/**
 * Reads Asset file from a previously initialized package reader
 *
 * @param PackageReader	Previously initialized package reader that maps to the data to be read
 * @param AssetDataList	List of the read asset data
 */
bool ReadAssetFile(FPackageReader& PackageReader, IAssetRegistry::FLoadPackageRegistryData& InOutData);

/** If InOutMountPoints is not already set, sets it and reads mount points into it from FPackageName. */
void InitializeMountPoints(TOptional<TSet<FString>>& InOutMountPoints);

/**
 * Returns true if path belongs to one of the mount points provided
 *
 * @param	Path				Path to check if mounted, example "/MyPlugin/SomeAsset"
 * @param	MountPointsNoTrailingSlashes		Mount points without the trailing slash. Example: "/MyPlugin"
 * @param	StringBuffer		String buffer to avoid re-allocation performance hit when searching TSet
 */
bool IsPathMounted(const FString& Path, const TSet<FString>& MountPointsNoTrailingSlashes, FString& StringBuffer);

/**
 * Iterate all UObjects where this->IsAsset and !ShouldSkipAsset, create an FAssetData for them and call the callback.
 * Stores packagenames of all assets found in OutPackageNamesWithAssets for later filtering of on-disk assets.
 * Stops iteration and sets bOutStopIteration=true iff the callback returns false.
 */
void EnumerateAllMemoryAssets(TSet<FName>& OutPackageNamesWithAssets, bool& bOutStopIteration, TFunctionRef<bool(FAssetData&&)> Callback);
/**
 * Helper for EnumerateMemoryAssets* functions. Calls the given callback on all memory assets that pass the portion of the
 * given filter that does not require tags, and passes in an FAssetData constructed from the UObject but missing tags.
 * Fills in OutPackageNamesWithAssets with names of all packages tested.
 */
void EnumerateMemoryAssetsHelper(const FARCompiledFilter& InFilter, TSet<FName>& OutPackageNamesWithAssets,
	bool& bOutStopIteration, TFunctionRef<bool(const UObject* Object, FAssetData&& PartialAssetData)> Callback,
	bool bSkipARFilteredAssets);
/**
 * Call the given callback on all UObjects in memory that pass the given filter.
 * Fills in OutPackageNamesWithAssets with names of all packages tested.
*/
void EnumerateMemoryAssets(const FARCompiledFilter& InFilter, TSet<FName>& OutPackageNamesWithAssets,
	bool& bOutStopIteration, UE::AssetRegistry::Private::FInterfaceRWLock& InterfaceLock, const FAssetRegistryState& GuardedDataState,
	TFunctionRef<bool(FAssetData&&)> Callback, bool bSkipARFilteredAssets);

} // namespace Utils within namespace UE::AssetRegistry

// Normalization functions for paths. We require that normalized paths are case-insensitively string-identical for the same input
// path specification. e.g. all slashes must be replaced with forward-slashes. Special case: a leading \\ is left unreplaced,
// so //share will be seen as different than \\share.
FString CreateStandardFilename(FStringView InPath);
FString CreateStandardFilename(const FString& InFilename);
// Normalization for LongPackageNames that identify a directory. Ensures it ends with a trailing forward-slash.
FString CreateStandardLongPackageNameDirectory(FStringView LongPackageName);
// Normalization for LongPackageNames that should not end in a trailing slash.
FName CreateStandardLongPackageFNameNoTrailingSlash(FStringView LongPackageName);

} // namespace UE::AssetRegistry


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////


namespace UE::AssetRegistry
{

#if WITH_EDITOR
inline uint32& FAssetRegistryImpl::GetBackgroundTickInterruptionsCount()
{
	return BackgroundTickInterruptionsCount;
}

inline bool FAssetRegistryImpl::IsGameThreadTakeOverGatherEachTick() const
{
	return bGameThreadTakeOverGatherEachTick.load(std::memory_order_relaxed);
}

inline void FAssetRegistryImpl::SetGameThreadTakeOverGatherEachTick(bool bValue)
{
	bGameThreadTakeOverGatherEachTick.store(bValue, std::memory_order_relaxed);
}
#endif

} // namespace UE::AssetRegistry
