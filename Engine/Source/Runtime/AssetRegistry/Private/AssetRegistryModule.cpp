// Copyright Epic Games, Inc. All Rights Reserved.


#include "AssetRegistry/AssetRegistryModule.h"

#include "AssetRegistry.h"
#include "AssetRegistryConsoleCommands.h"

IMPLEMENT_MODULE( FAssetRegistryModule, AssetRegistry );

void FAssetRegistryModule::StartupModule()
{
	// Create the UAssetRegistryImpl default object early, so it is ready for the caller of LoadModuleChecked<FAssetRegistryModule>().Get()
	LLM_SCOPE(ELLMTag::AssetRegistry);
	GetDefault<UAssetRegistryImpl>();
}
