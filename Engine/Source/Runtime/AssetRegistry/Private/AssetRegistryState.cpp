// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/AssetRegistryState.h"

#include "Algo/Compare.h"
#include "AssetRegistry/ParallelSort.h"
#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetBundleData.h"
#include "AssetRegistryArchive.h"
#include "AssetRegistryImpl.h"
#include "AssetRegistryPrivate.h"
#include "Async/ParallelFor.h"
#include "Blueprint/BlueprintSupport.h"
#include "Containers/MpscQueue.h"
#include "DependsNode.h"
#include "GenericPlatform/GenericPlatformChunkInstall.h"
#include "Logging/StructuredLog.h"
#include "Memory/LinearAllocator.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/StringBuilder.h"
#include "Misc/WildcardString.h"
#include "NameTableArchive.h"
#include "AssetRegistry/PackageReader.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Tasks/Task.h"
#include "Templates/SharedPointer.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/MetaData.h"
#include "UObject/NameBatchSerialization.h"
#include "UObject/PrimaryAssetId.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"

// Even if bSerializeDependencies is enabled, this can bypass serializing at runtime.
// Update your <project>.Target.cs build scripts to define as needed.
#ifndef ASSET_REGISTRY_ALLOW_DEPENDENCY_SERIALIZATION
#define ASSET_REGISTRY_ALLOW_DEPENDENCY_SERIALIZATION 1
#endif

FAssetRegistryState& FAssetRegistryState::operator=(FAssetRegistryState&& Rhs)
{
	Reset();

	CachedAssets						= MoveTemp(Rhs.CachedAssets);
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	IndirectAssetDataArrays				= MoveTemp(Rhs.IndirectAssetDataArrays);
#endif
	CachedAssetsByPackageName			= MoveTemp(Rhs.CachedAssetsByPackageName);
	CachedAssetsByPath					= MoveTemp(Rhs.CachedAssetsByPath);
	CachedAssetsByClass					= MoveTemp(Rhs.CachedAssetsByClass);
#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
	CachedAssetsByTag					= MoveTemp(Rhs.CachedAssetsByTag);
#else
	CachedClassesByTag					= MoveTemp(Rhs.CachedClassesByTag);
#endif
	CachedDependsNodes					= MoveTemp(Rhs.CachedDependsNodes);
	CachedPackageData					= MoveTemp(Rhs.CachedPackageData);
	PreallocatedAssetDataBuffers		= MoveTemp(Rhs.PreallocatedAssetDataBuffers);
	PreallocatedDependsNodeDataBuffers	= MoveTemp(Rhs.PreallocatedDependsNodeDataBuffers);
	PreallocatedPackageDataBuffers		= MoveTemp(Rhs.PreallocatedPackageDataBuffers);
	Swap(NumAssets,				Rhs.NumAssets);
	Swap(NumDependsNodes,		Rhs.NumDependsNodes);
	Swap(NumPackageData,		Rhs.NumPackageData);

	return *this;
}

FAssetRegistryState::~FAssetRegistryState()
{
	Reset();
}

void FAssetRegistryState::Reset()
{
	// if we have preallocated all the FAssetData's in a single block, free it now, instead of one at a time
	if (PreallocatedAssetDataBuffers.Num())
	{
		for (FAssetData* Buffer : PreallocatedAssetDataBuffers)
		{
			delete[] Buffer;
		}
		PreallocatedAssetDataBuffers.Reset();

		NumAssets = 0;
	}
	else
	{
		// Delete all assets in the cache
		for (FAssetData* AssetData : CachedAssets)
		{
			delete AssetData;
			NumAssets--;
		}
	}

	// Make sure we have deleted all our allocated FAssetData objects
	// March 2021: Temporarily remove this ensure to allow passing builds  while we find and fix the cause
	// TODO: Restore the ensure
	// ensure(NumAssets == 0);
	UE_CLOGF(NumAssets != 0, LogAssetRegistry, Display,
		"AssetRegistryState::Reset: NumAssets does not match the number of CachedAssets entries. Leaking some allocations.");

	if (PreallocatedDependsNodeDataBuffers.Num())
	{
		for (FDependsNode* Buffer : PreallocatedDependsNodeDataBuffers)
		{
			delete[] Buffer;
		}
		PreallocatedDependsNodeDataBuffers.Reset();
		NumDependsNodes = 0;
	}
	else
	{
		// Delete all depends nodes in the cache
		for (TMap<FAssetIdentifier, FDependsNode*>::TConstIterator DependsIt(CachedDependsNodes); DependsIt; ++DependsIt)
		{
			if (DependsIt.Value())
			{
				delete DependsIt.Value();
				NumDependsNodes--;
			}
		}
	}

	// Make sure we have deleted all our allocated FDependsNode objects
	ensure(NumDependsNodes == 0);

	if (PreallocatedPackageDataBuffers.Num())
	{
		for (FAssetPackageData* Buffer : PreallocatedPackageDataBuffers)
		{
			delete[] Buffer;
		}
		PreallocatedPackageDataBuffers.Reset();
		NumPackageData = 0;
	}
	else
	{
		// Delete all PackageDatas in the cache
		for (TMap<FName, FAssetPackageData*>::TConstIterator PackageDataIt(CachedPackageData); PackageDataIt; ++PackageDataIt)
		{
			if (PackageDataIt.Value())
			{
				delete PackageDataIt.Value();
				NumPackageData--;
			}
		}
	}

	// Make sure we have deleted all our allocated package data objects
	ensure(NumPackageData == 0);

	// Clear cache
	CachedAssetsByPackageName.Empty();
	CachedAssetsByPath.Empty();
	CachedAssetsByClass.Empty();
#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
	CachedAssetsByTag.Empty();
#else
	CachedClassesByTag.Empty();
#endif
	CachedDependsNodes.Empty();
	CachedPackageData.Empty();
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	IndirectAssetDataArrays.Empty();
#endif
	CachedAssets.Empty();
}

void FAssetRegistryState::FilterTags(const FAssetDataTagMapSharedView& InTagsAndValues, FAssetDataTagMap& OutTagsAndValues,
	const TSet<FName>* ClassSpecificFilterList, const FAssetRegistrySerializationOptions& Options)
{
	const TSet<FName>* AllClassesFilterList = Options.CookFilterlistTagsByClass.Find(UE::AssetRegistry::WildcardPathName);

	TStringBuilder<64> TagNameStr;

	// Exclude denied tags or include only allowed tags, based on how we were configured in ini
	for (const auto& TagPair : InTagsAndValues)
	{
		bool bKeep = false;

		// Cook_ tags, aka DevelopmentAssetRegistryTags are special; they are kept depending on whether the Options
		// are development or runtime and they do not use the options' filter list.
		TagNameStr.Reset();
		TagNameStr << TagPair.Key;
		if (FStringView(TagNameStr).StartsWith(UE::AssetRegistry::CookTagPrefix, ESearchCase::IgnoreCase))
		{
			bKeep = Options.bKeepDevelopmentAssetRegistryTags;
		}
		else
		{
			const bool bInAllClassesList = AllClassesFilterList
				&& (AllClassesFilterList->Contains(TagPair.Key)
					|| AllClassesFilterList->Contains(UE::AssetRegistry::WildcardFName));
			const bool bInClassSpecificList = ClassSpecificFilterList
				&& (ClassSpecificFilterList->Contains(TagPair.Key)
					|| ClassSpecificFilterList->Contains(UE::AssetRegistry::WildcardFName));
			if (Options.bUseAssetRegistryTagsAllowListInsteadOfDenyList)
			{
			// It's an allow list, only include it if it is in the all classes list or in the class specific list
				bKeep = bInAllClassesList || bInClassSpecificList;
			}
			else
			{
				// It's a deny list, include it unless it is in the all classes list or in the class specific list
				bKeep = !bInAllClassesList && !bInClassSpecificList;
			}
		}
		if (bKeep)
		{
			OutTagsAndValues.Add(TagPair.Key, TagPair.Value.ToLoose());
		}
	}
}

TSharedPtr<FAssetBundleData> FAssetRegistryState::FilterBundles(const TSharedPtr<FAssetBundleData>& InAssetBundles, const FAssetRegistrySerializationOptions& Options)
{
	if (InAssetBundles)
	{
		TArray<FAssetBundleEntry> Bundles;
		Bundles.Reserve(InAssetBundles->Bundles.Num());
		bool bIsDifferent = false;
		for (auto It = InAssetBundles->Bundles.CreateConstIterator(); It; ++It)
		{
			TStringBuilder<NAME_SIZE> Name;
			It->BundleName.AppendString(Name);
			bool bPassesFilters = true;
			for (const FString& Filter : Options.AssetBundlesDenyList)
			{
				if (FWildcardString::IsMatchSubstring(*Filter, Name.GetData(), Name.GetData() + Name.Len(), ESearchCase::IgnoreCase))
				{
					bPassesFilters = false;
					bIsDifferent = true;
					break;
				}
			}
			if (bPassesFilters)
			{
				Bundles.Add(*It);
			}
		}
		if (bIsDifferent)
		{
			Bundles.Shrink();
			TSharedPtr<FAssetBundleData> RetData = MakeShared<FAssetBundleData>();
			RetData->Bundles = MoveTemp(Bundles);
			return RetData;
		}
	}
	return InAssetBundles;
}

void FAssetRegistryState::InitializeFromExistingAndPrune(const FAssetRegistryState & ExistingState,
	const TSet<FName>& RequiredPackages, const TSet<FName>& RemovePackages,
	const TSet<int32> ChunksToKeep, const FAssetRegistrySerializationOptions& Options)
{
	const bool bIsFilteredByChunkId = ChunksToKeep.Num() != 0;
	const bool bIsFilteredByRequiredPackages = RequiredPackages.Num() != 0;
	const bool bIsFilteredByRemovedPackages = RemovePackages.Num() != 0;

	TMpscQueue<FName> RequiredDependNodePackagesQueue;
	TMpscQueue<FAssetData*> NewAssetDatas;

	// Duplicate asset data entries
	ExistingState.EnumerateAllMutableAssets(
		[&RequiredPackages, &RemovePackages, &ChunksToKeep, &Options,
		bIsFilteredByChunkId, bIsFilteredByRequiredPackages, bIsFilteredByRemovedPackages,
		&RequiredDependNodePackagesQueue, &NewAssetDatas, this]
		(FAssetData& AssetData)
	{
		bool bRemoveAssetData = false;
		bool bRemoveDependencyData = true;

		if (bIsFilteredByChunkId &&
			!AssetData.GetChunkIDs().ContainsByPredicate([&ChunksToKeep](int32 ChunkId)
				{
					return ChunksToKeep.Contains(ChunkId);
				}))
		{
			bRemoveAssetData = true;
		}
		else if (bIsFilteredByRequiredPackages && !RequiredPackages.Contains(AssetData.PackageName))
		{
			bRemoveAssetData = true;
		}
		else if (bIsFilteredByRemovedPackages && RemovePackages.Contains(AssetData.PackageName))
		{
			bRemoveAssetData = true;
		}
		else if (Options.bFilterAssetDataWithNoTags && AssetData.TagsAndValues.Num() == 0 &&
			!FPackageName::IsLocalizedPackage(WriteToString<256>(AssetData.PackageName)))
		{
			bRemoveAssetData = true;
			bRemoveDependencyData = Options.bFilterDependenciesWithNoTags;
		}

		if (bRemoveAssetData)
		{
			if (!bRemoveDependencyData)
			{
				RequiredDependNodePackagesQueue.Enqueue(AssetData.PackageName);
			}
			return;
		}

		FAssetDataTagMap NewTagsAndValues;
		FAssetRegistryState::FilterTags(AssetData.TagsAndValues, NewTagsAndValues,
			Options.CookFilterlistTagsByClass.Find(AssetData.AssetClassPath), Options);

		FAssetData* NewAssetData = nullptr;
		if (AssetData.IsTopLevelAsset())
		{
			NewAssetData = new FAssetData(AssetData.PackageName, AssetData.PackagePath, AssetData.AssetName,
				AssetData.AssetClassPath, NewTagsAndValues, AssetData.GetChunkIDs(), AssetData.PackageFlags);
		}
		else
		{
			NewAssetData = new FAssetData(AssetData.PackageName.ToString(), AssetData.GetObjectPathString(), 
				AssetData.AssetClassPath, NewTagsAndValues, AssetData.GetChunkIDs(), AssetData.PackageFlags);
		}

		NewAssetData->TaggedAssetBundles = FAssetRegistryState::FilterBundles(AssetData.TaggedAssetBundles, Options);

		// Add asset to new state
		NewAssetDatas.Enqueue(NewAssetData);
	}, UE::AssetRegistry::EEnumerateAssetsFlags::Parallel | UE::AssetRegistry::EEnumerateAssetsFlags::AllowAll);

	TSet<FName> RequiredDependNodePackages;
	FName DependNodePackageName;
	while (RequiredDependNodePackagesQueue.Dequeue(DependNodePackageName))
	{
		RequiredDependNodePackages.Add(DependNodePackageName);
	}

	FAssetData* NewAssetDataToAdd;
	while (NewAssetDatas.Dequeue(NewAssetDataToAdd))
	{
		AddAssetData(NewAssetDataToAdd);
	}

	// Create package data for all script and required packages
	// Do this in a ParallelFor since it requires a call to IsScriptPackage which can be expensive when called 1 million times
	TArray<bool> ShouldAddAssetPackageData;
	int32 CachedPackageDataMaxIndex = ExistingState.CachedPackageData.GetMaxIndex();
	ShouldAddAssetPackageData.SetNumUninitialized(CachedPackageDataMaxIndex);
	ParallelFor(CachedPackageDataMaxIndex,
		[&ExistingState, &ShouldAddAssetPackageData, this](int32 Index)
		{
			FSetElementId Id = FSetElementId::FromInteger(Index);
			bool bShouldAdd = false;
			if (ExistingState.CachedPackageData.IsValidId(Id))
			{
				const TPair<FName, FAssetPackageData*>& Pair = ExistingState.CachedPackageData.Get(Id);
				if (Pair.Value)
				{
					// Only add if also in asset data map, or script package
					if (CachedAssetsByPackageName.Find(Pair.Key) ||
						FPackageName::IsScriptPackage(WriteToString<256>(Pair.Key)))
					{
						bShouldAdd = true;
					}
				}
			}
			ShouldAddAssetPackageData[Index] = bShouldAdd;
		});
	for (int32 Index = 0; Index < CachedPackageDataMaxIndex; ++Index)
	{
		if (ShouldAddAssetPackageData[Index])
		{
			FSetElementId Id = FSetElementId::FromInteger(Index);
			check(ExistingState.CachedPackageData.IsValidId(Id));
			const TPair<FName, FAssetPackageData*>& Pair = ExistingState.CachedPackageData.Get(Id);
			FAssetPackageData* NewData = CreateOrGetAssetPackageData(Pair.Key);
			*NewData = *Pair.Value;
		}
	}

	// Find valid dependency nodes for all script and required packages, and create a new
	// node corresponding to each old node.
	TMap<FDependsNode*, FDependsNode*> RedirectCache;
	TMap<FDependsNode*, FDependsNode*> OldNodeToNewNode;
	OldNodeToNewNode.Reserve(ExistingState.CachedDependsNodes.Num());
	for (const TPair<FAssetIdentifier, FDependsNode*>& Pair : ExistingState.CachedDependsNodes)
	{
		FDependsNode* Node = Pair.Value;
		const FAssetIdentifier& Id = Node->GetIdentifier();
		bool bRemoveDependsNode = false;

		if (Options.bFilterSearchableNames && Id.IsValue())
		{
			bRemoveDependsNode = true;
		}
		else if (Id.IsPackage() &&
			!CachedAssetsByPackageName.Contains(Id.PackageName) &&
			!RequiredDependNodePackages.Contains(Id.PackageName) &&
			!Node->IsScriptPath())
		{
			bRemoveDependsNode = true;
		}

		if (!bRemoveDependsNode)
		{
			FDependsNode*& ExistingNewNode = OldNodeToNewNode.FindOrAdd(Node);
			if (!ExistingNewNode)
			{
				ExistingNewNode = CreateOrFindDependsNode(Node->GetIdentifier());
			}
		}
		else
		{
			// If we're removing this Node because it is a redirector, store its target
			// in RedirectCache so that referencers can redirect to the target instead.
			ExistingState.ResolveRedirector(Node, ExistingState.CachedAssets, RedirectCache);
		}
	}

	TMpscQueue<TPair<FDependsNode*, FDependsNode*>> ReferencerPairs;

	int32 OldNodeToNewNodeMaxId = OldNodeToNewNode.GetMaxIndex();
	ParallelFor(OldNodeToNewNodeMaxId, [&OldNodeToNewNode, &ReferencerPairs, &RedirectCache](int32 Index)
		{
			FSetElementId Id = FSetElementId::FromInteger(Index);
			if (OldNodeToNewNode.IsValidId(Id))
			{
				const TPair<FDependsNode*, FDependsNode*>& Pair = OldNodeToNewNode.Get(Id);
				FDependsNode* OldNode = Pair.Key;
				FDependsNode* NewNode = Pair.Value;
				NewNode->Reserve(OldNode);
				NewNode->SetIsDependencyListSorted(UE::AssetRegistry::EDependencyCategory::All, false);
				NewNode->SetIsReferencersSorted(false);

				OldNode->IterateOverDependencies([&, OldNode, NewNode](FDependsNode* InDependency,
					UE::AssetRegistry::EDependencyCategory InCategory,
					UE::AssetRegistry::EDependencyProperty InFlags,
					bool bDuplicate)
					{
						// Only add link if it's part of the filtered asset set
						FDependsNode** NewDependency = OldNodeToNewNode.Find(InDependency);
						if (!NewDependency)
						{
							// Check for a redirected target; redirectors might be pruned but their targets might not be.
							FDependsNode** RedirectedInDependency = RedirectCache.Find(InDependency);
							if (RedirectedInDependency && *RedirectedInDependency != InDependency)
							{
								NewDependency = OldNodeToNewNode.Find(*RedirectedInDependency);
							}
						}
						if (NewDependency)
						{
							NewNode->AddDependency(*NewDependency, InCategory, InFlags);
							ReferencerPairs.Enqueue(NewNode, *NewDependency);
						}
					});
				NewNode->SetIsScriptDependenciesInitialized(OldNode->IsScriptDependenciesInitialized());
				NewNode->SetIsDependencyListSorted(UE::AssetRegistry::EDependencyCategory::All, true);
			}
		});

	// Add the Referencer link for each dependency link we added
	TPair<FDependsNode*, FDependsNode*> ReferencerPair;
	while (ReferencerPairs.Dequeue(ReferencerPair))
	{
		ReferencerPair.Value->AddReferencer(ReferencerPair.Key);
	}

	// Remove any orphaned depends nodes. This will leave cycles in but those might represent useful data
	for (TMap<FDependsNode*, FDependsNode*>::TIterator Iter(OldNodeToNewNode); Iter; ++Iter)
	{
		FDependsNode* DependsNode = Iter->Value;
		if (DependsNode->GetConnectionCount() == 0)
		{
			RemoveDependsNode(DependsNode->GetIdentifier());
			Iter.RemoveCurrent();
		}
	}
	OldNodeToNewNodeMaxId = OldNodeToNewNode.GetMaxIndex();

	// Restore the referencer sortedness that we turned off for performance when creating each DependsNode
	ParallelFor(OldNodeToNewNodeMaxId, [&OldNodeToNewNode](int32 Index)
		{
			FSetElementId Id = FSetElementId::FromInteger(Index);
			if (OldNodeToNewNode.IsValidId(Id))
			{
				const TPair<FDependsNode*, FDependsNode*>& Pair = OldNodeToNewNode.Get(Id);
				FDependsNode* NewNode = Pair.Value;
				NewNode->SetIsReferencersSorted(true);
			}
		});
}

void FAssetRegistryState::InitializeFromExisting(const FAssetDataMap& AssetDataMap,
	const TMap<FAssetIdentifier, FDependsNode*>& DependsNodeMap, 
	const TMap<FName, FAssetPackageData*>& AssetPackageDataMap,
	const FAssetRegistrySerializationOptions& Options,
	EInitializationMode InInitializationMode,
	FAssetRegistryAppendResult* OutAppendResult)
{
	TArray<FAssetData*> ExistingAssetForNewAsset;
	int32 AssetDataMapMaxIndex = AssetDataMap.GetMaxIndex();
	if (InInitializationMode == EInitializationMode::Rebuild)
	{
		Reset();
	}
	else
	{
		ExistingAssetForNewAsset.SetNum(AssetDataMapMaxIndex);
		for (int32 Index = 0; Index < AssetDataMapMaxIndex; ++Index)
		{
			FAssetData* ExistingAsset = nullptr;
			FSetElementId SetId = FSetElementId::FromInteger(Index);
			if (AssetDataMap.IsValidId(SetId))
			{
				FAssetData* AssetData = AssetDataMap.Get(SetId);
				if (FAssetData* const* Ptr = CachedAssets.Find(FCachedAssetKey(AssetData)))
				{
					ExistingAsset = *Ptr;
				}
			}
			ExistingAssetForNewAsset[Index] = ExistingAsset;
		}
	}

	// FStoreBuilder is a serialized tag builder which will allow us to store the asset tags much more efficiently.
	TArray<TPair<FAssetData*, FixedTagPrivate::FPartialMapHandle>> AssetsToUpdate;
	FixedTagPrivate::FStoreBuilder Builder(FAssetRegistryWriterOptions(Options).Tags);

	// We accumulate Assets to add to *this in a queue inside the ParallelFor. Construction of each asset is expensive
	// and we want to work on it on multiple threads. But we can't add them to *this simultaneously because we can't append
	// to our CachedAsset and query accelerator maps in parallel. So we need to add them all in a batch after the
	// ParallelFor is over. But waiting until after the ParallelFor is over causes cache misses on the ids, so instead
	// of doing them all in a single batch, we add them in smaller batches as we go. Each worker of the parallelfor
	// tries to take responsibility for adding the batch under the lock when the batch reaches our desired size, and
	// the winner of the race is the one that adds the batch to *this.
	const int32 QueueFlushLength = 100; // Might want to change this to be proportional to platform's Cache size.
	std::atomic<int32> AssetsToAddNum(0);
	FCriticalSection FlushQueueLock;

	TMpscQueue<TPair<FAssetData*, FAssetData*>> AssetsToAdd;

	auto FlushQueue = [&AssetsToAdd, this, OutAppendResult, &Builder, &AssetsToUpdate]()
		{
			TPair<FAssetData*, FAssetData*> NewAndExisting;
			while (AssetsToAdd.Dequeue(NewAndExisting))
			{
				FAssetData* NewData = NewAndExisting.Key;
				FAssetData* ExistingData = NewAndExisting.Value;
				if (ExistingData)
				{
					bool bIsModified = false;
					UpdateAssetData(ExistingData, MoveTemp(*NewData), &bIsModified);
					if (OutAppendResult && bIsModified)
					{
						OutAppendResult->UpdatedAssets.Add(ExistingData);
					}
					delete NewData;
				}
				else
				{
					AddAssetData(NewData);
					if (OutAppendResult)
					{
						OutAppendResult->AddedAssets.Add(NewData);
					}
					AssetsToUpdate.Add(TPair<FAssetData*, FixedTagPrivate::FPartialMapHandle>(NewData,
						Builder.AddTagMap(NewData->TagsAndValues)));
				}
			}
		};

	auto ConditionalAddDuplicatedAssetToQueue =
		[InInitializationMode, &Options, &AssetsToAdd, &AssetDataMap, &ExistingAssetForNewAsset,
		&AssetsToAddNum, QueueFlushLength, &FlushQueue, &FlushQueueLock]
		(int32 Index)
		{
			FSetElementId SetId = FSetElementId::FromInteger(Index);
			if (!AssetDataMap.IsValidId(SetId))
			{
				return;
			}
			const FAssetData* AssetDataPtr = AssetDataMap.Get(SetId);

			if (AssetDataPtr == nullptr)
			{
				return;
			}
			const FAssetData& AssetData = *AssetDataPtr;

			FAssetData* ExistingData = nullptr;
			if (InInitializationMode != EInitializationMode::Rebuild) // minor optimization to avoid lookup in rebuild mode
			{
				ExistingData = ExistingAssetForNewAsset[Index];
			}
			if (InInitializationMode == EInitializationMode::OnlyUpdateExisting && ExistingData == nullptr)
			{
				return;
			}
			if (InInitializationMode == EInitializationMode::OnlyUpdateNew && ExistingData != nullptr)
			{
				return;
			}

			if (Options.bFilterAssetDataWithNoTags && AssetData.TagsAndValues.Num() == 0 &&
				!FPackageName::IsLocalizedPackage(WriteToString<256>(AssetData.PackageName)))
			{
				// based on current serialization settings this should be skipped.
				return;
			}

			// Filter asset registry tags now
			FAssetDataTagMap LocalTagsAndValues;
			FAssetRegistryState::FilterTags(AssetData.TagsAndValues, LocalTagsAndValues,
				Options.CookFilterlistTagsByClass.Find(AssetData.AssetClassPath), Options);

			// Filter AssetBundles separately, LocalAssetBundles will point to the same TaggedAssetBundles if it is not different
			TSharedPtr<FAssetBundleData> LocalAssetBundles = FAssetRegistryState::FilterBundles(AssetData.TaggedAssetBundles, Options);

			FAssetData* NewData = new FAssetData(AssetData);
			NewData->TagsAndValues = FAssetDataTagMapSharedView(MoveTemp(LocalTagsAndValues));
			NewData->TaggedAssetBundles = MoveTemp(LocalAssetBundles);
			AssetsToAdd.Enqueue(NewData, ExistingData);
			int32 OldNum = AssetsToAddNum.fetch_add(1, std::memory_order_relaxed);
			if (OldNum == QueueFlushLength)
			{
				AssetsToAddNum.store(0, std::memory_order_relaxed);
				// Only the winner of the race that happens to add the element that sets num to QueueFlushLength will
				// enter this OldNum == QueueFlushLength block, but as the other threads continue adding, it is
				// possible that another thread will encounter it the next time around after we reset to 0, so to
				// avoid simultaneous access from all winners of the races, we need to also enter a critical section.
				if (FlushQueueLock.TryLock())
				{
					FlushQueue();
					FlushQueueLock.Unlock();
				}
			}
		};

	ParallelFor(AssetDataMapMaxIndex,
		[&ConditionalAddDuplicatedAssetToQueue](int32 Index)
		{
			ConditionalAddDuplicatedAssetToQueue(Index);
		});
	FlushQueue();
	// Construct the optimized tag data then set the handle map back to the assets we need to update
	TRefCountPtr<const FixedTagPrivate::FStore> FixedStore = FixedTagPrivate::InitializeFromBuilder(Builder);
	for (TPair<FAssetData*, FixedTagPrivate::FPartialMapHandle>& AssetToHandle : AssetsToUpdate)
	{
		FixedTagPrivate::FMapHandle FixedMapHandle = AssetToHandle.Value.MakeFullHandle(FixedStore->Index);
		AssetToHandle.Key->TagsAndValues = FAssetDataTagMapSharedView(FixedMapHandle);
	}

	TSet<FAssetIdentifier> ScriptPackages;
	if (InInitializationMode != EInitializationMode::OnlyUpdateExisting)
	{
		for (const TPair<FName, FAssetPackageData*>& Pair : AssetPackageDataMap)
		{
			bool bIsScriptPackage = FPackageName::IsScriptPackage(WriteToString<256>(Pair.Key));
			if (InInitializationMode == EInitializationMode::OnlyUpdateNew && CachedPackageData.Find(Pair.Key))
			{
				continue;
			}
			if (Pair.Value)
			{
				// Only add if also in asset data map, or script package
				FAssetPackageData* NewData = nullptr;
				if (bIsScriptPackage)
				{
					ScriptPackages.Add(Pair.Key);
					NewData = CreateOrGetAssetPackageData(Pair.Key);
				}
				else if (CachedAssetsByPackageName.Find(Pair.Key))
				{
					NewData = CreateOrGetAssetPackageData(Pair.Key);
				}

				if (NewData)
				{
					// Add the new location to any existing location as it's possible we 
					// have the same content available from more than one location.
					FPackageName::EPackageLocationFilter OriginalLocation = NewData->GetPackageLocation();
					*NewData = *Pair.Value;
					NewData->SetPackageLocation(FPackageName::EPackageLocationFilter(uint8(NewData->GetPackageLocation()) | uint8(OriginalLocation)));
				}
			}
		}

		TMap<FAssetIdentifier, FDependsNode*> FilteredDependsNodeMap;
		const TMap<FAssetIdentifier, FDependsNode*>* DependsNodesToAdd = &DependsNodeMap;
		if (InInitializationMode == EInitializationMode::OnlyUpdateNew)
		{
			// Keep the original DependsNodeMap for reference,
			// but remove from NodesToAdd all nodes that already have dependency data
			// Also reserve up-front all (unfiltered) nodes we are adding, to avoid reallocating the Referencers array.
			FilteredDependsNodeMap.Reserve(DependsNodeMap.Num());
			DependsNodesToAdd = &FilteredDependsNodeMap;
			for (const TPair<FAssetIdentifier, FDependsNode*>& Pair : DependsNodeMap)
			{
				bool bTargetNodeAlreadyExists = false;
				FDependsNode* SourceNode = Pair.Value;
				FDependsNode* TargetNode = CreateOrFindDependsNode(Pair.Key, &bTargetNodeAlreadyExists);
				if (!bTargetNodeAlreadyExists)
				{
					FilteredDependsNodeMap.Add(Pair.Key, SourceNode);
				}
				TargetNode->Reserve(SourceNode);
			}
		}
		else
		{
			// Reserve up-front all the nodes that we are adding, so we do not reallocate
			// the Referencers array multiple times on a node as we add nodes that refer to it
			for (const TPair<FAssetIdentifier, FDependsNode*>& Pair : DependsNodeMap)
			{
				FDependsNode* SourceNode = Pair.Value;
				FDependsNode* TargetNode = CreateOrFindDependsNode(Pair.Key);
				TargetNode->Reserve(SourceNode);
			}
		}

		for (const TPair<FAssetIdentifier, FDependsNode*>& Pair : *DependsNodesToAdd)
		{
			FDependsNode* SourceNode = Pair.Value;
			FDependsNode* TargetNode = CreateOrFindDependsNode(Pair.Key);
			SourceNode->IterateOverDependencies([this, &DependsNodeMap, &ScriptPackages, TargetNode]
			(FDependsNode* InDependency, UE::AssetRegistry::EDependencyCategory InCategory,
				UE::AssetRegistry::EDependencyProperty InFlags,
				bool bDuplicate)
			{
				const FAssetIdentifier& Identifier = InDependency->GetIdentifier();
				if (DependsNodeMap.Find(Identifier) || ScriptPackages.Contains(Identifier))
				{
					// Only add if this node is in the incoming map
					FDependsNode* TargetDependency = CreateOrFindDependsNode(Identifier);
					TargetNode->SetIsDependencyListSorted(InCategory, false);
					TargetNode->AddDependency(TargetDependency, InCategory, InFlags);
					TargetDependency->SetIsReferencersSorted(false);
					TargetDependency->AddReferencer(TargetNode);
				}
			});
			TargetNode->SetIsScriptDependenciesInitialized(SourceNode->IsScriptDependenciesInitialized());
		}

		// Restore the sortedness that we turned off for performance when creating each DependsNode
		SetDependencyNodeSorting(true, true);
	}
	Shrink();
}

void FAssetRegistryState::ConsumeExisting(FAssetRegistryState&& Consume,
	const FAssetRegistrySerializationOptions& Options,
	FAssetRegistryAppendResult* OutAppendResult)
{
	// Move over AssetDatas
	TArray<FAssetData*> AssetsToDeallocate;
	for (FAssetData* AssetData : Consume.CachedAssets)
	{
		if (CachedAssets.Find(FCachedAssetKey(AssetData)))
		{
			AssetsToDeallocate.Add(AssetData);
			continue;
		}

		AddAssetData(AssetData);
		if (OutAppendResult)
		{
			OutAppendResult->AddedAssets.Add(AssetData);
		}
	}

	// Move over PackageDatas
	TArray<FAssetPackageData*> PackageDatasToDeallocate;
	for (const TPair<FName, FAssetPackageData*>& Pair : Consume.CachedPackageData)
	{
		if (!Pair.Value)
		{
			continue;
		}
		FAssetPackageData*& ExistingData = CachedPackageData.FindOrAdd(Pair.Key);
		if (ExistingData)
		{
			PackageDatasToDeallocate.Add(Pair.Value);
			continue;
		}
		ExistingData = Pair.Value;
		NumPackageData++;
	}

	// Move over DependsNodes, more complicated because of the links.
	// First, move over DependsNodes, and create a map from OldNode to NewNode, either OldNode to itself because keep
	// it or OldNode to the existing node that takes priority over it.
	TArray<FDependsNode*> NodesToDeallocate;
	TMap<FDependsNode*, FDependsNode*> OldNodeToNewNode;
	OldNodeToNewNode.Reserve(Consume.CachedDependsNodes.Num());
	for (const TPair<FAssetIdentifier, FDependsNode*>& Pair : Consume.CachedDependsNodes)
	{
		FDependsNode* ConsumedNode = Pair.Value;
		FDependsNode*& ExistingNode = CachedDependsNodes.FindOrAdd(Pair.Key);
		if (ExistingNode == nullptr)
		{
			ExistingNode = ConsumedNode;
			NumDependsNodes++;
		}
		else
		{
			NodesToDeallocate.Add(ConsumedNode);
		}
		OldNodeToNewNode.Add(ConsumedNode, ExistingNode);
	}

	// Second, remap:
	// Add mirrored two-way links for any links between an old node we are keeping and an old node we are not
	// keeping: copy over the link from the old node we are not keeping into the existing node that takes
	// priority over it.
	for (const TPair<FDependsNode*, FDependsNode*>& Pair : OldNodeToNewNode)
	{
		FDependsNode* OldSourceNode = Pair.Key;
		FDependsNode* NewSourceNode = Pair.Value;
		// We are keeping a node only if it maps to itself in FDependsNode*
		if (NewSourceNode == OldSourceNode)
		{
			// For any dependencies or references from the kept old node to a replaced old node, copy the replaced old node's
			// mirrored reference or dependency onto the new node that replaces it.
			OldSourceNode->IterateOverDependencies([OldSourceNode, &OldNodeToNewNode]
			(FDependsNode* OldTargetNode,
				UE::AssetRegistry::EDependencyCategory Category,
				UE::AssetRegistry::EDependencyProperty Properties,
				bool bDuplicate)
				{
					if (!bDuplicate)
					{
						FDependsNode** NewTargetNode = OldNodeToNewNode.Find(OldTargetNode);
						if (NewTargetNode && *NewTargetNode != OldTargetNode)
						{
							(*NewTargetNode)->AddReferencer(OldSourceNode);
						}
					}
				});
			OldSourceNode->IterateOverReferencers([OldSourceNode, &OldNodeToNewNode]
			(FDependsNode* OldReferencer)
				{
					FDependsNode** NewReferencerPtr = OldNodeToNewNode.Find(OldReferencer);
					FDependsNode* NewReferencer = NewReferencerPtr ? *NewReferencerPtr : nullptr;
					if (NewReferencer && NewReferencer != OldReferencer)
					{
						// We have a remapped referencer and we need to add a dependency from the remapped version of that referencer
						// to OldSourceNode. We have to look up what kind of dependency to add from that old referencer.
						OldReferencer->IterateOverDependencies([OldSourceNode, NewReferencer](
							FDependsNode* OldTargetOfOldRefencer,
							UE::AssetRegistry::EDependencyCategory Category,
							UE::AssetRegistry::EDependencyProperty Properties,
							bool bDuplicate)
							{
								if (OldTargetOfOldRefencer == OldSourceNode)
								{
									NewReferencer->AddDependency(OldSourceNode, Category, Properties);
								}
							});
					}
				});

			// Now that we've read what we need to from the old targets of the old links on this kept old node, remap all those
			// links to old targets to instead point to the new remapped targets.
			OldSourceNode->RemapLinks(OldNodeToNewNode);

			// ConsumeExisting, like InitializeFromExisting, sets consumed nodes to sorted. We also set AllowShrinking to true.
			// We use this contract because Consume and InitializeFromExisting are almost exclusively used when not gathering,
			// so we should optimize for read and for memory rather than write speed.
			OldSourceNode->SetAllowShrinking(true);
			OldSourceNode->SetIsDependencyListSorted(UE::AssetRegistry::EDependencyCategory::All, true);
			OldSourceNode->SetIsReferencersSorted(true);
		}
	}

	// Steal all of the preallocated buffers; we don't currently track whether we are not using them.
	for (FAssetData* Buffer : Consume.PreallocatedAssetDataBuffers)
	{
		this->PreallocatedAssetDataBuffers.Add(Buffer);
	}
	for (FDependsNode* Buffer : Consume.PreallocatedDependsNodeDataBuffers)
	{
		this->PreallocatedDependsNodeDataBuffers.Add(Buffer);
	}
	for (FAssetPackageData* Buffer : Consume.PreallocatedPackageDataBuffers)
	{
		this->PreallocatedPackageDataBuffers.Add(Buffer);
	}

	// Reset the Consume state, but only delete its data that we didn't steal.
	if (Consume.PreallocatedAssetDataBuffers.Num())
	{
		Consume.PreallocatedAssetDataBuffers.Empty();
	}
	else
	{
		for (FAssetData* AssetData : AssetsToDeallocate)
		{
			delete AssetData;
		}
	}
	Consume.NumAssets = 0;

	if (Consume.PreallocatedDependsNodeDataBuffers.Num())
	{
		Consume.PreallocatedDependsNodeDataBuffers.Empty();
		// Clear memory on the OldNodes, since they are staying around unused.
		for (FDependsNode* DependsNode : NodesToDeallocate)
		{
			DependsNode->Empty();
		}
	}
	else
	{
		for (FDependsNode* DependsNode : NodesToDeallocate)
		{
			delete DependsNode;
		}
	}
	Consume.NumDependsNodes = 0;

	if (Consume.PreallocatedPackageDataBuffers.Num())
	{
		Consume.PreallocatedPackageDataBuffers.Empty();
	}
	else
	{
		for (FAssetPackageData* AssetPackageData : PackageDatasToDeallocate)
		{
			delete AssetPackageData;
		}
	}
	Consume.NumPackageData = 0;

	// Clear the containers on the consumed state; we can't call Reset since it also deletes the elements.
	Consume.CachedAssetsByPackageName.Empty();
	Consume.CachedAssetsByPath.Empty();
	Consume.CachedAssetsByClass.Empty();
#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
	Consume.CachedAssetsByTag.Empty();
#else
	Consume.CachedClassesByTag.Empty();
#endif
	Consume.CachedDependsNodes.Empty();
	Consume.CachedPackageData.Empty();
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	Consume.IndirectAssetDataArrays.Empty();
#endif
	Consume.CachedAssets.Empty();

	this->Shrink();
}

void FAssetRegistryState::PruneAssetData(const TSet<FName>& RequiredPackages, const TSet<FName>& RemovePackages,
	const FAssetRegistrySerializationOptions& Options)
{
	PruneAssetData(RequiredPackages, RemovePackages, TSet<int32>(), Options);
}

void FAssetRegistryState::PruneAssetData(const TSet<FName>& RequiredPackages, const TSet<FName>& RemovePackages,
	const TSet<int32> ChunksToKeep, const FAssetRegistrySerializationOptions& Options)
{
	FAssetRegistryPruneOptions PruneOptions;
	PruneOptions.RequiredPackages = RequiredPackages;
	PruneOptions.RemovePackages = RemovePackages;
	PruneOptions.ChunksToKeep = ChunksToKeep;
	PruneOptions.Options = Options;
	Prune(PruneOptions);
}

void FAssetRegistryState::Prune(const FAssetRegistryPruneOptions& PruneOptions)
{
	const TSet<FName>& RequiredPackages = PruneOptions.RequiredPackages;
	const TSet<FName>& RemovePackages = PruneOptions.RemovePackages;
	const TSet<int32>& ChunksToKeep = PruneOptions.ChunksToKeep;
	const FAssetRegistrySerializationOptions& Options = PruneOptions.Options;

	const bool bIsFilteredByChunkId = ChunksToKeep.Num() != 0;
	const bool bIsFilteredByRequiredPackages = RequiredPackages.Num() != 0;
	const bool bIsFilteredByRemovedPackages = RemovePackages.Num() != 0;

	TArray<FAssetData*> AllAssetData = CachedAssets.Array();
	int32 NumAssetDatas = AllAssetData.Num();
	int32 NumRemovedAssetDatas = 0;

	TArray<int8> KeepAssetDataIndex;
	TArray<FDependsNode*> AllDependsNodes;
	// RemapNodes is passed to RemapLinks for all nodes we are keeping. Nodes that we are keeping map to themselves.
	// Nodes that we are not keeping eventually either map to either null, or to a redirect that we are keeping.
	// While constructing it, all nodes we are not keeping map to null, and then we do a pass looking for redirects
	// and replace those nulls with redirects.
	TMap<FDependsNode*, FDependsNode*> RemapNodes;

	CachedDependsNodes.GenerateValueArray(AllDependsNodes);
	KeepAssetDataIndex.SetNum(AllAssetData.Num());
	RemapNodes.Reserve(AllDependsNodes.Num());
	for (FDependsNode* DependsNode : AllDependsNodes)
	{
		RemapNodes.Add(DependsNode, nullptr);
	}

	ParallelFor(AllAssetData.Num(),
		[&AllAssetData, &KeepAssetDataIndex, &RemapNodes, &NumRemovedAssetDatas,
		this, &RequiredPackages, &RemovePackages, &ChunksToKeep, &Options, &PruneOptions,
		bIsFilteredByChunkId, bIsFilteredByRequiredPackages, bIsFilteredByRemovedPackages
		]
		(int32 AssetDataIndex)
		{
			FAssetData* AssetData = AllAssetData[AssetDataIndex];
			// Remove assets, and remove FDependsNode for removed assets.
			// Exception: if !bFilterDependenciesWithNoTags, then keep the FDependsNode of Assets that were removed
			// only due to having no tags.
			bool bKeepAssetData = true;
			bool bKeepDependsNode = true;

			if (bIsFilteredByChunkId &&
				!AssetData->GetChunkIDs().ContainsByPredicate([&ChunksToKeep](int32 ChunkId)
					{
						return ChunksToKeep.Contains(ChunkId);
					}))
			{
				bKeepAssetData = false;
				bKeepDependsNode = false;
			}
			else if (bIsFilteredByRequiredPackages && !RequiredPackages.Contains(AssetData->PackageName))
			{
				bKeepAssetData = false;
				bKeepDependsNode = false;
			}
			else if (bIsFilteredByRemovedPackages && RemovePackages.Contains(AssetData->PackageName))
			{
				bKeepAssetData = false;
				bKeepDependsNode = false;
			}
			else if (Options.bFilterAssetDataWithNoTags && AssetData->TagsAndValues.Num() == 0 &&
				!FPackageName::IsLocalizedPackage(WriteToString<256>(AssetData->PackageName)) &&
				// TODO: Add a package flag for PKG_CookGenerator and check that here as well.
				!(AssetData->PackageFlags & PKG_CookGenerated))
			{
				bKeepAssetData = false;
				bKeepDependsNode = !Options.bFilterDependenciesWithNoTags;
			}

			KeepAssetDataIndex[AssetDataIndex] = bKeepAssetData;
			NumRemovedAssetDatas += bKeepAssetData ? 0 : 1;

			if (bKeepDependsNode)
			{
				FDependsNode** DependsNodePtr = CachedDependsNodes.Find(AssetData->PackageName);
				// We are keeping this node, so set its remap node to itself. 
				// Multiple assets can share the same dependsnode; keep the depends node if any of them say to keep it.
				// We write atomically to prevent a technical datarace from multiple writes to the same variable; it's not
				// an actual datarace because we always write the same value if we write anything at all.
				// AtomicStore_Relaxed only supports integer types, so we have to cast our pointer and value to an integertype.
				if (DependsNodePtr)
				{
					FDependsNode* DependsNode = *DependsNodePtr;
					FDependsNode*& TargetDependsNode = RemapNodes.FindChecked(DependsNode);
					FPlatformAtomics::AtomicStore_Relaxed((PTRINT*)&TargetDependsNode, (PTRINT)DependsNode);
				}

				FPrimaryAssetId PrimaryAssetId = AssetData->GetPrimaryAssetId();
				if (PrimaryAssetId.IsValid())
				{
					DependsNodePtr = CachedDependsNodes.Find(PrimaryAssetId);
					if (DependsNodePtr)
					{
						FDependsNode* DependsNode = *DependsNodePtr;
						FDependsNode*& TargetDependsNode = RemapNodes.FindChecked(DependsNode);
						FPlatformAtomics::AtomicStore_Relaxed((PTRINT*)&TargetDependsNode, (PTRINT)DependsNode);
					}
				}
			}

		});

	// Mark-for-removal all other dependsnodes that are filtered out by our settings
	int32 MaxId = RemapNodes.GetMaxIndex();
	ParallelFor(MaxId, [&RemapNodes, &Options, &PruneOptions](int32 Index)
		{
			FSetElementId SetId = FSetElementId::FromInteger(Index);
			if (!RemapNodes.IsValidId(SetId))
			{
				return;
			}
			TPair<FDependsNode*, FDependsNode*>& RemapPair = RemapNodes.Get(SetId);

			FDependsNode* DependsNode = RemapPair.Key;
			FDependsNode*& RemapNode = RemapPair.Value;
			bool bKeepDependsNode = DependsNode == RemapNode; // kept DependsNodes map to themselves.
			const FAssetIdentifier& Id = DependsNode->GetIdentifier();

			// bKeepDependsNode is initially the value false if the dependsnode does not correspond to an AssetData
			// (SearchableName, or it does not exist in AssetDatas) somehow. If it does correspond to an AssetData,
			// it is true if we decided to keep the package's DependsNode based on whether we kept the AssetData and on
			// some flags, in the loop above.
			// Override this initial value with some further type-specific rules for keeping or discarding DependsNode.

			if (Id.IsValue())
			{
				// SearchableName; KeepDependsNode is always initially false, set it instead to remove the
				// SearchableName depends node if and only if we're removing all SearchableName depends nodes.
				bKeepDependsNode = !Options.bFilterSearchableNames;
			}
			else if (Id.PrimaryAssetType.IsValid())
			{
				const FPrimaryAssetId PrimaryAssetId = Id.GetPrimaryAssetId();
				if (!PrimaryAssetId.IsValid())
				{
					UE_LOGF(LogAssetRegistry, Warning, "FAssetRegistryState::Prune: Unknown type of DependsNode: %ls", *Id.ToString());
					bKeepDependsNode = false;
				}
				else
				{
					// PrimaryAsset. bKeepDependsNode is false if it does not have an AssetData, or if we decided to
					// discard that AssetData above. Override this to additionally keep the asset if we are not
					// bRemoveDependenciesWithoutPackages or if it is in
					// RemoveDependenciesWithoutPackagesKeepPrimaryAssetTypes.
					bKeepDependsNode |= !PruneOptions.bRemoveDependenciesWithoutPackages ||
						PruneOptions.RemoveDependenciesWithoutPackagesKeepPrimaryAssetTypes.Contains(
							PrimaryAssetId.PrimaryAssetType);
				}
			}
			else if (Id.IsPackage())
			{
				// Package. bKeepDependsNode is false if the package does not have an AssetData, or if we decided to
				// discard above. Override for script packages, which do not have AssetDatas and which we always keep.
				bKeepDependsNode |= FPackageName::IsScriptPackage(WriteToString<256>(Id.PackageName));
			}
			else if (!Id.PackageName.IsNone() && !Id.ObjectName.IsNone())
			{
				// A DependsNode to an object within a package. Invalid, ignore it.
				bKeepDependsNode = false;
			}
			else
			{
				// Invalid depends node with all fields == NAME_None; remove it
				bKeepDependsNode = false;
			}

			RemapNode = bKeepDependsNode ? DependsNode : nullptr;
		});

	// For pruned nodes, check for redirectors. If the redirection target is not pruned, mark that the pruned node maps
	// to the redirection target. Redirectors have to be searched for before removing any pruned asset datas, because
	// we use that AssetData information to check whether each asset is a redirector.
	TMap<FDependsNode*, FDependsNode*> RedirectCache;
	for (TPair<FDependsNode*, FDependsNode*>& RemapPair : RemapNodes)
	{
		if (!RemapPair.Value)
		{
			FDependsNode* DependsNode = RemapPair.Key;
			FDependsNode* TargetNode = ResolveRedirector(DependsNode, CachedAssets, RedirectCache);
			if (TargetNode != DependsNode)
			{
				bool bKeepTargetNode = (RemapNodes.FindChecked(TargetNode) == TargetNode);
				if (bKeepTargetNode)
				{
					RemapPair.Value = TargetNode;
				}
			}
		}
	}

	// Now that we have finished all our use for the pruned AssetDatas, remove them.
	TArray<FAssetData*> RemovedAssetDatas;
	RemovedAssetDatas.Reserve(NumRemovedAssetDatas);
	for (int32 AssetDataIndex = 0; AssetDataIndex < NumAssetDatas; ++AssetDataIndex)
	{
		if (!KeepAssetDataIndex[AssetDataIndex])
		{
			RemovedAssetDatas.Add(AllAssetData[AssetDataIndex]);
		}
	}
	TBitArray<> UnusedRemoveAssetData;
	TBitArray<> UnusedRemovedDependencyData;
	RemoveAssetDatas(RemovedAssetDatas, false /* bRemoveDependencyData*/, UnusedRemoveAssetData, UnusedRemovedDependencyData);

	TMpscQueue<TPair<FDependsNode*, FDependsNode*>> ReferencersToAdd;
	TMpscQueue<FDependsNode*> PotentialOrphans;

	ParallelFor(MaxId, [&RemapNodes, &ReferencersToAdd, &PotentialOrphans, &Options, &PruneOptions](int32 Index)
		{
			FSetElementId SetId = FSetElementId::FromInteger(Index);
			if (!RemapNodes.IsValidId(SetId))
			{
				return;
			}
			TPair<FDependsNode*, FDependsNode*>& RemapPair = RemapNodes.Get(SetId);
			bool bKeepDependsNode = RemapPair.Key == RemapPair.Value; // kept DependsNodes map to themselves.
			if (bKeepDependsNode) 
			{
				FDependsNode* DependsNode = RemapPair.Key;
				DependsNode->RemapDependencies(
					[DependsNode, &RemapNodes, &ReferencersToAdd, &PotentialOrphans]
					(FDependsNode* ExistingDependsNode)
					{
						FDependsNode* Remapped = RemapNodes.FindChecked(ExistingDependsNode);

						// Remapped == Existing: it's a keeper.
						// Remapped == nullptr: pruned and either not redirected or target is also pruned; remove it.
						if (Remapped != ExistingDependsNode && Remapped != nullptr)
						{
							// Remapped == some other node: Apply the remap.
							// 
							// Since we are adding Remapped to DependsNode->Dependencies, we also need to add
							// DependsNode to Remapped->Referencers; we have to do that after leaving the
							// ParallelFor to avoid data race on Remapped->Referencers.
							// 
							// We don't worry about removing the Referencer from ExistingDependsNode, because we
							// are deleting that node and do not plan to read its Referencers anymore.
							ReferencersToAdd.Enqueue(TPair<FDependsNode*, FDependsNode*>{ Remapped, DependsNode });
						}
						return Remapped;
					});
				DependsNode->RemapReferencers([&RemapNodes](FDependsNode* ExistingDependsNode)
					{
						// Drop references from deleted nodes, no need to handle remapping them.
						FDependsNode* Remapped = RemapNodes.FindChecked(ExistingDependsNode);
						if (Remapped == ExistingDependsNode)
						{
							return ExistingDependsNode;
						}
						return (FDependsNode*)nullptr;
					});

				// Mark for remove any currently orphaned depends nodes. Wait until after ReferencersToAdd is
				// processed to decide whether they are orphaned, since ReferencersToAdd may add links to them.
				if (DependsNode->GetConnectionCount() == 0)
				{
					PotentialOrphans.Enqueue(DependsNode);
				}
			}
		});
	TPair<FDependsNode*, FDependsNode*> ReferencerPair;
	while (ReferencersToAdd.Dequeue(ReferencerPair))
	{
		ReferencerPair.Key->AddReferencer(ReferencerPair.Value);
	}
	FDependsNode* Orphan;
	while (PotentialOrphans.Dequeue(Orphan))
	{
		if (Orphan->GetConnectionCount() == 0)
		{
			RemapNodes.FindChecked(Orphan) = nullptr;
		}
	}

	// Remove all of the marked-for-removal dependsnodes; we have already removed links to them so we don't need to call
	// the full RemoveDependsNode function.
	for (TPair<FDependsNode*, FDependsNode*>& RemapPair : RemapNodes)
	{
		bool bKeepDependsNode = RemapPair.Key == RemapPair.Value;
		if (!bKeepDependsNode)
		{
			FDependsNode* DependsNode = RemapPair.Key;
			CachedDependsNodes.Remove(DependsNode->GetIdentifier());
			NumDependsNodes--;
			// if the depends nodes were preallocated in a block, we can't delete them one at a time,
			// only the whole chunk in the destructor
			if (PreallocatedDependsNodeDataBuffers.Num() == 0)
			{
				delete DependsNode;
			}
			else
			{
				DependsNode->ClearDependencies();
				DependsNode->ClearReferencers();
			}
		}
	}
}

bool FAssetRegistryState::HasAssets(const FName PackagePath, bool bSkipARFilteredAssets) const
{
	bool bHasAssets = false;
	EnumerateAssetsByPackagePath(PackagePath, [this, bSkipARFilteredAssets, &bHasAssets](const FAssetData* AssetData)
		{
			if (AssetData && !IsPackageUnmountedAndFiltered(AssetData->PackageName)
				&& (!bSkipARFilteredAssets
					|| !UE::AssetRegistry::FFiltering::ShouldSkipAsset(
						AssetData->AssetClassPath, AssetData->PackageFlags)))
			{
				bHasAssets = true;
				return false; // Stop iterating
			}
			return true; // keep iterating
		});
	return bHasAssets;
}

bool FAssetRegistryState::GetAssets(const FARCompiledFilter& Filter, const TSet<FName>& PackageNamesToSkip,
	TArray<FAssetData>& OutAssetData, bool bSkipARFilteredAssets) const
{
	const UE::AssetRegistry::EEnumerateAssetsFlags Flags = bSkipARFilteredAssets
		? UE::AssetRegistry::EEnumerateAssetsFlags::None
		: UE::AssetRegistry::EEnumerateAssetsFlags::AllowUnfilteredArAssets;
	return EnumerateAssets(Filter, PackageNamesToSkip, [&OutAssetData](const FAssetData& AssetData)
	{
		OutAssetData.Emplace(AssetData);
		return true;
	},
	Flags);
}

namespace UE::AssetRegistry::Private
{

bool DecideIntersectionMethod(int32 PreviousSize, int32 FilterResultsSize, int32 FilterComplexity)
{
	// Cost of intersection of previous results with new results is the cost to construct a TMap of the smaller
	// set plus the cost to query larger set against the TMap. TMap construction is more expensive than TMap query.
	constexpr uint64 TMapConstructionCost = 3;
	uint64 SmallSize;
	uint64 LargeSize;
	if (PreviousSize < FilterResultsSize)
	{
		SmallSize = (uint64)PreviousSize;
		LargeSize = (uint64)FilterResultsSize;
	}
	else
	{
		SmallSize = (uint64)FilterResultsSize;
		LargeSize = (uint64)PreviousSize;
	}
	uint64 ArrayCost = SmallSize * TMapConstructionCost + LargeSize;
	// Cost of filtering previous results by FilterFunction is filtercomplexity times size of the previous results
	uint64 FilterCost = ((uint64)FilterComplexity) * ((uint64)PreviousSize);

	// Our two sets of cost calculation are not on the same scale; they are off by some factor that is dependent 
	// upon ArrayIntersection code, TMap code, and hardware dependent factors. But we assume they are for on the
	// same scale for simplicity. Despite the invalid assumption, our comparison will still work in the important
	// cases: a large FilterComplexity will use ArrayIntersection and a large FilterResultsSize will use filtering.
	return FilterCost < ArrayCost;
}

void ArrayIntersection(TArray<const FAssetData*>& InOutResults,
	TConstArrayView<TConstArrayView<const FAssetData*>> Matches, int32 TotalMatches)
{
	if (InOutResults.Num() < TotalMatches)
	{
		TMap<const FAssetData*, bool> Exists;
		Exists.Reserve(InOutResults.Num());
		for (const FAssetData* Result : InOutResults)
		{
			Exists.Add(Result, false);
		}
		InOutResults.Empty();
		for (TConstArrayView<const FAssetData*> Assets : Matches)
		{
			for (const FAssetData* Asset : Assets)
			{
				bool* Result = Exists.Find(Asset);
				if (Result && 
					!(*Result) // If there are duplicates of an Asset in multiple elements of Matches,
					           // only add the first one
				)
				{
					*Result = true;
					InOutResults.Add(Asset);
				}
 			}
		}
	}
	else
	{
		TSet<const FAssetData*> Exists;
		Exists.Reserve(TotalMatches);
		for (TConstArrayView<const FAssetData*> Assets : Matches)
		{
			for (const FAssetData* Asset : Assets)
			{
				Exists.Add(Asset);
			}
		}

		InOutResults.RemoveAllSwap([&Exists](const FAssetData* Asset)
			{
				return !Exists.Contains(Asset);
			});
	}
}

#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
void ArrayIntersection(TArray<const FAssetData*>& InOutResults,
	TConstArrayView<TConstArrayView<FAssetDataPtrIndex>> Matches, int32 TotalMatches,
	const FAssetDataMap& CachedAssets)
{
	if (InOutResults.Num() < TotalMatches)
	{
		TMap<const FAssetData*, bool> Exists;
		Exists.Reserve(InOutResults.Num());
		for (const FAssetData* Result : InOutResults)
		{
			Exists.Add(Result, false);
		}
		InOutResults.Empty();
		for (TConstArrayView<FAssetDataPtrIndex> Assets : Matches)
		{
			for (FAssetDataPtrIndex AssetIndex: Assets)
			{
				const FAssetData* Asset = CachedAssets[AssetIndex];
				bool* Result = Exists.Find(Asset);
				if (Result && 
					!(*Result) // If there are duplicates of an Asset in multiple elements of Matches,
					           // only add the first one
				)
				{
					*Result = true;
					InOutResults.Add(Asset);
				}
 			}
		}
	}
	else
	{
		TSet<const FAssetData*> Exists;
		Exists.Reserve(TotalMatches);
		for (TConstArrayView<FAssetDataPtrIndex> Assets : Matches)
		{
			for (FAssetDataPtrIndex AssetIndex : Assets)
			{
				Exists.Add(CachedAssets[AssetIndex]);
			}
		}

		InOutResults.RemoveAllSwap([&Exists](const FAssetData* Asset)
			{
				return !Exists.Contains(Asset);
			});
	}
}
#endif

template<class ArrayType, typename KeyType, typename CallbackType>
void FilterAssets(TArray<const FAssetData*>& InOutResults, const TMap<KeyType, ArrayType>& AccelerationMap,
	const TSet<KeyType>& Keys, CallbackType&& FunctionToKeepAsset, int32 FilterComplexity, const FAssetDataMap& CachedAssets)
{
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	TArray<TConstArrayView<const FAssetData*>, TInlineAllocator<10>> Matches;
#else
	TArray<TConstArrayView<FAssetDataPtrIndex>, TInlineAllocator<10>> Matches;
#endif
	Matches.Reserve(Keys.Num());
	uint32 TotalMatches = 0;

	for (const KeyType& Key : Keys)
	{
		if (const ArrayType* Assets = AccelerationMap.Find(Key))
		{
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
			const FAssetData** AssetPtr = const_cast<const FAssetData**>(Assets->GetData());
			Matches.Add(TConstArrayView<const FAssetData*>(AssetPtr, Assets->Num()));
#else
			FAssetDataPtrIndex* AssetPtr = const_cast<FAssetDataPtrIndex*>(Assets->GetData());
			Matches.Add(TConstArrayView<FAssetDataPtrIndex>(AssetPtr, Assets->Num()));
#endif
			TotalMatches += Assets->Num();
		}
	}

	// Keys is a TSet and entries in the AccelerationMap do not overlap,
	// so there should be no duplicates to remove in Matches
	if (InOutResults.IsEmpty())
	{
		// No previous Results; set Results equal to the values found in AccelerationMap
		InOutResults.Reserve(TotalMatches);
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		for (TConstArrayView<const FAssetData*> Assets : Matches)
		{
			InOutResults.Append(Assets);
		}
#else
		for (TConstArrayView<FAssetDataPtrIndex> Assets : Matches)
		{
			for (FAssetDataPtrIndex AssetIndex : Assets)
			{
				InOutResults.Add(CachedAssets[AssetIndex]);
			}
		}
#endif
	}
	else
	{
		bool bUseFiltering = DecideIntersectionMethod(InOutResults.Num(), TotalMatches, FilterComplexity);
		if (bUseFiltering)
		{
			InOutResults.RemoveAllSwap([&FunctionToKeepAsset](const FAssetData* AssetData)
				{
					return !FunctionToKeepAsset(AssetData);
				});
		}
		else
		{
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
			ArrayIntersection(InOutResults, TConstArrayView<TConstArrayView<const FAssetData*>>(Matches), TotalMatches);
#else
			ArrayIntersection(InOutResults, TConstArrayView<TConstArrayView<FAssetDataPtrIndex>>(Matches), TotalMatches, CachedAssets);
#endif
		}
	}
}

#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
template<typename CallbackType>
void FilterAssetsByPackageName(
	TArray<const FAssetData*>& InOutResults,
	const FAssetPackageNameMap& AccelerationMap,
	const TSet<FName>& Keys, CallbackType&& FunctionToKeepAsset, int32 FilterComplexity,
	const FAssetDataMap& CachedAssets)
{
	TArray<TConstArrayView<FAssetDataPtrIndex>, TInlineAllocator<10>> Matches;
	Matches.Reserve(Keys.Num());
	uint32 TotalMatches = 0;

	for (const FName& Key : Keys)
	{
		if (TOptional<TConstArrayView<FAssetDataPtrIndex>> Array = AccelerationMap.Find(Key))
		{
			Matches.Add(*Array);
			TotalMatches += Array->Num();
		}
	}

	// Keys is a TSet and entries in the AccelerationMap do not overlap,
	// so there should be no duplicates to remove in Matches
	if (InOutResults.IsEmpty())
	{
		// No previous Results; set Results equal to the values found in AccelerationMap
		InOutResults.Reserve(TotalMatches);
		for (TConstArrayView<FAssetDataPtrIndex> Assets : Matches)
		{
			for (FAssetDataPtrIndex AssetIndex : Assets)
			{
				InOutResults.Add(CachedAssets[AssetIndex]);
			}
		}
	}
	else
	{
		bool bUseFiltering = DecideIntersectionMethod(InOutResults.Num(), TotalMatches, FilterComplexity);
		if (bUseFiltering)
		{
			InOutResults.RemoveAllSwap([&FunctionToKeepAsset](const FAssetData* AssetData)
				{
					return !FunctionToKeepAsset(AssetData);
				});
		}
		else
		{
			ArrayIntersection(InOutResults, TConstArrayView<TConstArrayView<FAssetDataPtrIndex>>(Matches), TotalMatches, CachedAssets);
		}
	}
}
#endif

template<typename CallbackType>
void FilterAssets(TArray<const FAssetData*>&InOutResults,
	const UE::AssetRegistry::Private::FAssetDataMap &AccelerationMap, const TSet<FSoftObjectPath>& Keys,
	CallbackType&& FunctionToKeepAsset, int32 FilterComplexity)
{
	TArray<const FAssetData*, TInlineAllocator<10>> Matches;
	Matches.Reserve(Keys.Num());

	for (const FSoftObjectPath& Key : Keys)
	{
		if (FAssetData* const* AssetDataPtr = AccelerationMap.Find(UE::AssetRegistry::Private::FCachedAssetKey(Key)))
		{
			Matches.Add(*AssetDataPtr);
		}
	}

	// Keys is a TSet, so there should be no duplicates to remove in Matches
	if (InOutResults.IsEmpty())
	{
		// No previous Results; set Results equal to the values found in AccelerationMap
		InOutResults = MoveTemp(Matches);
	}
	else
	{
		bool bUseFiltering = DecideIntersectionMethod(InOutResults.Num(), Matches.Num(), FilterComplexity);
		if (bUseFiltering)
		{
			InOutResults.RemoveAllSwap([&FunctionToKeepAsset](const FAssetData* AssetData)
				{
					return !FunctionToKeepAsset(AssetData);
				});
		}
		else
		{
			TConstArrayView<const FAssetData*> ArrayView(Matches);
			TConstArrayView<TConstArrayView<const FAssetData*>> ArrayViewOfArrayViews(&ArrayView, 1);
			ArrayIntersection(InOutResults, ArrayViewOfArrayViews, Matches.Num());
		}
	}
}

bool AssetDataMatchesTag(const FAssetData* AssetData, FName TagName, const TOptional<FString>& TagValue)
{
	if (!AssetData)
	{
		return false;
	}
	if (!TagValue.IsSet())
	{
		return AssetData->TagsAndValues.Contains(TagName);
	}
	else
	{
		return AssetData->TagsAndValues.ContainsKeyValue(TagName, TagValue.GetValue());
	}
}

template<typename CallbackType, typename AccelerationMapType>
void FilterAssets(TArray<const FAssetData*>& InOutResults, const AccelerationMapType& AccelerationMap,
	const TMultiMap<FName, TOptional<FString>>& TagsAndValues,
	CallbackType&& FunctionToKeepAsset, int32 FilterComplexity, const FAssetDataMap& CachedAssets)
{
	// AccelerationMapValueTypePtr is either TArray<FAssetData*>* or TArray<FAssetDataPtrIndex>*
	using AccelerationMapValueTypePtr = decltype(AccelerationMap.Find(FName()));
	struct FMatchData
	{
		FName TagName;
		const TOptional<FString>* TagValuePtr;
		AccelerationMapValueTypePtr AssetsWithTag;
	};

	TArray<FMatchData, TInlineAllocator<10>> Matches;
	Matches.Reserve(TagsAndValues.Num());
	uint32 EstimateOfTotalMatches = 0;

	for (const TPair<FName, TOptional<FString>>& TagPair : TagsAndValues)
	{
		AccelerationMapValueTypePtr AssetsWithTag = AccelerationMap.Find(TagPair.Key);
		if (AssetsWithTag)
		{
			Matches.Add({ TagPair.Key, &TagPair.Value, AssetsWithTag });
			EstimateOfTotalMatches += AssetsWithTag->Num();
		}
	}

	if (InOutResults.IsEmpty())
	{
		// No previous Results; set Results equal to the values found in AccelerationMap
		for (FMatchData& MatchData : Matches)
		{
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
			for (FAssetData* AssetData : *MatchData.AssetsWithTag)
			{
#else
			for (FAssetDataPtrIndex AssetIndex : *MatchData.AssetsWithTag)
			{
				FAssetData* AssetData = CachedAssets[AssetIndex];
#endif
				if (AssetDataMatchesTag(AssetData, MatchData.TagName, *MatchData.TagValuePtr))
				{
					InOutResults.Add(AssetData);
				}
			}
		}
		// Remove duplicates
		Algo::Sort(InOutResults);
		InOutResults.SetNum(Algo::Unique(InOutResults));
	}
	else
	{
		bool bUseFiltering = DecideIntersectionMethod(InOutResults.Num(), EstimateOfTotalMatches, FilterComplexity);
		if (bUseFiltering)
		{
			InOutResults.RemoveAllSwap([&FunctionToKeepAsset](const FAssetData* AssetData)
				{
					return !FunctionToKeepAsset(AssetData);
				});
		}
		else
		{
			TArray<TArray<const FAssetData*>, TInlineAllocator<10>> MatchArrays;
			MatchArrays.Reserve(Matches.Num());

			int32 TotalMatches = 0;
			for (FMatchData& MatchData : Matches)
			{
				TArray<const FAssetData*>& MatchArray = MatchArrays.Emplace_GetRef();
				MatchArray.Reserve(MatchData.AssetsWithTag->Num());
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
				for (FAssetData* AssetData : *MatchData.AssetsWithTag)
				{
#else
				for (FAssetDataPtrIndex AssetIndex : *MatchData.AssetsWithTag)
				{
					FAssetData* AssetData = CachedAssets[AssetIndex];
#endif
					if (AssetDataMatchesTag(AssetData, MatchData.TagName, *MatchData.TagValuePtr))
					{
						MatchArray.Add(AssetData);
						++TotalMatches;
					}
				}
			}

			// Convert Array of Arrays into format required by ArrayIntersection: Array of ArrayViews
			TArray<TConstArrayView<const FAssetData*>, TInlineAllocator<10>> ArrayViewMatches;
			ArrayViewMatches.Reserve(MatchArrays.Num());
			for (TArray<const FAssetData*>& MatchesElement : MatchArrays)
			{
				ArrayViewMatches.Emplace(MatchesElement);
			}

			// ArrayIntersection handles removing any duplicates from Matches
			ArrayIntersection(InOutResults, ArrayViewMatches, TotalMatches);
		}
	}
}

#if !UE_ASSETREGISTRY_CACHEDASSETSBYTAG
template<typename CallbackType>
void FilterAssetsByCachedClassesByTag(TArray<const FAssetData*>& InOutResults,
	const TMap<FName, TSet<FTopLevelAssetPath>>& CachedClassesByTag,
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	const TMap<FTopLevelAssetPath, TArray<FAssetData*>>& CachedAssetsByClass,
#else
	const TMap<FTopLevelAssetPath, TArray<FAssetDataPtrIndex>>& CachedAssetsByClass,
#endif
	const TMultiMap<FName, TOptional<FString>>& TagsAndValues,
	CallbackType&& FunctionToKeepAsset, int32 FilterComplexity,
	const FAssetDataMap& CachedAssets)
{
	TArray<TArray<const FAssetData*>, TInlineAllocator<10>> Matches;
	Matches.Reserve(TagsAndValues.Num());
	uint32 TotalMatches = 0;

	for (const TPair<FName, TOptional<FString>>& TagPair : TagsAndValues)
	{
		TArray<const FAssetData*>& Results = Matches.Emplace_GetRef();

		// The lists of assets in CachedAssetsByClass are non-intersecting (each list is only the exact instances of that
		// class and does not include subclasses), so we do not need to handle removing duplicates when merging lists from
		// multiple classes.
		if (const TSet<FTopLevelAssetPath>* TagClasses = CachedClassesByTag.Find(TagPair.Key))
		{
			for (const FTopLevelAssetPath& ClassPath : *TagClasses)
			{
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
				if (const TArray<FAssetData*>* ClassAssets = CachedAssetsByClass.Find(ClassPath))
				{
					Results.Append(*ClassAssets);
				}
#else
				if (const TArray<FAssetDataPtrIndex>* ClassAssets = CachedAssetsByClass.Find(ClassPath))
				{
					Results.Reserve(Results.Num() + ClassAssets->Num());
					for (FAssetDataPtrIndex Index : *ClassAssets)
					{
						Results.Add(CachedAssets[Index]);
					}
				}
#endif
			}
		}
		// Some assets are in a class that could have the tag, but the specific asset actually does not have the tag.
		for (TArray<const FAssetData*>::TIterator Iter(Results); Iter; ++Iter)
		{
			const FAssetData* AssetData = *Iter;
			if (!AssetDataMatchesTag(AssetData, TagPair.Key, TagPair.Value))
			{
				Iter.RemoveCurrentSwap();
			}
		}
		TotalMatches += Results.Num();
	}

	if (InOutResults.IsEmpty())
	{
		// No previous Results; set Results equal to the values found
		InOutResults.Reserve(TotalMatches);
		for (TArray<const FAssetData*>& Assets : Matches)
		{
			InOutResults.Append(Assets);
		}
		// Remove duplicates
		Algo::Sort(InOutResults);
		InOutResults.SetNum(Algo::Unique(InOutResults));
	}
	else
	{
		bool bUseFiltering = DecideIntersectionMethod(InOutResults.Num(), TotalMatches, FilterComplexity);
		if (bUseFiltering)
		{
			InOutResults.RemoveAllSwap([&FunctionToKeepAsset](const FAssetData* AssetData)
				{
					return !FunctionToKeepAsset(AssetData);
				});
		}
		else
		{
			// Convert Array of Arrays into format required by ArrayIntersection: Array of ArrayViews
			TArray<TConstArrayView<const FAssetData*>, TInlineAllocator<10>> ArrayViewMatches;
			ArrayViewMatches.Reserve(Matches.Num());
			for (TArray<const FAssetData*>& MatchesElement : Matches)
			{
				ArrayViewMatches.Emplace(MatchesElement);
			}

			// ArrayIntersection handles removing any duplicates from Matches
			ArrayIntersection(InOutResults, ArrayViewMatches, TotalMatches);
		}
	}
}
#endif

} // namespace UE::AssetRegistry::Private

bool FAssetRegistryState::EnumerateAssets(const FARCompiledFilter& Filter, const TSet<FName>& PackageNamesToSkip,
	TFunctionRef<bool(const FAssetData&)> Callback, UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags) const
{
	using namespace UE::AssetRegistry::Private;

	// Verify filter input. If all assets are needed, use EnumerateAllAssets() instead.
	if (Filter.IsEmpty() || !IsFilterValid(Filter))
	{
		return false;
	}

	const uint32 FilterWithoutPackageFlags = Filter.WithoutPackageFlags;
	const uint32 FilterWithPackageFlags = Filter.WithPackageFlags;
	auto ShouldSkipAssetData =
		[this, &PackageNamesToSkip, InEnumerateFlags, FilterWithoutPackageFlags, FilterWithPackageFlags]
		(const FAssetData* AssetData)
		{
			if (PackageNamesToSkip.Contains(AssetData->PackageName) |			//-V792
				AssetData->HasAnyPackageFlags(FilterWithoutPackageFlags) |
				!AssetData->HasAllPackageFlags(FilterWithPackageFlags))
			{
				return true;
			}

			if (!EnumHasAnyFlags(InEnumerateFlags, UE::AssetRegistry::EEnumerateAssetsFlags::AllowUnmountedPaths)
				&& IsPackageUnmountedAndFiltered(AssetData->PackageName))
			{
				return true;
			}

			return (!EnumHasAnyFlags(InEnumerateFlags,
				UE::AssetRegistry::EEnumerateAssetsFlags::AllowUnfilteredArAssets)
				&& UE::AssetRegistry::FFiltering::ShouldSkipAsset(AssetData->AssetClassPath, AssetData->PackageFlags));
		};


	// Some of our filters are accelerated: we have TMaps that list for each value of the filter all of the assets
	// that pass that filter. But some of those assets-passing-FilterN-ValueV are very large, and just merging the
	// lists of FAssetData* can be expensive. So for each new filter we need to decide whether it is more expensive to
	// merge previous results with the acceleration list or to apply the filter to every element in previous results.
	// This decision is handled by FilterAssets.
	// To benefit from the filter method we want to have as small a list of results as possible at each step, so
	// order the filters from most-likely to have few results to least-likely to have few results.
	TArray<const FAssetData*> AccumulatedResults;

	if (Filter.SoftObjectPaths.Num() > 0)
	{
		FilterAssets(AccumulatedResults, CachedAssets, Filter.SoftObjectPaths,
			[&Filter](const FAssetData* AssetData)
			{
				return Filter.SoftObjectPaths.Contains(AssetData->GetSoftObjectPath());
			},
			Filter.SoftObjectPaths.Num());
		if (AccumulatedResults.IsEmpty())
		{
			return true;
		}
	}

	if (Filter.PackageNames.Num() > 0)
	{
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		FilterAssets(
#else
		FilterAssetsByPackageName(
#endif
			AccumulatedResults, CachedAssetsByPackageName, Filter.PackageNames,
			[&Filter](const FAssetData* AssetData)
			{
				return Filter.PackageNames.Contains(AssetData->PackageName);
			},
			Filter.PackageNames.Num(), CachedAssets);
		if (AccumulatedResults.IsEmpty())
		{
			return true;
		}
	}

	if (Filter.PackagePaths.Num() > 0)
	{
		FilterAssets(AccumulatedResults, CachedAssetsByPath, Filter.PackagePaths,
			[&Filter](const FAssetData* AssetData)
			{
				return Filter.PackagePaths.Contains(AssetData->PackagePath);
			},
			Filter.PackagePaths.Num(), CachedAssets);
		if (AccumulatedResults.IsEmpty())
		{
			return true;
		}
	}

	if (Filter.TagsAndValues.Num() > 0)
	{
#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
		FilterAssets(AccumulatedResults, CachedAssetsByTag,
#else
		FilterAssetsByCachedClassesByTag(AccumulatedResults, CachedClassesByTag, CachedAssetsByClass,
#endif
			Filter.TagsAndValues,
			[&Filter](const FAssetData* AssetData)
			{
				for (const TPair<FName, TOptional<FString>>& TagPair : Filter.TagsAndValues)
				{
					if (AssetDataMatchesTag(AssetData, TagPair.Key, TagPair.Value))
					{
						return true; // keep
					}
				}
				return false; // remove
			},
			Filter.TagsAndValues.Num(), CachedAssets);
		if (AccumulatedResults.IsEmpty())
		{
			return true;
		}
	}

	if (Filter.ClassPaths.Num() > 0)
	{
		FilterAssets(AccumulatedResults, CachedAssetsByClass, Filter.ClassPaths,
			[&Filter](const FAssetData* AssetData)
			{
				return Filter.ClassPaths.Contains(AssetData->AssetClassPath);
			},
			Filter.ClassPaths.Num(), CachedAssets);
		if (AccumulatedResults.IsEmpty())
		{
			return true;
		}
	}

	// If AccumulatedResults is still empty at this point, then it means that none of the above accelerated filters were set 
	// (as they would have early returned), and so we need to run all assets through the non-accelerated filter
	if (AccumulatedResults.IsEmpty())
	{
		AccumulatedResults.Reserve(CachedAssets.Num());
		for (const FAssetData* AssetData : CachedAssets)
		{
			AccumulatedResults.Add(AssetData);
		}
	}

	// Run the remaining non-accelerated filters on every element of AccumulatedResults

#if WITH_ENGINE && WITH_EDITOR
	// Parallelize the non-accelerated filter. This optimizes large queries coming from e.g. CookByTheBookFinished in the
	// functions FAssetRegistryGenerator::PreSave and GenerateLocalizationReferences through FAssetRegistryImpl::EnumerateDiskAssets.
	
	// Each thread will work on a full cache line to avoid false sharing.
	constexpr int32 ChecksPerCacheLine = PLATFORM_CACHE_LINE_SIZE / sizeof(bool);
	if (IsInGameThread() && AccumulatedResults.Num() >= ChecksPerCacheLine)
	{
		// Ensure UE::AssetRegistry::FFiltering::ShouldSkipAsset is thread safe.
		UE::AssetRegistry::FFiltering::InitializeShouldSkipAsset();

		TArray<bool, TAlignedHeapAllocator<PLATFORM_CACHE_LINE_SIZE>> ShouldSkip;
		ShouldSkip.SetNum(AccumulatedResults.Num());
		
		const int32 CacheLineBatches = FMath::DivideAndRoundUp(AccumulatedResults.Num(), ChecksPerCacheLine);

		ParallelFor(CacheLineBatches, [&ShouldSkipAssetData, &AccumulatedResults, &ShouldSkip](int32 CacheLineIndex)
		{
			const int32 CacheLineOffset = CacheLineIndex * ChecksPerCacheLine;
			for (int32 CheckIndex = 0; CheckIndex < ChecksPerCacheLine; ++CheckIndex)
			{
				const int32 AccumulatedResultsIndex = CacheLineOffset + CheckIndex;
				if (AccumulatedResultsIndex < AccumulatedResults.Num())
				{
					ShouldSkip[AccumulatedResultsIndex] = ShouldSkipAssetData(AccumulatedResults[AccumulatedResultsIndex]);
				}
			}
		});

		for (int32 Index = 0; Index < AccumulatedResults.Num(); ++Index)
		{
			if (ShouldSkip[Index])
			{
				continue;
			}
			const FAssetData* AssetData = AccumulatedResults[Index];
			bool bContinueFiltering = Callback(*AssetData);
			if (!bContinueFiltering)
			{
				return true;
			}
		}
	}
	else
#endif 
	{
		for (const FAssetData* AssetData : AccumulatedResults)
		{
			if (ShouldSkipAssetData(AssetData))
			{
				continue;
			}
			bool bContinueFiltering = Callback(*AssetData);
			if (!bContinueFiltering)
			{
				return true;
			}
		}
	}

	return true;
}

bool FAssetRegistryState::GetAllAssets(const TSet<FName>& PackageNamesToSkip, TArray<FAssetData>& OutAssetData,
	bool bSkipARFilteredAssets) const
{
	const UE::AssetRegistry::EEnumerateAssetsFlags EnumerateFlags = bSkipARFilteredAssets
		? UE::AssetRegistry::EEnumerateAssetsFlags::None
		: UE::AssetRegistry::EEnumerateAssetsFlags::AllowUnfilteredArAssets;
	OutAssetData.Reserve(OutAssetData.Num() + CachedAssets.Num() - PackageNamesToSkip.Num());
	return EnumerateAllAssets(PackageNamesToSkip, [&OutAssetData](const FAssetData& AssetData)
	{
		OutAssetData.Emplace(AssetData);
		return true;
	},
	EnumerateFlags);
}

void FAssetRegistryState::EnumerateAllAssets(TFunctionRef<void(const FAssetData&)> Callback,
	UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags) const
{
	EnumerateAllMutableAssets([&Callback](FAssetData& AssetData)
		{
			Callback(AssetData);
		}, InEnumerateFlags);
}

void FAssetRegistryState::EnumerateAllMutableAssets(TFunctionRef<void(FAssetData&)> Callback,
	UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags) const
{
	using namespace UE::AssetRegistry;

	bool bAllowUnfilteredAssets = EnumHasAnyFlags(InEnumerateFlags, EEnumerateAssetsFlags::AllowUnfilteredArAssets);
	bool bAllowUnmountedPaths = EnumHasAnyFlags(InEnumerateFlags, EEnumerateAssetsFlags::AllowUnmountedPaths);
	if (bAllowUnfilteredAssets && bAllowUnmountedPaths)
	{
		if (EnumHasAnyFlags(InEnumerateFlags, EEnumerateAssetsFlags::Parallel))
		{
			int32 MaxId = CachedAssets.GetMaxIndex();
			ParallelFor(MaxId, [this, &Callback](int32 Index)
				{
					FSetElementId Id = FSetElementId::FromInteger(Index);
					if (CachedAssets.IsValidId(Id))
					{
						FAssetData* AssetData = CachedAssets.Get(Id);
						check(AssetData);
						Callback(*AssetData);
					}
				});
		}
		else
		{
			for (FAssetData* AssetData : CachedAssets)
			{
				check(AssetData);
				Callback(*AssetData);
			}
		}
	}
	else
	{
		if (EnumHasAnyFlags(InEnumerateFlags, EEnumerateAssetsFlags::Parallel))
		{
			int32 MaxId = CachedAssets.GetMaxIndex();
			ParallelFor(MaxId, [this, &Callback, InEnumerateFlags, bAllowUnfilteredAssets, bAllowUnmountedPaths]
			(int32 Index)
				{
					FSetElementId Id = FSetElementId::FromInteger(Index);
					if (CachedAssets.IsValidId(Id))
					{
						FAssetData* AssetData = CachedAssets.Get(Id);
						check(AssetData);
						if ((bAllowUnmountedPaths || !IsPackageUnmountedAndFiltered(AssetData->PackageName))
							&& (bAllowUnfilteredAssets
								|| !FFiltering::ShouldSkipAsset(AssetData->AssetClassPath, AssetData->PackageFlags)))
						{
							Callback(*AssetData);
						}
					}
				});
		}
		else
		{
			for (FAssetData* AssetData : CachedAssets)
			{
				check(AssetData);
				if ((bAllowUnmountedPaths || !IsPackageUnmountedAndFiltered(AssetData->PackageName))
					&& (bAllowUnfilteredAssets
						|| !FFiltering::ShouldSkipAsset(AssetData->AssetClassPath, AssetData->PackageFlags)))
				{
					Callback(*AssetData);
				}
			}
		}
	}
}

bool FAssetRegistryState::EnumerateAllAssets(const TSet<FName>& PackageNamesToSkip,
	TFunctionRef<bool(const FAssetData&)> Callback,
	UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags) const
{
	using namespace UE::AssetRegistry;

	EnumerateAllMutableAssets([&PackageNamesToSkip, &Callback, this](const FAssetData& AssetData)
		{
			if (!PackageNamesToSkip.Contains(AssetData.PackageName))
			{
				if (!Callback(AssetData))
				{
					return false; // Stop iterating
				}
			}
			return true; // keep iterating
		}, InEnumerateFlags);
	return true;
}

void FAssetRegistryState::EnumerateAllPaths(TFunctionRef<void(FName PathName)> Callback) const
{
	for (const auto& Pair : CachedAssetsByPath)
	{
		Callback(Pair.Key);
	}
}

void FAssetRegistryState::GetPackagesByName(FStringView PackageName, TArray<FName>& OutPackageNames) const
{
	// Note that we use CachedAssetsByPackageName rather than CachedPackageData because CachedPackageData
	// is often stripped out of the runtime AssetRegistry
	if (!FPackageName::IsShortPackageName(PackageName))
	{
		FName PackageFName(PackageName);
		if (CachedAssetsByPackageName.Contains(PackageFName))
		{
			OutPackageNames.Add(PackageFName);
		}
	}
	else
	{
		TStringBuilder<128> PackageNameStr;
		for (const auto& It : CachedAssetsByPackageName)
		{
			It.Key.ToString(PackageNameStr);
			FStringView ExistingBaseName = FPathViews::GetBaseFilename(PackageNameStr);
			if (ExistingBaseName.Equals(PackageName, ESearchCase::IgnoreCase))
			{
				OutPackageNames.Add(It.Key);
			}
		}
	}
}

FName FAssetRegistryState::GetFirstPackageByName(FStringView PackageName) const
{
	TArray<FName> LongPackageNames;
	GetPackagesByName(PackageName, LongPackageNames);
	if (LongPackageNames.Num() == 0)
	{
		return NAME_None;
	}
	if (LongPackageNames.Num() > 1)
	{
		LongPackageNames.Sort(FNameLexicalLess());
		UE_LOGF(LogAssetRegistry, Warning,
			"GetFirstPackageByName('%.*ls') is returning '%ls', but it also found '%ls'%ls.",
			PackageName.Len(), PackageName.GetData(), *LongPackageNames[0].ToString(), *LongPackageNames[1].ToString(),
			(LongPackageNames.Num() > 2 ? *FString::Printf(TEXT(" and %d others"), LongPackageNames.Num() - 2) : TEXT("")));
	}
	return LongPackageNames[0];
}

bool FAssetRegistryState::GetDependencies(const FAssetIdentifier& AssetIdentifier,
	TArray<FAssetIdentifier>& OutDependencies,
	UE::AssetRegistry::EDependencyCategory Category, const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	const FDependsNode* const* NodePtr = CachedDependsNodes.Find(AssetIdentifier);
	const FDependsNode* Node = nullptr;
	if (NodePtr != nullptr)
	{
		Node = *NodePtr;
	}

	if (Node != nullptr)
	{
		Node->GetDependencies(OutDependencies, Category, Flags);
		return true;
	}
	else
	{
		return false;
	}
}

bool FAssetRegistryState::GetDependencies(const FAssetIdentifier& AssetIdentifier,
	TArray<FAssetDependency>& OutDependencies,
	UE::AssetRegistry::EDependencyCategory Category, const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	const FDependsNode* const* NodePtr = CachedDependsNodes.Find(AssetIdentifier);
	const FDependsNode* Node = nullptr;
	if (NodePtr != nullptr)
	{
		Node = *NodePtr;
	}

	if (Node != nullptr)
	{
		Node->GetDependencies(OutDependencies, Category, Flags);
		return true;
	}
	else
	{
		return false;
	}
}

bool FAssetRegistryState::ContainsDependency(const FAssetIdentifier& AssetIdentifier,
	const FAssetIdentifier& QueryAsset,
	UE::AssetRegistry::EDependencyCategory Category, const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	const FDependsNode* const* NodePtr = CachedDependsNodes.Find(AssetIdentifier);
	const FDependsNode* const* QueryNodePtr = CachedDependsNodes.Find(QueryAsset);
	if (!NodePtr || !QueryNodePtr)
	{
		return false;
	}

	return (*NodePtr)->ContainsDependency(*QueryNodePtr, Category, Flags);
}

bool FAssetRegistryState::GetReferencers(const FAssetIdentifier& AssetIdentifier,
	TArray<FAssetIdentifier>& OutReferencers,
	UE::AssetRegistry::EDependencyCategory Category, const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	const FDependsNode* const* NodePtr = CachedDependsNodes.Find(AssetIdentifier);
	const FDependsNode* Node = nullptr;

	if (NodePtr != nullptr)
	{
		Node = *NodePtr;
	}

	if (Node != nullptr)
	{
		TArray<FDependsNode*> DependencyNodes;
		Node->GetReferencers(DependencyNodes, Category, Flags);

		OutReferencers.Reserve(OutReferencers.Num() + DependencyNodes.Num());
		for (FDependsNode* DependencyNode : DependencyNodes)
		{
			OutReferencers.Add(DependencyNode->GetIdentifier());
		}

		return true;
	}
	else
	{
		return false;
	}
}

bool FAssetRegistryState::GetReferencers(const FAssetIdentifier& AssetIdentifier,
	TArray<FAssetDependency>& OutReferencers,
	UE::AssetRegistry::EDependencyCategory Category, const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	const FDependsNode* const* NodePtr = CachedDependsNodes.Find(AssetIdentifier);
	const FDependsNode* Node = nullptr;

	if (NodePtr != nullptr)
	{
		Node = *NodePtr;
	}

	if (Node != nullptr)
	{
		Node->GetReferencers(OutReferencers, Category, Flags);
		return true;
	}
	else
	{
		return false;
	}
}

void FAssetRegistryState::ClearDependencies(const FAssetIdentifier& AssetIdentifier,
	UE::AssetRegistry::EDependencyCategory Category)
{
	FDependsNode* ReferencerNode = FindDependsNode(AssetIdentifier);
	if (!ReferencerNode)
	{
		return;
	}

	TArray<FDependsNode*> OldDependencies;
	ReferencerNode->GetDependencies(OldDependencies);
	ReferencerNode->ClearDependencies(Category);

	for (FDependsNode* DependencyNode : OldDependencies)
	{
		if (!ReferencerNode->ContainsDependency(DependencyNode))
		{
			DependencyNode->RemoveReferencer(ReferencerNode);
		}
	}
}

void FAssetRegistryState::AddDependencies(const FAssetIdentifier& AssetIdentifier,
	TConstArrayView<FAssetDependency> Dependencies)
{
	if (Dependencies.IsEmpty())
	{
		return;
	}
	FDependsNode* ReferencerNode = CreateOrFindDependsNode(AssetIdentifier);
	for (const FAssetDependency& Dependency : Dependencies)
	{
		FDependsNode* DependencyNode = CreateOrFindDependsNode(Dependency.AssetId);
		ReferencerNode->AddDependency(DependencyNode, Dependency.Category, Dependency.Properties);
		DependencyNode->AddReferencer(ReferencerNode);
	}
}

void FAssetRegistryState::SetDependencies(const FAssetIdentifier& AssetIdentifier,
	TConstArrayView<FAssetDependency> Dependencies, UE::AssetRegistry::EDependencyCategory Category)
{
	for (const FAssetDependency& Dependency : Dependencies)
	{
		checkf(!(Dependency.Category & ~Category),
			TEXT("Input dependency has category %d which is outside of the requested categories %d."),
			(int32)Dependency.Category, (int32)Category);
	}

	ClearDependencies(AssetIdentifier, Category);
	AddDependencies(AssetIdentifier, Dependencies);
}

void FAssetRegistryState::ClearReferencers(const FAssetIdentifier& AssetIdentifier,
	UE::AssetRegistry::EDependencyCategory Category)
{
	FDependsNode* DependencyNode = FindDependsNode(AssetIdentifier);
	if (!DependencyNode)
	{
		return;
	}

	TArray<FDependsNode*> OldExisting;
	DependencyNode->GetReferencers(OldExisting, Category);
	for (FDependsNode* ReferencerNode : OldExisting)
	{
		ReferencerNode->RemoveDependency(DependencyNode, Category);
		if (!ReferencerNode->ContainsDependency(DependencyNode))
		{
			DependencyNode->RemoveReferencer(ReferencerNode);
		}
	}
}

void FAssetRegistryState::AddReferencers(const FAssetIdentifier& AssetIdentifier,
	TConstArrayView<FAssetDependency> Referencers)
{
	if (Referencers.IsEmpty())
	{
		return;
	}
	FDependsNode* DependencyNode = CreateOrFindDependsNode(AssetIdentifier);
	for (const FAssetDependency& Referencer : Referencers)
	{
		FDependsNode* ReferencerNode = CreateOrFindDependsNode(Referencer.AssetId);
		ReferencerNode->AddDependency(DependencyNode, Referencer.Category, Referencer.Properties);
		DependencyNode->AddReferencer(ReferencerNode);
	}
}

void FAssetRegistryState::SetReferencers(const FAssetIdentifier& AssetIdentifier,
	TConstArrayView<FAssetDependency> Referencers, UE::AssetRegistry::EDependencyCategory Category)
{
	for (const FAssetDependency& Referencer : Referencers)
	{
		checkf(!(Referencer.Category & ~Category),
			TEXT("Input referencer has category %d which is outside of the requested categories %d."),
			(int32)Referencer.Category, (int32)Category);
	}

	ClearReferencers(AssetIdentifier, Category);
	AddReferencers(AssetIdentifier, Referencers);
}

bool FAssetRegistryState::Serialize(FArchive& Ar, const FAssetRegistrySerializationOptions& Options)
{
	return Ar.IsSaving() ? Save(Ar, Options) : Load(Ar, FAssetRegistryLoadOptions(Options));
}

static UTF8CHAR* ToLowerStrNDup(UE::FLinearBlockAllocatorThreadAccessor& Allocator, const UTF8CHAR* Src, int32 Len)
{
	if (!Src || Len == 0)
	{
		return nullptr;
	}

	UTF8CHAR* Dest = reinterpret_cast<UTF8CHAR*>(Allocator.Malloc((Len + 1) * sizeof(UTF8CHAR), alignof(UTF8CHAR)));
	for (int32 N = 0; N < Len; ++N)
	{
		Dest[N] = TChar<UTF8CHAR>::ToLower(Src[N]);
	}
	Dest[Len] = '\0';
	return Dest;
}

TArray<FAssetData*> FAssetRegistryState::GetMutableAssetsSortedByObjectPath()
{
	struct FSortData
	{
		FAssetData* AssetData;
		// Components of AssetData.GetSoftObjectPath, separated for faster sorting
		UTF8CHAR* PackageName;
		FName AssetName;
		UTF8CHAR* SubPath;
	};

	TArray<FSortData> SortDatas;
	SortDatas.Reserve(CachedAssets.Num());
	EnumerateAllMutableAssets([&SortDatas](FAssetData& AssetData)
		{
			SortDatas.Emplace_GetRef().AssetData = &AssetData;
		});

	UE::FLinearBlockAllocator SharedAllocator(nullptr, 0U, ESPMode::NotThreadSafe);
	TArray<UE::FLinearBlockAllocatorThreadAccessor> ThreadAllocators;
	// Set minbatchsize to avoid wasting memory by having threads allocate a 64K block and then not use most of it
	constexpr int32 MinSetupBatchSize = 500;
	int32 CachedAssetsMaxIndex = CachedAssets.GetMaxIndex();
	ParallelForWithTaskContext(TEXT("GetMutableAssetsSortedByObjectPath"), ThreadAllocators, SortDatas.Num(),
		MinSetupBatchSize,
		// ContextConstructor
		[&SharedAllocator](int32 ThreadIndex, int32 NumThreads) -> UE::FLinearBlockAllocator&
		{
			// ParallelFor passes return value into the FLinearBlockAllocatorThreadAccessor constructor.
			return SharedAllocator;
		},
		// Body
		[this, &SortDatas](UE::FLinearBlockAllocatorThreadAccessor& Allocator, int32 Index)
		{
			FSortData& Data = SortDatas[Index];
			FSoftObjectPath AssetId = Data.AssetData->GetSoftObjectPath();

			TUtf8StringBuilder<1024> PackageName(InPlace, AssetId.GetLongPackageFName());
			Data.PackageName = ToLowerStrNDup(Allocator, PackageName.GetData(), PackageName.Len());
			Data.AssetName = AssetId.GetAssetFName();
			const FUtf8String& SubPath = AssetId.GetSubPathUtf8String();
			Data.SubPath = ToLowerStrNDup(Allocator, GetData(SubPath), SubPath.Len());
		}
	);

	auto CompareSortData = [](const FSortData& A, const FSortData& B)
		{
			int32 PackageNameDiff = TCString<UTF8CHAR>::Strcmp(A.PackageName, B.PackageName);
			if (PackageNameDiff != 0)
			{
				return PackageNameDiff < 0;
			}
			int32 AssetNameDiff = A.AssetName.Compare(B.AssetName);
			if (AssetNameDiff != 0)
			{
				return AssetNameDiff < 0;
			}
			if ((A.SubPath == nullptr) != (B.SubPath == nullptr))
			{
				return A.SubPath == nullptr;
			}
			if (A.SubPath)
			{
				int32 SubPathDiff = TCString<UTF8CHAR>::Strcmp(A.SubPath, B.SubPath);
				if (SubPathDiff != 0)
				{
					return SubPathDiff < 0;
				}
			}
			return false;
		};
	Algo::ParallelSort(SortDatas, CompareSortData);

	TArray<FAssetData*> Result;
	Result.Reserve(SortDatas.Num());
	for (const FSortData& SortData : SortDatas)
	{
		Result.Add(SortData.AssetData);
	}
	return Result;
}

bool FAssetRegistryState::Save(FArchive& OriginalAr, const FAssetRegistrySerializationOptions& Options)
{
	SCOPED_BOOT_TIMING("FAssetRegistryState::Save");

	check(!OriginalAr.IsLoading());

#if !ALLOW_NAME_BATCH_SAVING
	checkf(false, TEXT("Cannot save cooked AssetRegistryState in this configuration"));
#else
	check(CachedAssets.Num() == NumAssets);

	FAssetRegistryHeader Header;
	Header.Version = FAssetRegistryVersion::LatestVersion;
	Header.bFilterEditorOnlyData = OriginalAr.IsFilterEditorOnly();
	Header.SerializeHeader(OriginalAr);

	// Set up fixed asset registry writer
	FAssetRegistryWriter Ar(FAssetRegistryWriterOptions(Options), OriginalAr);

	// serialize number of objects
	int32 AssetCount = CachedAssets.Num();
	Ar << AssetCount;

	// Write asset data first
	{
		TArray<FAssetData*> SortedAssetsByObjectPath = GetMutableAssetsSortedByObjectPath();
		for (FAssetData* Asset : SortedAssetsByObjectPath)
		{
			// Hardcoding FAssetRegistryVersion::LatestVersion here so that branches can get optimized out in
			// the forceinlined SerializeForCache
			Asset->SerializeForCache(Ar);
		}
	}

	// Serialize Dependencies
	// Write placeholder data for the size
	int64 OffsetToDependencySectionSize = Ar.Tell();
	int64 DependencySectionSize = 0;
	Ar << DependencySectionSize;
	int64 DependencySectionStart = Ar.Tell();
	if (!Options.bSerializeDependencies)
	{
		int32 NumDependencies = 0;
		Ar << NumDependencies;
	}
	else
	{
		TArray<FDependsNode*> Dependencies;

		// Scan dependency nodes, we won't save all of them if we filter out certain types
		for (TPair<FAssetIdentifier, FDependsNode*>& Pair : CachedDependsNodes)
		{
			FDependsNode* Node = Pair.Value;

			if (Node->GetIdentifier().IsPackage() 
				|| (Options.bSerializeSearchableNameDependencies && Node->GetIdentifier().IsValue())
				|| (Options.bSerializeManageDependencies && Node->GetIdentifier().GetPrimaryAssetId().IsValid()))
			{
				Dependencies.Add(Node);
			}
		}
		Algo::Sort(Dependencies, [](FDependsNode* A, FDependsNode* B) 
			{
				return A->GetIdentifier().LexicalLess(B->GetIdentifier());
			});
		int32 NumDependencies = Dependencies.Num();

		TMap<FDependsNode*, int32> DependsIndexMap;
		DependsIndexMap.Reserve(NumDependencies);
		int32 Index = 0;
		for (FDependsNode* Node : Dependencies)
		{
			DependsIndexMap.Add(Node, Index++);
		}

		TUniqueFunction<int32(FDependsNode*, bool bAsReferencer)> GetSerializeIndexFromNode =
			[this, &DependsIndexMap](FDependsNode* InDependency, bool bAsReferencer)
		{
			if (!InDependency)
			{
				return -1;
			}
			int32* DependencyIndex = DependsIndexMap.Find(InDependency);
			if (!DependencyIndex)
			{
				return -1;
			}
			return *DependencyIndex;
		};

		FDependsNode::FSaveScratch Scratch;
		Ar << NumDependencies;
		for (FDependsNode* DependentNode : Dependencies)
		{
			DependentNode->SerializeSave(Ar, GetSerializeIndexFromNode, Scratch, Options);
		}
	}
	// Write the real value to the placeholder data for the DependencySectionSize
	int64 DependencySectionEnd = Ar.Tell();
	DependencySectionSize = DependencySectionEnd - DependencySectionStart;
	Ar.Seek(OffsetToDependencySectionSize);
	Ar << DependencySectionSize;
	check(Ar.Tell() == DependencySectionStart);
	Ar.Seek(DependencySectionEnd);


	// Serialize the PackageData
	int32 PackageDataCount = 0;
	if (Options.bSerializePackageData)
	{
		PackageDataCount = CachedPackageData.Num();
		Ar << PackageDataCount;

		TArray<TPair<FName, FAssetPackageData*>> SortedPackageData = CachedPackageData.Array();
		Algo::Sort(SortedPackageData, [](TPair<FName, FAssetPackageData*>& A, TPair<FName, FAssetPackageData*>& B)
			{
				return A.Key.LexicalLess(B.Key);
			});
		for (TPair<FName, FAssetPackageData*>& Pair : SortedPackageData)
		{
			Ar << Pair.Key;
			Pair.Value->SerializeForCache(Ar);
		}
	}
	else
	{
		Ar << PackageDataCount;
	}
#endif // ALLOW_NAME_BATCH_SAVING

	return !OriginalAr.IsError();
}

bool FAssetRegistryState::Load(FArchive& OriginalAr, const FAssetRegistryLoadOptions& Options,
	FAssetRegistryVersion::Type* OutVersion)
{
	FAssetRegistryLoadResults Results;
	bool bResult = Load(OriginalAr, Options, Results);
	if (OutVersion)
	{
		*OutVersion = Results.Version;
	}
	return bResult;
}

bool FAssetRegistryState::Load(FArchive& OriginalAr, const FAssetRegistryLoadOptions& Options,
	FAssetRegistryLoadResults& OutResults)
{
	LLM_SCOPE(ELLMTag::AssetRegistry);
	FAssetRegistryHeader Header;
	Header.SerializeHeader(OriginalAr);
	OutResults.Version = Header.Version;
	OutResults.Store = FixedTagPrivate::FWeakStorePtr();

	FSoftObjectPathSerializationScope SerializationScope(NAME_None, NAME_None,
		ESoftObjectPathCollectType::NonPackage, ESoftObjectPathSerializeType::AlwaysSerialize);

	if (Header.Version < FAssetRegistryVersion::RemovedMD5Hash)
	{
		// Cannot read states before this version
		return false;
	}
	else if (Header.Version < FAssetRegistryVersion::FixedTags)
	{
		FNameTableArchiveReader NameTableReader(OriginalAr);
		Load(NameTableReader, Header, Options);
	}
	else
	{
		FAssetRegistryReader Reader(OriginalAr, nullptr /* BackingFile */,
			Options.Store, Options.ParallelWorkers, Header);

		if (Reader.IsError())
		{
			return false;
		}

		// Load won't resolve asset registry tag values loaded in parallel
		// and can run before WaitForTasks
		Load(Reader, Header, Options);

		Reader.WaitForTasks();
		OutResults.Store = FixedTagPrivate::FWeakStorePtr(Reader.GetTags());
	}

	return !OriginalAr.IsError();
}

/* static */ bool FAssetRegistryState::LoadFromDisk(const TCHAR* InPath, const FAssetRegistryLoadOptions& InOptions,
	FAssetRegistryState& OutState, FAssetRegistryVersion::Type* OutVersion)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAssetRegistryState::LoadFromDisk");
	check(InPath);

	TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(InPath));
	if (FileReader)
	{
		// It's faster to load the whole file into memory on a Gen5 console
		TArray64<uint8> Data;
		Data.SetNumUninitialized(FileReader->TotalSize());
		FileReader->Serialize(Data.GetData(), Data.Num());
		check(!FileReader->IsError());

		FLargeMemoryReader MemoryReader(Data.GetData(), Data.Num());
		return OutState.Load(MemoryReader, InOptions, OutVersion);
	}

	return false;
}

template<class Archive>
void FAssetRegistryState::Load(Archive&& Ar, const FAssetRegistryHeader& Header,
	const FAssetRegistryLoadOptions& Options)
{
	FAssetRegistryVersion::Type Version = Header.Version;

	// serialize number of objects
	int32 LocalNumAssets = 0;
	Ar << LocalNumAssets;

	// allocate one single block for all asset data structs (to reduce tens of thousands of heap allocations)
	TArray<FAssetData*> AllocatedAssets;
	if (!Options.bMountPointsRequireSeparateAllocationBlocks)
	{
		AllocatedAssets.SetNumUninitialized(LocalNumAssets);
		FAssetData* PreallocatedAssetDataBuffer = new FAssetData[LocalNumAssets];
		PreallocatedAssetDataBuffers.Add(PreallocatedAssetDataBuffer);
		for (int32 Index = 0; Index < LocalNumAssets; ++Index)
		{
			AllocatedAssets[Index] = &PreallocatedAssetDataBuffer[Index];
		}
	}
	else
	{
		AllocatedAssets.SetNumZeroed(LocalNumAssets);
	}

	for (FAssetData*& NewAssetData : AllocatedAssets)
	{
		if (!NewAssetData)
		{
			NewAssetData = new FAssetData();
		}
		// Optimizing serialization of latest asset data format by moving version checking out of SerializeForCache
		// function and falling back to versioned serialization should we attempt to load an older version of AR
		// (usually commandlets)
		if (Version == FAssetRegistryVersion::LatestVersion)
		{
			NewAssetData->SerializeForCache(Ar);
		}
		else
		{
			NewAssetData->SerializeForCacheOldVersion(Ar, Version);
		}

		if (Options.ShouldKeepPackage && !Options.ShouldKeepPackage(NewAssetData->PackageName))
		{
			if (!PreallocatedAssetDataBuffers.Num())
			{
				delete NewAssetData;
			}
			NewAssetData = nullptr;
		}
	}
	SetAssetDatas(AllocatedAssets, Options);

	LoadDependencies(Ar, Options, Version);

	int32 LocalNumPackageData = 0;
	Ar << LocalNumPackageData;

	if (LocalNumPackageData > 0)
	{
		FAssetPackageData SerializedElement;
		FAssetPackageData* PreallocatedPackageDataBuffer = nullptr;
		if (Options.bLoadPackageData)
		{
			CachedPackageData.Reserve(LocalNumPackageData);
			if (!Options.bMountPointsRequireSeparateAllocationBlocks)
			{
				PreallocatedPackageDataBuffer = new FAssetPackageData[LocalNumPackageData];
				PreallocatedPackageDataBuffers.Add(PreallocatedPackageDataBuffer);
			}
		}

		for (int32 PackageDataIndex = 0; PackageDataIndex < LocalNumPackageData; PackageDataIndex++)
		{
			FName PackageName;
			Ar << PackageName;
			FAssetPackageData* NewPackageData = nullptr;
			if (!Options.bLoadPackageData ||
				(Options.ShouldKeepPackage && !Options.ShouldKeepPackage(PackageName)))
			{
				NewPackageData = &SerializedElement;
			}
			else
			{
				if (!Options.bMountPointsRequireSeparateAllocationBlocks)
				{
					NewPackageData = &PreallocatedPackageDataBuffer[PackageDataIndex];
				}
				else
				{
					NewPackageData = new FAssetPackageData();
				}
				CachedPackageData.Add(PackageName, NewPackageData);
				++NumPackageData;
			}
			check(NewPackageData);

			if (Version >= FAssetRegistryVersion::LatestVersion)
			{
				NewPackageData->SerializeForCache(Ar);
			}
			else
			{
				NewPackageData->SerializeForCacheOldVersion(Ar, Version);
			}
		}
	}
}

void FAssetRegistryState::LoadDependencies(FArchive& Ar, const FAssetRegistryLoadOptions& Options,
	FAssetRegistryVersion::Type Version)
{
	if (Version < FAssetRegistryVersion::AddedDependencyFlags)
	{
		// ShouldKeepPackage is not supported in this legacy loading path
		check(!Options.ShouldKeepPackage);
		// bMountPointsRequireSeparateAllocationBlocks is not supported in this legacy loading path
		check(!Options.bMountPointsRequireSeparateAllocationBlocks);
		LegacyLoadDependencies_BeforeFlags(Ar, Options.bLoadDependencies, Version);
	}
	else
	{
		int64 DependencySectionSize;
		Ar << DependencySectionSize;
		int64 DependencySectionEnd = Ar.Tell() + DependencySectionSize;

#if ASSET_REGISTRY_ALLOW_DEPENDENCY_SERIALIZATION
		if (Options.bLoadDependencies)
		{
			if (Version == FAssetRegistryVersion::LatestVersion)
			{
				LoadDependenciesImpl<true>(Ar, Options, Version);
			}
			else
			{
				LoadDependenciesImpl<false>(Ar, Options, Version);
			}
		}

		if (!Options.bLoadDependencies || Ar.IsError())
		{
			Ar.Seek(DependencySectionEnd);
		}
#else
		Ar.Seek(DependencySectionEnd);
#endif
	}
}

template <bool bLatestVersion>
void FAssetRegistryState::LoadDependenciesImpl(FArchive& Ar, const FAssetRegistryLoadOptions& Options,
	FAssetRegistryVersion::Type Version)
{
	int32 LocalNumDependsNodes = 0;
	Ar << LocalNumDependsNodes;

	if (LocalNumDependsNodes <= 0)
	{
		return;
	}

	TArray<FDependsNode*> AllocatedNodes;
	AllocatedNodes.SetNumUninitialized(LocalNumDependsNodes);
	if (!Options.bMountPointsRequireSeparateAllocationBlocks)
	{
		FDependsNode* PreallocatedDependsNodeDataBuffer = new FDependsNode[LocalNumDependsNodes];
		PreallocatedDependsNodeDataBuffers.Add(PreallocatedDependsNodeDataBuffer);
		for (int32 Index = 0; Index < LocalNumDependsNodes; ++Index)
		{
			AllocatedNodes[Index] = &PreallocatedDependsNodeDataBuffer[Index];
		}
	}
	else
	{
		for (FDependsNode*& DependsNode : AllocatedNodes)
		{
			DependsNode = new FDependsNode();
		}
	}
	CachedDependsNodes.Reserve(LocalNumDependsNodes);

	TUniqueFunction<FDependsNode*(int32)> GetNodeFromSerializeIndex =
		[&AllocatedNodes, LocalNumDependsNodes](int32 Index) -> FDependsNode*
	{
		if (Index < 0 || LocalNumDependsNodes <= Index)
		{
			return nullptr;
		}
		return AllocatedNodes[Index];
	};

	TSet<FDependsNode*> RemoveNodes;

	FDependsNode::FLoadScratch Scratch;
	for (int32 DependsNodeIndex = 0; DependsNodeIndex < LocalNumDependsNodes; DependsNodeIndex++)
	{
		FDependsNode* DependsNode = AllocatedNodes[DependsNodeIndex];
		DependsNode->SerializeLoad<bLatestVersion>(Ar, GetNodeFromSerializeIndex, Scratch, Version);
		if (Options.ShouldKeepPackage && !Options.ShouldKeepPackage(DependsNode->GetPackageName()))
		{
			RemoveNodes.Add(DependsNode);
		}
		else
		{
			CachedDependsNodes.Add(DependsNode->GetIdentifier(), DependsNode);
			++NumDependsNodes;
		}
	}

	if (!RemoveNodes.IsEmpty())
	{
		for (FDependsNode* RemoveNode : RemoveNodes)
		{
			RemoveNode->IterateOverDependencies([&RemoveNodes, RemoveNode](FDependsNode* Dependency,
				UE::AssetRegistry::EDependencyCategory Category,
				UE::AssetRegistry::EDependencyProperty Properties,
				bool bDuplicate)
				{
					if (!bDuplicate && !RemoveNodes.Contains(Dependency))
					{
						Dependency->RemoveReferencer(RemoveNode);
					}
				});
			RemoveNode->IterateOverReferencers([&RemoveNodes, RemoveNode](FDependsNode* Referencer)
				{
					if (!RemoveNodes.Contains(Referencer))
					{
						Referencer->RemoveDependency(RemoveNode);
					}
				});
			RemoveNode->Empty();
		}
		if (Options.bMountPointsRequireSeparateAllocationBlocks)
		{
			for (FDependsNode* RemoveNode : RemoveNodes)
			{
				delete RemoveNode;
			}
		}
	}
}

void FAssetRegistryState::LegacyLoadDependencies_BeforeFlags(FArchive& Ar, bool bSerializeDependencies,
	FAssetRegistryVersion::Type Version)
{
	int32 LocalNumDependsNodes = 0;
	Ar << LocalNumDependsNodes;

	FDependsNode Placeholder;
	FDependsNode* PreallocatedDependsNodeDataBuffer = nullptr;
	if (bSerializeDependencies && LocalNumDependsNodes > 0)
	{
		PreallocatedDependsNodeDataBuffer = new FDependsNode[LocalNumDependsNodes];
		PreallocatedDependsNodeDataBuffers.Add(PreallocatedDependsNodeDataBuffer);
		CachedDependsNodes.Reserve(LocalNumDependsNodes);
	}
	TUniqueFunction<FDependsNode* (int32)> GetNodeFromSerializeIndex =
		[&PreallocatedDependsNodeDataBuffer, LocalNumDependsNodes](int32 Index)->FDependsNode *
	{
		if (Index < 0 || LocalNumDependsNodes <= Index)
		{
			return nullptr;
		}
		return &PreallocatedDependsNodeDataBuffer[Index];
	};

	TArray<FDependsNode*> DependsNodes;
	for (int32 DependsNodeIndex = 0; DependsNodeIndex < LocalNumDependsNodes; DependsNodeIndex++)
	{
		// Create the node if we're actually saving dependencies, otherwise just fake serialize
		FDependsNode* DependsNode = nullptr;
		if (bSerializeDependencies)
		{
			DependsNode = &PreallocatedDependsNodeDataBuffer[DependsNodeIndex];
		}
		else
		{
			DependsNode = &Placeholder;
		}

		// Call the DependsNode legacy serialization function
		DependsNode->LegacySerializeLoad_BeforeFlags(Ar, Version, PreallocatedDependsNodeDataBuffer,
			LocalNumDependsNodes, bSerializeDependencies);

		// Register the DependsNode with its AssetIdentifier
		if (bSerializeDependencies)
		{
			CachedDependsNodes.Add(DependsNode->GetIdentifier(), DependsNode);
		}
	}
}

SIZE_T FAssetRegistryState::GetAllocatedSize(bool bLogDetailed) const
{
	SIZE_T MapMemory = CachedAssets.GetAllocatedSize();
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	MapMemory += IndirectAssetDataArrays.GetAllocatedSize();
#endif
	MapMemory += CachedAssetsByPackageName.GetAllocatedSize();
	MapMemory += CachedAssetsByPath.GetAllocatedSize();
	MapMemory += CachedAssetsByClass.GetAllocatedSize();
#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
	MapMemory += CachedAssetsByTag.GetAllocatedSize();
#else
	MapMemory += CachedClassesByTag.GetAllocatedSize();
#endif
	MapMemory += CachedDependsNodes.GetAllocatedSize();
	MapMemory += CachedPackageData.GetAllocatedSize();
	MapMemory += PreallocatedAssetDataBuffers.GetAllocatedSize();
	MapMemory += PreallocatedDependsNodeDataBuffers.GetAllocatedSize();
	MapMemory += PreallocatedPackageDataBuffers.GetAllocatedSize();

	SIZE_T MapArrayMemory = 0;
	auto SubArray = 
		[&MapArrayMemory](const auto& A)
	{
		for (auto& Pair : A)
		{
			MapArrayMemory += Pair.Value.GetAllocatedSize();
		}
	};
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	SubArray(CachedAssetsByPackageName);
#endif
	SubArray(CachedAssetsByPath);

	for (auto& Pair : CachedAssetsByClass)
	{
		MapArrayMemory += Pair.Value.GetAllocatedSize();
	}

#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
	SubArray(CachedAssetsByTag);
#else
	SubArray(CachedClassesByTag);
#endif

	if (bLogDetailed)
	{
		UE_LOG(LogAssetRegistry, Log, TEXT("Index Size: %" SIZE_T_FMT "k"), MapMemory / 1024);
	}

	SIZE_T AssetDataSize = 0;
	SIZE_T AssetBundlesSize = 0;
	SIZE_T NumAssetBundles = 0;
	SIZE_T NumSoftObjectPaths = 0;
	SIZE_T NumTopLevelAssetPaths = 0;
	FAssetDataTagMapSharedView::FMemoryCounter TagMemoryUsage;

	EnumerateAllAssets(
		[&AssetDataSize, &TagMemoryUsage, &NumAssetBundles, &NumSoftObjectPaths, &AssetBundlesSize,
		&NumTopLevelAssetPaths]
		(const FAssetData& AssetData)
	{
		AssetDataSize += sizeof(AssetData);
		TagMemoryUsage.Include(AssetData.TagsAndValues);
		if (AssetData.TaggedAssetBundles.IsValid())
		{
			AssetBundlesSize += sizeof(FAssetBundleData);
			AssetBundlesSize += AssetData.TaggedAssetBundles->Bundles.GetAllocatedSize();
			NumAssetBundles += AssetData.TaggedAssetBundles->Bundles.Num();
			for (const FAssetBundleEntry& Entry : AssetData.TaggedAssetBundles->Bundles)
			{
#if WITH_EDITORONLY_DATA
				PRAGMA_DISABLE_DEPRECATION_WARNINGS;
				AssetBundlesSize += Entry.BundleAssets.GetAllocatedSize();
				NumSoftObjectPaths += Entry.BundleAssets.Num();
				for (const FSoftObjectPath& Path : Entry.BundleAssets)
				{
					AssetBundlesSize += Path.GetSubPathString().GetAllocatedSize();
				}
				PRAGMA_ENABLE_DEPRECATION_WARNINGS;
#endif
				AssetBundlesSize += Entry.AssetPaths.GetAllocatedSize();
				NumTopLevelAssetPaths += Entry.AssetPaths.Num();
			}
		}
	});

	if (bLogDetailed)
	{
		UE_LOGF(LogAssetRegistry, Log, "AssetData Count: %d", CachedAssets.Num());
		UE_LOG(LogAssetRegistry, Log, TEXT("AssetData Static Size: %" SIZE_T_FMT "k"), AssetDataSize / 1024);
		UE_LOG(LogAssetRegistry, Log, TEXT("Loose Tags: %" SIZE_T_FMT "k"), TagMemoryUsage.GetLooseSize() / 1024);
		TagMemoryUsage.ReportFixedStoreBreakdown();
		UE_LOG(LogAssetRegistry, Log, TEXT("Fixed Tags: %" SIZE_T_FMT "k"), TagMemoryUsage.GetFixedSize() / 1024);
		UE_LOG(LogAssetRegistry, Log, TEXT("TArray<FAssetData*>: %" SIZE_T_FMT "k"), MapArrayMemory / 1024);
		UE_LOG(LogAssetRegistry, Log, TEXT("AssetBundle Count: %" SIZE_T_FMT ), NumAssetBundles);
		UE_LOG(LogAssetRegistry, Log, TEXT("AssetBundle Size: %" SIZE_T_FMT "k"), AssetBundlesSize / 1024);
		UE_LOG(LogAssetRegistry, Log, TEXT("AssetBundle FSoftObjectPath Count: %" SIZE_T_FMT), NumSoftObjectPaths);
		UE_LOG(LogAssetRegistry, Log, TEXT("AssetBundle FTopLevelAssetPath Count: %" SIZE_T_FMT), NumTopLevelAssetPaths);
	}

	SIZE_T DependNodesSize = 0, DependenciesSize = 0;

	for (const TPair<FAssetIdentifier, FDependsNode*>& DependsNodePair : CachedDependsNodes)
	{
		const FDependsNode& DependsNode = *DependsNodePair.Value;
		DependNodesSize += sizeof(DependsNode);

		DependenciesSize += DependsNode.GetAllocatedSize();
	}

	if (bLogDetailed)
	{
		UE_LOGF(LogAssetRegistry, Log, "Dependency Node Count: %d", CachedDependsNodes.Num());
		UE_LOG(LogAssetRegistry, Log, TEXT("Dependency Node Static Size: %" SIZE_T_FMT "k"), DependNodesSize / 1024);
		UE_LOG(LogAssetRegistry, Log, TEXT("Dependency Arrays Size: %" SIZE_T_FMT "k"), DependenciesSize / 1024);
	}

	SIZE_T PackageDataSize = CachedPackageData.Num() * (sizeof(FAssetPackageData) + sizeof(FAssetPackageData*));
	for (const TPair<FName, FAssetPackageData*>& PackageDataPair : CachedPackageData)
	{
		PackageDataSize += PackageDataPair.Value->GetAllocatedSize();
	}

	SIZE_T TotalBytes = MapMemory + AssetDataSize + AssetBundlesSize + TagMemoryUsage.GetFixedSize()
		+ TagMemoryUsage.GetLooseSize() + DependNodesSize + DependenciesSize + PackageDataSize + MapArrayMemory;

	if (bLogDetailed)
	{
		UE_LOGF(LogAssetRegistry, Log, "PackageData Count: %d", CachedPackageData.Num());
		UE_LOG(LogAssetRegistry, Log, TEXT("PackageData Static Size: %" SIZE_T_FMT "k"), PackageDataSize / 1024);
		UE_LOG(LogAssetRegistry, Log, TEXT("Total State Size: %" SIZE_T_FMT "k"), TotalBytes / 1024);
	}

	return TotalBytes;
}

FDependsNode* FAssetRegistryState::ResolveRedirector(FDependsNode* InDependency,
													 const FAssetDataMap& InAllowedAssets,
													 TMap<FDependsNode*, FDependsNode*>& InCache) const
{
	if (InCache.Contains(InDependency))
	{
		return InCache[InDependency];
	}

	FDependsNode* CurrentDependency = InDependency;
	FDependsNode* Result = nullptr;

	TSet<FName> EncounteredDependencies;

	while (Result == nullptr)
	{
		checkSlow(CurrentDependency);

		if (EncounteredDependencies.Contains(CurrentDependency->GetPackageName()))
		{
			break;
		}

		EncounteredDependencies.Add(CurrentDependency->GetPackageName());

		if (CachedAssetsByPackageName.Contains(CurrentDependency->GetPackageName()))
		{
			// Get the list of assets contained in this package
			EnumerateAssetsByPackageName(CurrentDependency->GetPackageName(), 
				[&CurrentDependency, &InAllowedAssets, &Result, this](const FAssetData* Asset)
			{
				if (Asset->IsRedirector())
				{
					FDependsNode* ChainedRedirector = nullptr;
					// This asset is a redirector, so we want to look at its dependencies and find the asset that
					// it is redirecting to
					CurrentDependency->IterateOverDependencies(
						[&InAllowedAssets, &ChainedRedirector, &Result, this](FDependsNode* InDepends,
						UE::AssetRegistry::EDependencyCategory Category,
						UE::AssetRegistry::EDependencyProperty Property, bool bDuplicate)
					{
						if (bDuplicate)
						{
							return; // Already looked at this dependency node
						}
						const FAssetIdentifier& AssetId = InDepends->GetIdentifier();
						FSoftObjectPath AssetPath(FTopLevelAssetPath(AssetId.PackageName, AssetId.ObjectName), FString());
						if (InAllowedAssets.Contains(FCachedAssetKey(AssetPath)))
						{
							// This asset is in the allowed asset list, so take this as the redirect target
							Result = InDepends;
						}
						else if (CachedAssetsByPackageName.Contains(InDepends->GetPackageName()))
						{
							// This dependency isn't in the allowed list, but it is a valid asset in the registry.
							// Because this is a redirector, this should mean that the redirector is pointing at
							// ANOTHER redirector (or itself in some horrible situations) so we'll move to that node
							// and try again
							ChainedRedirector = InDepends;
						}
					}, UE::AssetRegistry::EDependencyCategory::Package);

					if (ChainedRedirector)
					{
						CurrentDependency = ChainedRedirector;
						return false; // Found a redirector, stop iterating assets for the current package
					}
				}
				else
				{
					Result = CurrentDependency;
				}

				if (Result)
				{
					// We found an allowed asset from the original dependency node. We're finished!
					return false; // stop iterating assets for the current package
				}
				return true; // keep iterating assets for the current package
			});
		}
		else
		{
			Result = CurrentDependency;
		}
	}

	InCache.Add(InDependency, Result);
	return Result;
}

template <typename KeyType, typename ValueType>
void ShrinkMultimap(TMap<KeyType, TArray<ValueType>>& Map)
{
	Map.Shrink();
	for (auto& Pair : Map)
	{
		Pair.Value.Shrink();
	}
};

void FAssetRegistryState::SetAssetDatas(TConstArrayView<FAssetData*> AssetDatas, const FAssetRegistryLoadOptions& Options)
{
	using namespace UE::AssetRegistry::Private;

	UE_CLOGF(NumAssets != 0, LogAssetRegistry, Fatal,
		"Can only load into empty asset registry states. Load into temporary and append using InitializeFromExisting() instead.");

	for (FAssetData* AssetData : AssetDatas)
	{
		if (AssetData)
		{
			++NumAssets;
		}
	}
	
	auto SetObjectPathCache = [this, &AssetDatas]()
	{
		CachedAssets.Empty(NumAssets);
		for (FAssetData* AssetData : AssetDatas)
		{
			if (AssetData)
			{
				CachedAssets.Add(AssetData);
			}
		}
		ensure(NumAssets == CachedAssets.Num());
	};

	// FAssetDatas sharing package name are very rare.
	// Reserve up front and don't bother shrinking. 
	auto SetPackageNameCache = [this, &AssetDatas]()
	{
		CachedAssetsByPackageName.Empty(AssetDatas.Num());
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		for (FAssetData* AssetData : AssetDatas)
		{
			if (AssetData)
			{
				CachedAssetsByPackageName.FindOrAdd(AssetData->PackageName).Add(AssetData);
			}
		}
#else
		CachedAssets.Enumerate([this](FAssetData& AssetData, FAssetDataPtrIndex AssetIndex)
			{
				CachedAssetsByPackageName.Add(AssetData.PackageName, AssetIndex);
				return true;
			});
#endif
	};

	auto SetPackagePathCache = [this, &AssetDatas]()
	{
		CachedAssetsByPath.Empty();
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		for (FAssetData* AssetData : AssetDatas)
		{
			if (AssetData)
			{
				CachedAssetsByPath.FindOrAdd(AssetData->PackagePath).Add(AssetData);
			}
		}
#else
		CachedAssets.Enumerate([this](FAssetData& AssetData, FAssetDataPtrIndex AssetIndex)
			{
				CachedAssetsByPath.FindOrAdd(AssetData.PackagePath).Add(AssetIndex);
				return true;
			});
#endif
		ShrinkMultimap(CachedAssetsByPath);
	};

	auto SetClassAndTagCaches = [this, &AssetDatas]()
	{
		CachedAssetsByClass.Empty();
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		for (FAssetData* AssetData : AssetDatas)
		{
			if (AssetData)
			{
				CachedAssetsByClass.FindOrAdd(AssetData->AssetClassPath).Add(AssetData);
			}
		}
#else
		CachedAssets.Enumerate([this](FAssetData& AssetData, FAssetDataPtrIndex AssetIndex)
			{
				CachedAssetsByClass.FindOrAdd(AssetData.AssetClassPath).Add(AssetIndex);
				return true;
			});
#endif
		ShrinkMultimap(CachedAssetsByClass);

#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
		CachedAssetsByTag.Empty();
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		for (FAssetData* AssetData : AssetDatas)
		{
			if (AssetData)
			{
				for (const TPair<FName, FAssetTagValueRef>& Pair : AssetData->TagsAndValues)
				{
					CachedAssetsByTag.FindOrAdd(Pair.Key).Add(AssetData);
				}
			}
		}
#else
		CachedAssets.Enumerate([this](FAssetData& AssetData, FAssetDataPtrIndex AssetIndex)
			{
				for (const TPair<FName, FAssetTagValueRef>& Pair : AssetData.TagsAndValues)
				{
					CachedAssetsByTag.FindOrAdd(Pair.Key).Add(AssetIndex);
				}
				return true;
			});
#endif
		CachedAssetsByTag.Shrink();
		for (auto& Pair : CachedAssetsByTag)
		{
			Pair.Value.Shrink();
		}
#else
		CachedClassesByTag.Empty();
		for (FAssetData* AssetData : AssetDatas)
		{
			if (AssetData)
			{
				for (const TPair<FName, FAssetTagValueRef>& Pair : AssetData->TagsAndValues)
				{
					CachedClassesByTag.FindOrAdd(Pair.Key).Add(AssetData->AssetClassPath);
				}
			}
		}

		CachedClassesByTag.Shrink();
		for (TPair<FName, TSet<FTopLevelAssetPath>>& Pair : CachedClassesByTag)
		{
			Pair.Value.Shrink();
		}
#endif
	};

	if (Options.ParallelWorkers <= 1)
	{
		SetObjectPathCache();
		SetPackageNameCache();
		SetPackagePathCache();
		SetClassAndTagCaches();
	}
	else
	{
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		TFuture<void> Task1 = Async(EAsyncExecution::TaskGraph, [&SetObjectPathCache]() { SetObjectPathCache(); });
		TFuture<void> Task2 = Async(EAsyncExecution::TaskGraph, [&SetPackageNameCache]() { SetPackageNameCache(); });
		SetPackagePathCache();
		SetClassAndTagCaches();
		Task1.Wait();
		Task2.Wait();
#else
		SetObjectPathCache();
		TFuture<void> Task1 = Async(EAsyncExecution::TaskGraph, [&SetPackagePathCache]() { SetPackagePathCache(); });
		TFuture<void> Task2 = Async(EAsyncExecution::TaskGraph, [&SetPackageNameCache]() { SetPackageNameCache(); });
		SetClassAndTagCaches();
		Task1.Wait();
		Task2.Wait();
#endif
	}
}

void FAssetRegistryState::AddAssetData(FAssetData* AssetData)
{
	using namespace UE::AssetRegistry::Private;

	bool bAlreadyInSet = false;
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	CachedAssets.Add(AssetData, &bAlreadyInSet);
	FAssetData* MapElement = AssetData;
#else
	FAssetDataPtrIndex MapElement = CachedAssets.Add(AssetData, &bAlreadyInSet);
#endif
	if (bAlreadyInSet)
	{
		UE_LOGF(LogAssetRegistry, Error, "AddAssetData called with ObjectPath %ls which already exists. "
			"This will overwrite and leak the existing AssetData.", *FCachedAssetKey(*AssetData).ToString());
	}
	else
	{
		++NumAssets;
	}

#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	CachedAssetsByPackageName.FindOrAdd(AssetData->PackageName).Add(MapElement);
#else
	CachedAssetsByPackageName.Add(AssetData->PackageName, MapElement);
#endif
	CachedAssetsByPath.FindOrAdd(AssetData->PackagePath).Add(MapElement);
	CachedAssetsByClass.FindOrAdd(AssetData->AssetClassPath).Add(MapElement);

	for (auto TagIt = AssetData->TagsAndValues.CreateConstIterator(); TagIt; ++TagIt)
	{
		FName Key = TagIt.Key();

#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
		CachedAssetsByTag.FindOrAdd(Key).Add(MapElement);
#else
		CachedClassesByTag.FindOrAdd(Key).Add(AssetData->AssetClassPath);
#endif
	}
}

void FAssetRegistryState::AddTagsToAssetData(const FSoftObjectPath& InObjectPath, FAssetDataTagMap&& InTagsAndValues)
{
	using namespace UE::AssetRegistry::Private;

	if (InTagsAndValues.IsEmpty())
	{
		return;
	}

#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	FSetElementId Id = CachedAssets.FindId(FCachedAssetKey(InObjectPath));
	if (!Id.IsValidId())
#else
	FAssetDataPtrIndex Id = CachedAssets.FindId(FCachedAssetKey(InObjectPath));
	if (Id == AssetDataPtrIndexInvalid)
#endif
	{
		UE_LOGF(LogAssetRegistry, Warning,
			"AddTagsToAssetData called with asset data that doesn't exist! Tags not added. ObjectPath: %ls",
			*InObjectPath.ToString());
		return;
	}
	FAssetData* AssetData = CachedAssets[Id];
	FAssetDataTagMap Tags = AssetData->TagsAndValues.CopyMap();
	Tags.Append(MoveTemp(InTagsAndValues));
	SetTagsOnExistingAsset(AssetData, MoveTemp(Tags));
}


void FAssetRegistryState::FilterTags(const FAssetRegistrySerializationOptions& Options)
{
	using namespace UE::AssetRegistry::Private;

	// Calling SetTagsOnExistingAsset for any changed tags might be slow.
	// For cases where many Assets change it might be faster to recreate CachedAssetsByTag/CachedClassesByTag rather
	// than trying to update its elements for each Asset change. For that reason we (currently) always recreate
	// CachedAssetsByTag/CachedClassesByTag.
#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
	for (auto& Pair : CachedAssetsByTag)
	{
		Pair.Value.Reset();
	}
#else
	for (TPair<FName, TSet<FTopLevelAssetPath>>& Pair : CachedClassesByTag)
	{
		Pair.Value.Reset();
	}
#endif

#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	for (FAssetData* AssetDataPtr : CachedAssets)
	{
		FAssetData& AssetData = *AssetDataPtr;
		FAssetData* AssetIndex = AssetDataPtr;
#else
	CachedAssets.Enumerate([this, &Options](FAssetData& AssetData, FAssetDataPtrIndex AssetIndex)
	{
#endif
		FilterTags(AssetData, Options);
		// Add the AssetData to all its CachedAssetsByTag/CachedClassesByTag keys even if FilterTags did not change
		// anything, because we are reconstructing all CachedAssetsByTag.
		for (auto TagIt = AssetData.TagsAndValues.CreateConstIterator(); TagIt; ++TagIt)
		{
#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
			CachedAssetsByTag.FindOrAdd(TagIt.Key()).Add(AssetIndex);
#else
			CachedClassesByTag.FindOrAdd(TagIt.Key()).Add(AssetData.AssetClassPath);
#endif
		}
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	}
#else
		return true;
	});
#endif
}

void FAssetRegistryState::FilterTags(FAssetData& InOutAssetData, const FAssetRegistrySerializationOptions& Options)
{
	FAssetDataTagMap LocalTagsAndValues;
	FilterTags(InOutAssetData.TagsAndValues, LocalTagsAndValues,
		Options.CookFilterlistTagsByClass.Find(InOutAssetData.AssetClassPath), Options);
	if (LocalTagsAndValues != InOutAssetData.TagsAndValues)
	{
		InOutAssetData.TagsAndValues = FAssetDataTagMapSharedView(MoveTemp(LocalTagsAndValues));
	}
	InOutAssetData.TaggedAssetBundles = FAssetRegistryState::FilterBundles(InOutAssetData.TaggedAssetBundles, Options);
}

void FAssetRegistryState::SetTagsOnExistingAsset(FAssetData* AssetData, FAssetDataTagMap&& NewTags)
{
	// Update the tag cache map to remove deleted tags
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	FAssetData* AssetIndex = AssetData;
#else
	FAssetDataPtrIndex AssetIndex = CachedAssets.FindId(FCachedAssetKey(AssetData));
#endif
	for (auto TagIt = AssetData->TagsAndValues.CreateConstIterator(); TagIt; ++TagIt)
	{
		const FName FNameKey = TagIt.Key();

		if (!NewTags.Contains(FNameKey))
		{
#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
			auto* OldTagAssets = CachedAssetsByTag.Find(FNameKey);
			if (OldTagAssets)
			{
				OldTagAssets->Remove(AssetIndex);
			}
#else
			// For CachedClassesByTag, we do not need to remove the asset's class from the entries
			// for the old tags. The class is not changed and still has the possibility of containing the tags.
#endif
		}
	}
	// Update the tag cache map to add added tags
	for (auto TagIt = NewTags.CreateConstIterator(); TagIt; ++TagIt)
	{
		const FName FNameKey = TagIt.Key();

		if (!AssetData->TagsAndValues.Contains(FNameKey))
		{
#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
			CachedAssetsByTag.FindOrAdd(FNameKey).Add(AssetIndex);
#else
			CachedClassesByTag.FindOrAdd(FNameKey).Add(AssetData->AssetClassPath);
#endif
		}
	}
	AssetData->TagsAndValues = FAssetDataTagMapSharedView(MoveTemp(NewTags));
}

void FAssetRegistryState::SetDependencyNodeSorting(bool bSortDependencies, bool bSortReferencers)
{
	for (TPair<FAssetIdentifier, FDependsNode*>& Pair : CachedDependsNodes)
	{
		FDependsNode* DependsNode = Pair.Value;
		const bool bOldAllowShrinking = DependsNode->GetAllowShrinking();
		DependsNode->SetAllowShrinking(true);
		DependsNode->SetIsDependencyListSorted(UE::AssetRegistry::EDependencyCategory::All, bSortDependencies);
		DependsNode->SetIsReferencersSorted(bSortReferencers);
		DependsNode->SetAllowShrinking(bOldAllowShrinking);
	}
}

void FAssetRegistryState::UpdateAssetData(const FAssetData& NewAssetData, bool bCreateIfNotExists)
{
	FAssetData* AssetData = GetMutableAssetByObjectPath(FCachedAssetKey(NewAssetData));
	if (AssetData)
	{
		UpdateAssetData(AssetData, NewAssetData);
	}
	else if (bCreateIfNotExists)
	{
		AddAssetData(new FAssetData(NewAssetData));
	}
}

void FAssetRegistryState::UpdateAssetData(FAssetData&& NewAssetData, bool bCreateIfNotExists)
{
	FAssetData* AssetData = GetMutableAssetByObjectPath(FCachedAssetKey(NewAssetData));
	if (AssetData)
	{
		UpdateAssetData(AssetData, MoveTemp(NewAssetData));
	}
	else if (bCreateIfNotExists)
	{
		AddAssetData(new FAssetData(MoveTemp(NewAssetData)));
	}
}

void FAssetRegistryState::UpdateAssetData(FAssetData* AssetData, const FAssetData& NewAssetData, bool* bOutModified)
{
	UpdateAssetData(AssetData, FAssetData(NewAssetData), bOutModified);
}

void FAssetRegistryState::UpdateAssetData(FAssetData* AssetData, FAssetData&& NewAssetData, bool* bOutModified)
{
	using namespace UE::AssetRegistry::Private;

	bool bKeyFieldIsModified = false;
	FCachedAssetKey OldKey(AssetData);
	FCachedAssetKey NewKey(NewAssetData);

#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	FAssetData* AssetIndex = AssetData;
#else
	FAssetDataPtrIndex AssetIndex = CachedAssets.FindId(OldKey);
	check(AssetIndex != AssetDataPtrIndexInvalid);
#endif

	// Update ObjectPath
	if (OldKey != NewKey)
	{
		bKeyFieldIsModified = true;
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		int32 NumRemoved = CachedAssets.Remove(OldKey);
#else
		int32 NumRemoved = CachedAssets.RemoveOnlyKeyLookup(OldKey);
#endif
		check(NumRemoved <= 1);
		if (NumRemoved == 0)
		{
			UE_LOGF(LogAssetRegistry, Error,
				"UpdateAssetData called on AssetData %ls that is not present in the AssetRegistry.",
				*AssetData->GetObjectPathString());
		}
		NumAssets -= NumRemoved;
	}

	// Update PackageName
	if (AssetData->PackageName != NewAssetData.PackageName)
	{
		bKeyFieldIsModified = true;
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		CachedAssetsByPackageName.Find(AssetData->PackageName)->Remove(AssetIndex);
		CachedAssetsByPackageName.FindOrAdd(NewAssetData.PackageName).Add(AssetIndex);
#else
		CachedAssetsByPackageName.Remove(AssetData->PackageName, AssetIndex);
		CachedAssetsByPackageName.Add(NewAssetData.PackageName, AssetIndex);
#endif
	}

	// Update PackagePath
	if (AssetData->PackagePath != NewAssetData.PackagePath)
	{
		bKeyFieldIsModified = true;

		CachedAssetsByPath.Find(AssetData->PackagePath)->Remove(AssetIndex);
		CachedAssetsByPath.FindOrAdd(NewAssetData.PackagePath).Add(AssetIndex);
	}

	// AssetName is not a keyfield; compared below

	// Update AssetClass
	if (AssetData->AssetClassPath != NewAssetData.AssetClassPath)
	{
		bKeyFieldIsModified = true;

		CachedAssetsByClass.Find(AssetData->AssetClassPath)->Remove(AssetIndex);
		CachedAssetsByClass.FindOrAdd(NewAssetData.AssetClassPath).Add(AssetIndex);
	}

	// PackageFlags is not a keyfield; compared below

	// Update Tags
	if (AssetData->TagsAndValues != NewAssetData.TagsAndValues)
	{ 
		bKeyFieldIsModified = true;
		for (auto TagIt = AssetData->TagsAndValues.CreateConstIterator(); TagIt; ++TagIt)
		{
			const FName FNameKey = TagIt.Key();

			if (!NewAssetData.TagsAndValues.Contains(FNameKey))
			{
#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
				CachedAssetsByTag.Find(FNameKey)->Remove(AssetIndex);
#else
				// For CachedClassesByTag, we do not need to remove the asset's class from the entries
				// for the old tags. The class is not changed and still has the possibility of containing the tags.
#endif
			}
		}

		for (auto TagIt = NewAssetData.TagsAndValues.CreateConstIterator(); TagIt; ++TagIt)
		{
			const FName FNameKey = TagIt.Key();

			if (!AssetData->TagsAndValues.Contains(FNameKey))
			{
#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
				CachedAssetsByTag.FindOrAdd(FNameKey).Add(AssetIndex);
#else
				CachedClassesByTag.FindOrAdd(FNameKey).Add(AssetData->AssetClassPath);
#endif
			}
		}
	}

	// TaggedAssetBundles is not a keyfield; compared below

	// ChunkIds is not a keyfield; compared below

	if (bOutModified)
	{
		// Computing equality is expensive; if the caller needs to know it, do cheap compares first 
		// so we can skip the more expensive compares if the inequality is already known
		// This is not possible for keyfields - we have to take action on those even if inequality is already known -
		// so we start with whether bKeyFieldIsModified
		*bOutModified = bKeyFieldIsModified ||
			AssetData->AssetName != NewAssetData.AssetName ||
			AssetData->PackageFlags != NewAssetData.PackageFlags ||
			!AssetData->HasSameChunkIDs(NewAssetData) ||
			(AssetData->TaggedAssetBundles.IsValid() != NewAssetData.TaggedAssetBundles.IsValid() ||
				(AssetData->TaggedAssetBundles.IsValid() &&
					// First check whether the pointers are the same
					AssetData->TaggedAssetBundles.Get() != NewAssetData.TaggedAssetBundles.Get()
					// If the pointers differ, check whether the contents differ
					&& *AssetData->TaggedAssetBundles != *NewAssetData.TaggedAssetBundles
					));
	}

	// Copy in new values
	*AssetData = MoveTemp(NewAssetData);

	// Can only re-add to asset map after we update the key fields, because those change the hashvalue in CachedAssets
	if (OldKey != NewKey)
	{
		bool bExisting = false;
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		CachedAssets.Add(AssetData, &bExisting);
#else
		CachedAssets.AddKeyLookup(AssetData, AssetIndex, &bExisting);
#endif
		if (bExisting)
		{
			UE_LOGF(LogAssetRegistry, Error,
				"UpdateAssetData called with a change in ObjectPath from Old=\"%ls\" to New=\"%ls\", " "but the new ObjectPath is already present with another AssetData. This will overwrite and leak the existing AssetData.",
				*OldKey.ToString(), *NewKey.ToString());
}
		else
		{
			++NumAssets;
		}
	}
}

bool FAssetRegistryState::UpdateAssetDataPackageFlags(FName PackageName, uint32 PackageFlags)
{
	bool bFoundValue = false;
	EnumerateMutableAssetsByPackageName(PackageName, [&bFoundValue, PackageFlags](FAssetData* AssetData)
		{
			AssetData->PackageFlags = PackageFlags;
			bFoundValue = true;
			return true;
		});
	return bFoundValue;
}

void FAssetRegistryState::RemoveAssetData(FAssetData* AssetData, bool bRemoveDependencyData,
	bool& bOutRemovedAssetData, bool& bOutRemovedPackageData)
{
	TBitArray<> bOutRemovedAssetDatas(false, 1);
	TBitArray<> bOutRemovedPackageDatas(false, 1);
	RemoveAssetDatas({ &AssetData, 1 }, bRemoveDependencyData, bOutRemovedAssetDatas, bOutRemovedPackageDatas);
	bOutRemovedAssetData = bOutRemovedAssetDatas[0];
	bOutRemovedPackageData = bOutRemovedPackageDatas[0];
}

void FAssetRegistryState::RemoveAssetDatas(TArrayView<FAssetData*> AssetDatas, bool bRemoveDependencyData,
	TBitArray<>& bOutRemovedAssetData, TBitArray<>& bOutRemovedPackageData)
{
	RemoveAssetDatasImpl(AssetDatas, bRemoveDependencyData, bOutRemovedAssetData, bOutRemovedPackageData);
}

template <typename BitArrayAllocator>
void FAssetRegistryState::RemoveAssetDatasImpl(TArrayView<FAssetData*> AssetDatas, bool bRemoveDependencyData,
	TBitArray<BitArrayAllocator>& bOutRemovedAssetDatas, TBitArray<BitArrayAllocator>& bOutRemovedPackageDatas)
{
	using namespace UE::AssetRegistry::Private;

	bOutRemovedAssetDatas.SetNum(AssetDatas.Num(), false);
	bOutRemovedPackageDatas.SetNum(AssetDatas.Num(), false);

	TArray<FCachedAssetKey, TInlineAllocator<1>> AssetKeys;
	AssetKeys.Reserve(AssetDatas.Num());
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	TArray<FAssetDataPtrIndex, TInlineAllocator<1>> AssetIndices;
	AssetIndices.Reserve(AssetDatas.Num());
#endif
	for (FAssetData* AssetData : AssetDatas)
	{
		FCachedAssetKey& AssetKey = AssetKeys.Emplace_GetRef(AssetData);
		(void) AssetKey; // Avoid unused variable warnings when not using UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		AssetIndices.Add(CachedAssets.FindId(AssetKey));
#endif
	}

	RemoveAssetDatasImpl(AssetDatas, AssetKeys, bRemoveDependencyData,
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		AssetIndices,
#endif
		bOutRemovedAssetDatas, bOutRemovedPackageDatas
	);

	for (TBitArray<>::FConstIterator It(bOutRemovedAssetDatas); It; ++It)
	{
		UE_CLOGF(!It.GetValue(), LogAssetRegistry, Error,
			"RemoveAssetData called on AssetData %ls that is not present in the AssetRegistry.",
			*FCachedAssetKey(*AssetDatas[It.GetIndex()]).ToString());
	}
}

void FAssetRegistryState::RemoveAssetData(const FSoftObjectPath& AssetPath, bool bRemoveDependencyData,
	bool& bOutRemovedAssetData, bool& bOutRemovedPackageData)
{
	using namespace UE::AssetRegistry::Private;

	FCachedAssetKey Key(AssetPath);
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	FAssetData** AssetDataPtrPtr = CachedAssets.Find(Key);
	FAssetData* AssetData = AssetDataPtrPtr ? *AssetDataPtrPtr : nullptr;
#else
	FAssetDataPtrIndex AssetIndex = CachedAssets.FindId(Key);
	FAssetData* AssetData = AssetIndex != AssetDataPtrIndexInvalid ? CachedAssets[AssetIndex] : nullptr;
#endif
	if (!AssetData)
	{
		bOutRemovedAssetData = false;
		bOutRemovedPackageData = false;
		return;
	}
	RemoveAssetData(AssetData, Key, bRemoveDependencyData,
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		AssetIndex,
#endif
		bOutRemovedAssetData, bOutRemovedPackageData
	);
}

void FAssetRegistryState::RemoveAssetData(FAssetData* AssetData, const FCachedAssetKey& Key,
	bool bRemoveDependencyData,
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	UE::AssetRegistry::Private::FAssetDataPtrIndex AssetIndex,
#endif
	bool& bOutRemovedAssetData, bool& bOutRemovedPackageData
)
{
	TBitArray<> bOutRemovedAssetDatas(false, 1);
	TBitArray<> bOutRemovedPackageDatas(false, 1);
	RemoveAssetDatasImpl({ &AssetData, 1 }, { &Key, 1 }, bRemoveDependencyData,
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		{ &AssetIndex, 1 },
#endif
		bOutRemovedAssetDatas, bOutRemovedPackageDatas
	);
	bOutRemovedAssetData = bOutRemovedAssetDatas[0];
	bOutRemovedPackageData = bOutRemovedPackageDatas[0];
}

template <typename BitArrayAllocator>
void FAssetRegistryState::RemoveAssetDatasImpl(TArrayView<FAssetData*> AssetDatas, TArrayView<const FCachedAssetKey> Keys, 
	bool bRemoveDependencyData,
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	TArrayView<UE::AssetRegistry::Private::FAssetDataPtrIndex> AssetIndices,
#endif
	TBitArray<BitArrayAllocator>& bOutRemovedAssetDatas, TBitArray<BitArrayAllocator>& bOutRemovedPackageDatas
)
{
	using namespace UE::AssetRegistry::Private;
	using namespace UE::Tasks;

	auto RemoveFromPackageAssets = 
		[this, AssetDatas, Keys, bRemoveDependencyData, &bOutRemovedPackageDatas
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		,AssetIndices
#endif
		]()
		{
			TSet<FAssetIdentifier, DefaultKeyFuncs<FAssetIdentifier>, TInlineSetAllocator<1>> PackagesToRemove;
			PackagesToRemove.Reserve(Keys.Num());

			for (int32 Index = 0; Index < Keys.Num(); Index++)
			{
				bOutRemovedPackageDatas[Index] = false;

				FAssetData* AssetData = AssetDatas[Index];

#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
				FAssetData* AssetIndex = AssetData;

				TArray<FAssetData*, TInlineAllocator<1>>* OldPackageAssets = CachedAssetsByPackageName.Find(AssetData->PackageName);
				bool bRemovedAssetFromPackageAssets = OldPackageAssets->RemoveSingleSwap(AssetIndex, EAllowShrinking::No) > 0;
				bool bOldPackageAssetsEmpty = bRemovedAssetFromPackageAssets && OldPackageAssets->Num() == 0;
				if (bOldPackageAssetsEmpty)
				{
					CachedAssetsByPackageName.Remove(AssetData->PackageName);
				}
#else
				UE::AssetRegistry::Private::FAssetDataPtrIndex AssetIndex = AssetIndices[Index];
				bool bRemovedAssetFromPackageAssets = CachedAssetsByPackageName.Remove(AssetData->PackageName, AssetIndex) > 0;
				bool bOldPackageAssetsEmpty = bRemovedAssetFromPackageAssets && !CachedAssetsByPackageName.Contains(AssetData->PackageName);
#endif
				if (bOldPackageAssetsEmpty)
				{
					PackagesToRemove.Add(AssetData->PackageName);
				}

				bOutRemovedPackageDatas[Index] = bOldPackageAssetsEmpty;
			}

			// Only remove dependencies and package data if there are no other known assets in the package
			if (bRemoveDependencyData && !PackagesToRemove.IsEmpty())
			{
				// We need to update the cached dependencies references cache so that they know we no
				// longer exist and so don't reference them.
				if (PackagesToRemove.Num() == 1)
				{
					// It's common enough to remove a single package at 
					// a time we should special case it
					RemoveDependsNode(*PackagesToRemove.FindArbitraryElement());
				}
				else
				{
					RemoveDependsNodes(PackagesToRemove);
				}
			}

			// Remove the package data as well
			RemovePackageData(PackagesToRemove);
		};

	auto RemoveFromOtherAccelerators =
		[this, AssetDatas, Keys
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		, AssetIndices
#endif
		]()
		{
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
			TMap<FTopLevelAssetPath, TSet<FAssetData*>> RemovedAssetsForClass;
#else
			TMap<FTopLevelAssetPath, TSet<UE::AssetRegistry::Private::FAssetDataPtrIndex>> RemovedAssetsForClass;
#endif

			for (int32 Index = 0; Index < Keys.Num(); Index++)
			{
				FAssetData* AssetData = AssetDatas[Index];

#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
				FAssetData* AssetIndex = AssetData;
#else
				UE::AssetRegistry::Private::FAssetDataPtrIndex AssetIndex = AssetIndices[Index];
#endif

				CachedAssetsByPath.Find(AssetData->PackagePath)->RemoveSingleSwap(AssetIndex);
				auto& ArrayOfAssetIndex = RemovedAssetsForClass.FindOrAdd(AssetData->AssetClassPath);
				if (ArrayOfAssetIndex.IsEmpty())
				{
					ArrayOfAssetIndex.Reserve(CachedAssetsByClass.Find(AssetData->AssetClassPath)->Num());
				}
				ArrayOfAssetIndex.Add(AssetIndex);

#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
				for (auto TagIt = AssetData->TagsAndValues.CreateConstIterator(); TagIt; ++TagIt)
				{
					CachedAssetsByTag.Find(TagIt.Key())->Remove(AssetIndex);
				}
#else
				// For CachedClassesByTag, we do not need to remove the asset's class from the entries
				// for the old tags. The class is not changed and still has the possibility of containing the tags.
#endif
			}

			// Remove the FAssetData from their class list. We do this as one operation per class, and with the removed
			// AssetDatas in a TSet, because the class list is a TArray and it would be expensive to find each removed
			// AssetData in the TArray and call RemoveSwap on it.
			for (const auto& ClassPathAndSetOfAssetIndexPair : RemovedAssetsForClass)
			{
				const FTopLevelAssetPath& ClassPath = ClassPathAndSetOfAssetIndexPair.Key;
				const auto& SetOfRemovedAssetIndex = ClassPathAndSetOfAssetIndexPair.Value;
				CachedAssetsByClass.Find(ClassPath)->RemoveAllSwap([&SetOfRemovedAssetIndex](auto ExistingAssetIndex)
					{
						return SetOfRemovedAssetIndex.Contains(ExistingAssetIndex);
					}, EAllowShrinking::Yes);
			}
		};

	auto RemoveFromCachedAssets =
		[this, Keys, &bOutRemovedAssetDatas]()
		{
			for (int32 Index = 0; Index < Keys.Num(); Index++)
			{
				const FCachedAssetKey& Key = Keys[Index];
				int32 NumRemoved = CachedAssets.Remove(Key);
				bOutRemovedAssetDatas[Index] = NumRemoved > 0;
				if (NumRemoved > 0)
				{
					NumAssets--;
				}
			}
		};

	constexpr int32 MinCountForMultiThreaded = 100;
	if (Keys.Num() < MinCountForMultiThreaded)
	{
		// Avoid the overhead of creating Tasks and just run the lambdas in the current thread
		RemoveFromPackageAssets();
		RemoveFromOtherAccelerators();
		RemoveFromCachedAssets();
	}
	else
	{
		FTask RemoveFromPackageAssetsTask = Launch(TEXT("RemoveAssetDatasImpl_PackageAssets"),
			MoveTemp(RemoveFromPackageAssets), ETaskPriority::Default, EExtendedTaskPriority::None);
		FTask RemoveFromOtherAcceleratorsTask = Launch(TEXT("RemoveAssetDatasImpl_OtherAccelerators"),
			MoveTemp(RemoveFromOtherAccelerators), ETaskPriority::Default, EExtendedTaskPriority::None);
		FTask RemoveFromCachedAssetsTask = Launch(TEXT("RemoveAssetDatasImpl_CachedAssets"),
			MoveTemp(RemoveFromCachedAssets), 
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
			// In the UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS case the other containers hold an index into
			// CachedAssets, so we can only remove from CachedAssets after removing from all other containers.
			UE::Tasks::Prerequisites(RemoveFromPackageAssetsTask, RemoveFromOtherAcceleratorsTask),
#endif
			ETaskPriority::Default, EExtendedTaskPriority::None);

		RemoveFromPackageAssetsTask.Wait();
		RemoveFromOtherAcceleratorsTask.Wait();
		RemoveFromCachedAssetsTask.Wait();
	}

	// If the assets were preallocated in a block, we can't delete them one at a time,
	// only the whole chunk in the destructor
	if (PreallocatedAssetDataBuffers.Num() == 0)
	{
		for (int32 Index = 0; Index < Keys.Num(); Index++)
		{
			FAssetData* AssetData = AssetDatas[Index];
			if (bOutRemovedAssetDatas[Index])
			{
				delete AssetData;
			}
		}
	}
}

void FAssetRegistryState::Shrink()
{
	CachedAssets.Shrink();

#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	IndirectAssetDataArrays.Shrink();
#endif

	CachedAssetsByPackageName.Shrink();
	ShrinkMultimap(CachedAssetsByPath);
	ShrinkMultimap(CachedAssetsByClass);

#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
	CachedAssetsByTag.Shrink();
#else
	CachedClassesByTag.Shrink();
#endif

	CachedDependsNodes.Shrink();
	CachedPackageData.Shrink();
}

FDependsNode* FAssetRegistryState::FindDependsNode(const FAssetIdentifier& Identifier) const
{
	FDependsNode*const* FoundNode = CachedDependsNodes.Find(Identifier);
	if (FoundNode)
	{
		return *FoundNode;
	}
	else
	{
		return nullptr;
	}
}

FDependsNode* FAssetRegistryState::CreateOrFindDependsNode(const FAssetIdentifier& Identifier, bool* bAlreadyExists)
{
	bool bLocalAlreadyExists = true;
	FDependsNode*& Node = CachedDependsNodes.FindOrAdd(Identifier);
	if (Node == nullptr)
	{
		bLocalAlreadyExists = false;
		Node = new FDependsNode(Identifier);
		NumDependsNodes++;
	}

	if (bAlreadyExists)
	{
		*bAlreadyExists = bLocalAlreadyExists;
	}

	return Node;
}

bool FAssetRegistryState::RemoveDependsNode(const FAssetIdentifier& Identifier)
{
	FDependsNode** NodePtr = CachedDependsNodes.Find(Identifier);

	if (NodePtr != nullptr)
	{
		FDependsNode* Node = *NodePtr;
		if (Node != nullptr)
		{
			TArray<FDependsNode*> DependencyNodes;
			Node->GetDependencies(DependencyNodes);

			// Remove the reference to this node from all dependencies
			for (FDependsNode* DependencyNode : DependencyNodes)
			{
				DependencyNode->RemoveReferencer(Node);
			}

			TArray<FDependsNode*> ReferencerNodes;
			Node->GetReferencers(ReferencerNodes);

			// Remove the reference to this node from all referencers
			for (FDependsNode* ReferencerNode : ReferencerNodes)
			{
				ReferencerNode->RemoveDependency(Node);
			}

			// Remove the node and delete it
			CachedDependsNodes.Remove(Identifier);
			NumDependsNodes--;

			// if the depends nodes were preallocated in a block, we can't delete them one at a time,
			// only the whole chunk in the destructor
			if (PreallocatedDependsNodeDataBuffers.Num() == 0)
			{
				delete Node;
			}

			return true;
		}
	}

	return false;
}

bool FAssetRegistryState::RemoveDependsNodes(const TSet<FAssetIdentifier>& Identifiers,
	ERemoveDependsFlags Flags)
{
	using namespace UE::Tasks;

	const bool bRemoveOrphans = EnumHasAnyFlags(Flags, ERemoveDependsFlags::RemoveOrphans);
	TSet<FDependsNode*> NodesToRemove;
	NodesToRemove.Reserve(Identifiers.Num());

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RemoveDependsNode_UpdateCachedDependsNode);
		for (FAssetIdentifier Identifier : Identifiers)
		{
			FDependsNode** NodePtr = CachedDependsNodes.Find(Identifier);
			if (NodePtr)
			{
				if (FDependsNode* Node = *NodePtr)
				{
					NodesToRemove.Add(Node);
					CachedDependsNodes.Remove(Identifier);
				}
			}
		}
	}

	TMpscQueue<FDependsNode*> OrphansToRemoveQueue;
	TMap<FDependsNode*, TSet<FDependsNode*>> RemovedReferences;
	FTask PopulateTask = Launch(TEXT("RemoveDependsNode_PopulateSchedule"),
		[&NodesToRemove, &RemovedReferences, bRemoveOrphans, &OrphansToRemoveQueue]()
		{
			for (FDependsNode* Node : NodesToRemove)
			{
				TArray<FDependsNode*> ReferencerNodes;
				Node->GetReferencers(ReferencerNodes);

				// Remove the reference to this node from all referencers
				for (FDependsNode* ReferencerNode : ReferencerNodes)
				{
					// Only update nodes that will live after removal
					if (!NodesToRemove.Contains(ReferencerNode))
					{
						ReferencerNode->RemoveDependency(Node);
						if (bRemoveOrphans && ReferencerNode->GetConnectionCount() == 0)
						{
							OrphansToRemoveQueue.Enqueue(ReferencerNode);
						}
					}
				}
			}

			for (FDependsNode* Node : NodesToRemove)
			{
				TArray<FDependsNode*> DependencyNodes;
				Node->GetDependencies(DependencyNodes);

				// Remove the reference to this node from all dependencies
				for (FDependsNode* DependencyNode : DependencyNodes)
				{
					// Only update nodes that will live after removal
					if (!NodesToRemove.Contains(DependencyNode))
					{
						TSet<FDependsNode*>& Nodes = RemovedReferences.FindOrAdd(DependencyNode);
						Nodes.Add(Node);
					}
				}
			}
		}, ETaskPriority::Default, EExtendedTaskPriority::None);

	FTask RemoveTask = Launch(TEXT("RemoveDependsNode_RemoveReferencerSchedule"),
		[&RemovedReferences, bRemoveOrphans, &OrphansToRemoveQueue]()
		{
			TArray<FDependsNode*> DependenciesToUpdate;
			RemovedReferences.GetKeys(DependenciesToUpdate);
			ParallelFor(TEXT("RemoveDependsNode_RemoveReferencerParallel"), DependenciesToUpdate.Num(), 1, 
				[&DependenciesToUpdate, &RemovedReferences, bRemoveOrphans, &OrphansToRemoveQueue](int32 Index)
				{
					FDependsNode* DependencyNode = DependenciesToUpdate[Index];
					TSet<FDependsNode*>* Nodes = RemovedReferences.Find(DependencyNode);
					DependencyNode->RemoveReferencers(*Nodes);
					if (bRemoveOrphans && DependencyNode->GetConnectionCount() == 0)
					{
						OrphansToRemoveQueue.Enqueue(DependencyNode);
					}
				}, EParallelForFlags::Unbalanced);
		}, PopulateTask);

	// If the depends nodes were preallocated in a block, we can't delete 
	// them one at a time, only the whole chunk in the destructor
	const bool bDeleteRemovedNodes = PreallocatedDependsNodeDataBuffers.Num() == 0;
	FTask DeleteTask;
	if (bDeleteRemovedNodes)
	{
		DeleteTask = Launch(TEXT("RemoveDependsNode_DeleteNodes"), [&NodesToRemove]()
			{
				for (FDependsNode* Node : NodesToRemove)
				{
					delete Node;
				}
			}, PopulateTask); // Only need to wait for populating to be done before deleting nodes
	}
	NumDependsNodes -= NodesToRemove.Num();

	RemoveTask.Wait();
	DeleteTask.Wait();

	if (!OrphansToRemoveQueue.IsEmpty())
	{
		// It should be impossible to have duplicates in OrphansToRemove, but guarantee that with Algo::Unique.
		TArray<FDependsNode*> OrphansToRemove;
		FDependsNode* Orphan;
		while (OrphansToRemoveQueue.Dequeue(Orphan))
		{
			OrphansToRemove.Add(Orphan);
		}
		Algo::Sort(OrphansToRemove);
		int32 PreUniqueNum = OrphansToRemove.Num();
		OrphansToRemove.SetNum(Algo::Unique(OrphansToRemove));
		if (OrphansToRemove.Num() != PreUniqueNum)
		{
			UE_LOGFMT(LogAssetRegistry, Error,
				"Logic error in FAssetRegistryState::RemoveDependsNodes, OrphansToRemove had duplicates. PreUniqueNum={PreUniqueNum}, UniqueNum={UniqueNum}.",
				PreUniqueNum, OrphansToRemove.Num());
		}

		// We had to wait for RemoveTask to complete, which was running in parallel with DeleteTask, so we
		// can't lump these in to DeleteTask. Do a pass over them now to remove and delete them.
		for (FDependsNode* Node : OrphansToRemove)
		{
			int32 NumRemoved = CachedDependsNodes.Remove(Node->GetIdentifier());
			NumDependsNodes -= NumRemoved;
			if (bDeleteRemovedNodes)
			{
				delete Node;
			}
		}
	}
	return !NodesToRemove.IsEmpty();
}

void FAssetRegistryState::GetPrimaryAssetsIds(TSet<FPrimaryAssetId>& OutPrimaryAssets) const
{
	EnumerateAllAssets([&OutPrimaryAssets](const FAssetData& AssetData)
		{
			FPrimaryAssetId PrimaryAssetId = AssetData.GetPrimaryAssetId();
			if (PrimaryAssetId.IsValid())
			{
				OutPrimaryAssets.Add(PrimaryAssetId);
			}
		});
}

const FAssetPackageData* FAssetRegistryState::GetAssetPackageData(FName PackageName) const
{
	FAssetPackageData* const* FoundData = CachedPackageData.Find(PackageName);
	return FoundData ? *FoundData : nullptr;
}

FAssetPackageData* FAssetRegistryState::GetAssetPackageData(FName PackageName)
{
	FAssetPackageData** FoundData = CachedPackageData.Find(PackageName);
	return FoundData ? *FoundData : nullptr;
}

const FAssetPackageData* FAssetRegistryState::GetAssetPackageData(FName PackageName,
	FName& OutCorrectCasePackageName) const
{
	// CachedPackageData is keyed using the Package Names whose casing matches the filesystem. In order to perform a
	// single look up for the AssetPackageData while also returning the value of the key used to add to the map
	// originally we create a KeyIterator which is currently the only means to get a TPair<Key,Value> from a
	// TMap<Key,Value>
	TMap<FName, FAssetPackageData*>::TConstKeyIterator It = CachedPackageData.CreateConstKeyIterator(PackageName);
	FSetElementId Id = It.GetId();
	if (!Id.IsValidId())
	{
		return nullptr;
	}

	const TPair<FName, FAssetPackageData*>& Pair = CachedPackageData.Get(Id);
	OutCorrectCasePackageName = Pair.Key;
	return Pair.Value;
}

FAssetPackageData* FAssetRegistryState::CreateOrGetAssetPackageData(FName PackageName)
{
	FAssetPackageData*& Data = CachedPackageData.FindOrAdd(PackageName);
	if (Data == nullptr)
	{
		Data = new FAssetPackageData();
		NumPackageData++;
	}
	return Data;
}

bool FAssetRegistryState::RemovePackageData(FName PackageName)
{
	const TSet<FAssetIdentifier, DefaultKeyFuncs<FAssetIdentifier>, TInlineSetAllocator<1>> PackageNames({ PackageName });
	return RemovePackageData(PackageNames);
}

bool FAssetRegistryState::RemovePackageDatas(const TSet<FAssetIdentifier>& PackageNames)
{
	// Call the templated function (templated based on TSet allocator) for the default TSet allocator.
	return RemovePackageData(PackageNames);
}

template<typename SetKeyFuncs, typename SetAllocator>
bool FAssetRegistryState::RemovePackageData(const TSet<FAssetIdentifier, SetKeyFuncs, SetAllocator>& AssetIdentifiers)
{
	bool bRemoved = false;
	for (const FAssetIdentifier& AssetIdentifier : AssetIdentifiers)
	{
		FAssetPackageData* Data;
		if (CachedPackageData.RemoveAndCopyValue(AssetIdentifier.PackageName, Data))
		{
			if (Data != nullptr)
			{
				NumPackageData--;

				// If the package data was preallocated in a block, we can't delete them one at a time,
				// only the whole chunk in the destructor
				if (PreallocatedPackageDataBuffers.Num() == 0)
				{
					delete Data;
				}

				bRemoved = true;
			}
		}
	}
	
	return bRemoved;
}

bool FAssetRegistryState::IsFilterValid(const FARCompiledFilter& Filter)
{
	return UE::AssetRegistry::Utils::IsFilterValid(Filter);
}

void FAssetRegistryState::EnumerateAssetsByTagName(const FName TagName,
	TFunctionRef<bool(const FAssetData* AssetData)> Callback) const
{
	using namespace UE::AssetRegistry::Private;

#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	const TSet<FAssetData*>* FoundAssets = CachedAssetsByTag.Find(TagName);
	if (FoundAssets)
	{
		for (const FAssetData* AssetData : *FoundAssets)
		{
			if (!Callback(AssetData))
			{
				break;
			}
		}
	}
#else
	const TSet<FAssetDataPtrIndex>* FoundAssets = CachedAssetsByTag.Find(TagName);
	if (FoundAssets)
	{
		for (FAssetDataPtrIndex AssetIndex : *FoundAssets)
		{
			if (!Callback(CachedAssets[AssetIndex]))
			{
				break;
			}
		}
	}
#endif
#else
	const TSet<FTopLevelAssetPath>* FoundClasses = CachedClassesByTag.Find(TagName);
	if (!FoundClasses)
	{
		return;
	}

	// The lists of assets in CachedAssetsByClass are non-intersecting (each list is only the exact instances of that
	// class and does not include subclasses), so we do not need to handle removing duplicates when merging lists from
	// multiple classes.
	TArray<FAssetData*> PossibleAssets;
	for (const FTopLevelAssetPath& ClassPath : *FoundClasses)
	{
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		const TArray<FAssetData*>* ClassAssets = CachedAssetsByClass.Find(ClassPath);
		if (ClassAssets)
		{
			PossibleAssets.Append(*ClassAssets);
		}
#else
		const TArray<FAssetDataPtrIndex>* ClassAssets = CachedAssetsByClass.Find(ClassPath);
		if (ClassAssets)
		{
			PossibleAssets.Reserve(PossibleAssets.Num() + ClassAssets->Num());
			for (FAssetDataPtrIndex AssetIndex : *ClassAssets)
			{
				PossibleAssets.Add(CachedAssets[AssetIndex]);
			}
		}
#endif
	}

	for (const FAssetData* AssetData : PossibleAssets)
	{
		// Some assets are in a class that could have the tag, but the specific asset actually does not have the tag.
		if (AssetData->FindTag(TagName))
		{
			if (!Callback(AssetData))
			{
				break;
			}
		}
	}
#endif
}

void FAssetRegistryState::EnumerateTagToAssetDatas(
	TFunctionRef<bool(FName TagName, IAssetRegistry::FEnumerateAssetDatasFunc EnumerateAssets)> Callback) const
{
#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	for (const TPair<FName, TSet<FAssetData*>>& Pair : CachedAssetsByTag)
	{
#else
	for (const TPair<FName, TSet<FAssetDataPtrIndex>>& Pair : CachedAssetsByTag)
	{
#endif
		const bool bKeepEnumerating = Callback(Pair.Key,
			[&Pair, this](IAssetRegistry::FAssetDataFunc AssetCallback)
			{
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
				for (const FAssetData* AssetData : Pair.Value)
				{
#else
				for (FAssetDataPtrIndex AssetIndex : Pair.Value)
				{
					FAssetData* AssetData = CachedAssets[AssetIndex];
#endif
					if (!AssetCallback(AssetData))
					{
						return false;
					}
				}

				return true;
			});

		if (!bKeepEnumerating)
		{
			break;
		}
	}
#else
	for (const TPair<FName, TSet<FTopLevelAssetPath>>& Pair : CachedClassesByTag)
	{
		const bool bKeepEnumerating = Callback(Pair.Key, [&Pair, this](IAssetRegistry::FAssetDataFunc AssetCallback)
			{
				EnumerateAssetsByTagName(Pair.Key, [&AssetCallback](const FAssetData* AssetData)
					{
						if (!AssetCallback(AssetData))
						{
							return false;
						}
						return true;
					});

				return true;
			});

		if (!bKeepEnumerating)
		{
			break;
		}
	}
#endif
}

bool FAssetRegistryState::IsPackageUnmountedAndFiltered(const FName PackageName) const
{
	// TODO: This can be removed once UE-178174 is fixed, as there will no longer be unmounted content to enumerate
#if WITH_EDITOR
	// Note: We currently only perform this filtering in the editor; runtime use will have to perform its own
	//       filtering via FPackageName::IsValidPath so that it can choose to accept the additional cost of running
	//       that filter
	return bCookedGlobalAssetRegistryState && GIsEditor && !FPackageName::IsValidPath(WriteToString<256>(PackageName));
#else
	return false;
#endif
}

namespace UE::AssetRegistry::Utils
{

bool IsFilterValid(const FARCompiledFilter& Filter)
{
	if (Filter.PackageNames.Contains(NAME_None) ||
		Filter.PackagePaths.Contains(NAME_None) ||
		Filter.SoftObjectPaths.Contains(FSoftObjectPath()) ||
		Filter.ClassPaths.Contains(FTopLevelAssetPath()) ||
		Filter.TagsAndValues.Contains(NAME_None)
		)
	{
		return false;
	}

	return true;
}

}

#if ASSET_REGISTRY_STATE_DUMPING_ENABLED
namespace UE::AssetRegistry
{

void PropertiesToString(EDependencyProperty Properties, FStringBuilderBase& Builder, EDependencyCategory CategoryFilter)
{
	bool bFirst = true;
	auto AppendPropertyName = [&Properties, &Builder, &bFirst](EDependencyProperty TestProperty,
		const TCHAR* NameWith, const TCHAR* NameWithout)
	{
		if (!bFirst)
		{
			Builder.Append(TEXT(","));
		}
		if (!!(Properties & TestProperty))
		{
			Builder.Append(NameWith);
		}
		else
		{
			Builder.Append(NameWithout);
		}
		bFirst = false;
	};
	if (!!(CategoryFilter & EDependencyCategory::Package))
	{
		AppendPropertyName(EDependencyProperty::Hard, TEXT("Hard"), TEXT("Soft"));
		AppendPropertyName(EDependencyProperty::Game, TEXT("Game"), TEXT("EditorOnly"));
		AppendPropertyName(EDependencyProperty::Build, TEXT("Build"), TEXT("NotBuild"));
	}
	if (!!(CategoryFilter & EDependencyCategory::Manage))
	{
		AppendPropertyName(EDependencyProperty::Direct, TEXT("Direct"), TEXT("Indirect"));
		AppendPropertyName(EDependencyProperty::CookRule, TEXT("CookRule"), TEXT("ChunkOnly"));
	}
	static_assert((EDependencyProperty::PackageMask
		| EDependencyProperty::SearchableNameMask
		| EDependencyProperty::ManageMask)
		== EDependencyProperty::AllMask,
		"Need to handle new flags in this function");
}

} // namespace UE::AssetRegistry

bool PrintAssetDataMapKeyIsLess(FName A, FName B)
{
	return A.Compare(B) < 0;
}

bool PrintAssetDataMapKeyIsLess(const FString& A, const FString& B)
{
	return A.Compare(B, ESearchCase::IgnoreCase) < 0;
}

bool PrintAssetDataMapKeyIsLess(const FTopLevelAssetPath& A, const FTopLevelAssetPath& B)
{
	return A.Compare(B) < 0;
}

template <typename KeyType>
struct FPrintAssetDataMapKeyIsLess
{
	bool operator()(const KeyType& A, const KeyType& B) const
	{
		return PrintAssetDataMapKeyIsLess(A, B);
	}
};

template <typename MapType>
static void PrintAssetDataMap(FString Name, const MapType& AssetMap, TStringBuilder<16>& PageBuffer,
	const TFunctionRef<void()>& AddLine,
	const UE::AssetRegistry::Private::FAssetDataMap& CachedAssets,
	TUniqueFunction<void(const typename MapType::KeyType& Key, const FAssetData& Data)>&& PrintValue = {})
{
	using namespace UE::AssetRegistry::Private;

	PageBuffer.Appendf(TEXT("--- Begin %s ---"), *Name);
	AddLine();

	TArray<typename MapType::KeyType> Keys;
	AssetMap.GenerateKeyArray(Keys);

	Keys.Sort(FPrintAssetDataMapKeyIsLess<typename MapType::KeyType>());

	TArray<FAssetData*> Items;
	Items.Reserve(1024);

	int32 ValidCount = 0;
	for (const typename MapType::KeyType& Key : Keys)
	{
		const auto& AssetArray = AssetMap.FindChecked(Key);
		if (AssetArray.Num() == 0)
		{
			continue;
		}
		++ValidCount;

		Items.Reset();
		Items.Reserve(AssetArray.Num());
		for (const auto& It : AssetArray)
		{
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
			Items.Add(It);
#else
			Items.Add(CachedAssets[It]);
#endif
		}
		Items.Sort([](const FAssetData& A, const FAssetData& B)
			{
				return A.GetSoftObjectPath().LexicalLess(B.GetSoftObjectPath());
			});

		PageBuffer.Append(TEXT("\t"));
		Key.AppendString(PageBuffer);
		PageBuffer.Appendf(TEXT(" : %d item(s)"), Items.Num());
		AddLine();
		for (const FAssetData* Data : Items)
		{
			PageBuffer.Append(TEXT("\t\t"));
			Data->AppendObjectPath(PageBuffer);
			if (PrintValue)
			{
				PrintValue(Key, *Data);
			}
			AddLine();
		}
	}

	PageBuffer.Appendf(TEXT("--- End %s : %d entries ---"), *Name, ValidCount);
	AddLine();
};

template <typename MapType>
static void PrintClassDataMap(FString Name, const MapType& ClassPathMap, TStringBuilder<16>& PageBuffer,
	const TFunctionRef<void()>& AddLine)
{
	PageBuffer.Appendf(TEXT("--- Begin %s ---"), *Name);
	AddLine();

	TArray<typename MapType::KeyType> Keys;
	ClassPathMap.GenerateKeyArray(Keys);

	Keys.Sort(FPrintAssetDataMapKeyIsLess<typename MapType::KeyType>());

	TArray<FTopLevelAssetPath> Items;
	Items.Reserve(1024);

	int32 ValidCount = 0;
	for (const typename MapType::KeyType& Key : Keys)
	{
		const TSet<FTopLevelAssetPath>& ClassPaths = ClassPathMap.FindChecked(Key);
		if (ClassPaths.Num() == 0)
		{
			continue;
		}
		++ValidCount;

		Items.Reset();
		Items.Reserve(ClassPaths.Num());
		for (const FTopLevelAssetPath& ClassPath : ClassPaths)
		{
			Items.Add(ClassPath);
		}
		Items.Sort([](const FTopLevelAssetPath& A, const FTopLevelAssetPath& B)
			{
				return A.Compare(B) < 0;
			});

		PageBuffer.Append(TEXT("\t"));
		Key.AppendString(PageBuffer);
		PageBuffer.Appendf(TEXT(" : %d item(s)"), Items.Num());
		AddLine();
		for (const FTopLevelAssetPath& Data : Items)
		{
			PageBuffer.Append(TEXT("\t\t"));
			Data.AppendString(PageBuffer);
			AddLine();
		}
	}

	PageBuffer.Appendf(TEXT("--- End %s : %d entries ---"), *Name, ValidCount);
	AddLine();
};

#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
template <typename MapType>
static void PrintPackageNameMap(FString Name, const MapType& AssetMap, TStringBuilder<16>& PageBuffer,
	const TFunctionRef<void()>& AddLine,
	const UE::AssetRegistry::Private::FAssetDataMap& CachedAssets)
{
	using namespace UE::AssetRegistry::Private;

	PageBuffer.Appendf(TEXT("--- Begin %s ---"), *Name);
	AddLine();

	TArray<typename MapType::KeyType> Keys;
	AssetMap.GenerateKeyArray(Keys);

	Keys.Sort(FPrintAssetDataMapKeyIsLess<typename MapType::KeyType>());

	TArray<FAssetData*> Items;
	Items.Reserve(1024);

	int32 ValidCount = 0;
	for (const typename MapType::KeyType& Key : Keys)
	{
		TOptional<TConstArrayView<FAssetDataPtrIndex>> AssetArrayPtr = AssetMap.Find(Key);
		TConstArrayView<FAssetDataPtrIndex> AssetArray = AssetArrayPtr
			? *AssetArrayPtr : TConstArrayView<FAssetDataPtrIndex>();
		if (AssetArray.Num() == 0)
		{
			continue;
		}
		++ValidCount;

		Items.Reset();
		Items.Reserve(AssetArray.Num());
		for (const auto& It : AssetArray)
		{
			Items.Add(CachedAssets[It]);
		}
		Items.Sort([](const FAssetData& A, const FAssetData& B)
			{
				return A.GetSoftObjectPath().LexicalLess(B.GetSoftObjectPath());
			});

		PageBuffer.Append(TEXT("\t"));
		Key.AppendString(PageBuffer);
		PageBuffer.Appendf(TEXT(" : %d item(s)"), Items.Num());
		AddLine();
		for (const FAssetData* Data : Items)
		{
			PageBuffer.Append(TEXT("\t\t"));
			Data->AppendObjectPath(PageBuffer);
			AddLine();
		}
	}

	PageBuffer.Appendf(TEXT("--- End %s : %d entries ---"), *Name, ValidCount);
	AddLine();
};
#endif

void FAssetRegistryState::Dump(const TArray<FString>& Arguments, TArray<FString>& OutPages, int32 LinesPerPage) const
{
	int32 ExpectedNumLines = 14 + CachedAssets.Num() * 5 + CachedDependsNodes.Num() + CachedPackageData.Num();
	const int32 EstimatedLinksPerNode = 10*2; // Each dependency shows up once as a dependency and once as a reference
	const int32 EstimatedCharactersPerLine = 100;
	bool bAllFields = Arguments.Contains(TEXT("All"));

	const bool bDumpDependencyDetails = bAllFields || Arguments.Contains(TEXT("DependencyDetails"));
	if (bDumpDependencyDetails)
	{
		ExpectedNumLines += CachedDependsNodes.Num() * (3 + EstimatedLinksPerNode);
	}
	LinesPerPage = FMath::Max(0, LinesPerPage);
	const int32 ExpectedNumPages = LinesPerPage > 0 ? (ExpectedNumLines / LinesPerPage) : 1;
	const int32 PageEndSearchLength = FMath::Min(LinesPerPage, ExpectedNumLines) / 20;
	// Pick a large starting value to bias against picking empty string
	const uint32 HashStartValue = MAX_uint32 - 49979693; 
	const uint32 HashMultiplier = 67867967;
	TStringBuilder<16> PageBuffer;
	TStringBuilder<16> OverflowText;

	OutPages.Reserve(ExpectedNumPages);
	// TODO: Add Reserve function to TStringBuilder
	PageBuffer.AddUninitialized(FMath::Min(LinesPerPage, ExpectedNumLines) * EstimatedCharactersPerLine);
	PageBuffer.Reset();
	OverflowText.AddUninitialized(PageEndSearchLength * EstimatedCharactersPerLine);
	OverflowText.Reset();
	int32 NumLinesInPage = 0;
	const int32 LineTerminatorLen = TCString<TCHAR>::Strlen(LINE_TERMINATOR);

	auto FinishPage = [&PageBuffer, &NumLinesInPage, HashStartValue, HashMultiplier, PageEndSearchLength,
		&OutPages, &OverflowText, LineTerminatorLen](bool bManualPageBreak)
	{
		int32 PageEndIndex = PageBuffer.Len();
		const TCHAR* BufferEnd = PageBuffer.GetData() + PageEndIndex;
		int32 NumOverflowLines = 0;
		// We want to facilitate diffing dumps between two different versions that should be similar,
		// but naively breaking up the dump into pages makes this difficult.
		// because after one missing or added line, every page from that point on will be offset and
		// therefore different, making false positive differences.
		// To make pages after one missing or added line the same, we look for a good page ending based 
		// on the text of all the lines near the end of the current page.
		// By choosing specific-valued texts as page breaks, we will usually randomly get lucky and
		// have the two diffs pick the same line for the end of the page.
		if (!bManualPageBreak && NumLinesInPage > PageEndSearchLength)
		{
			const TCHAR* WinningLineEnd = BufferEnd;
			uint32 WinningLineValue = 0;
			int32 WinningSearchIndex = 0;
			const TCHAR* LineEnd = BufferEnd;
			for (int32 SearchIndex = 0; SearchIndex < PageEndSearchLength; ++SearchIndex)
			{
				uint32 LineValue = HashStartValue;
				const TCHAR* LineStart = LineEnd;
				while (LineStart[-LineTerminatorLen] != LINE_TERMINATOR[0]
					|| TCString<TCHAR>::Strncmp(LINE_TERMINATOR,
						LineStart - LineTerminatorLen, LineTerminatorLen) != 0)
				{
					--LineStart;
					LineValue = LineValue * HashMultiplier + static_cast<uint32>(TChar<TCHAR>::ToLower(*LineStart));
				}
				// We arbitrarily choose the smallest hash as the winning value
				if (SearchIndex == 0 || LineValue < WinningLineValue)
				{
					WinningLineValue = LineValue;
					WinningLineEnd = LineEnd;
					WinningSearchIndex = SearchIndex;
				}
				LineEnd = LineStart - LineTerminatorLen;
			}
			if (WinningLineEnd != BufferEnd)
			{
				PageEndIndex = UE_PTRDIFF_TO_INT32(WinningLineEnd - PageBuffer.GetData());
				NumOverflowLines = WinningSearchIndex;
			}
		}

		OutPages.Add(FString::ConstructFromPtrSize(PageBuffer.GetData(), PageEndIndex));
		if (PageEndIndex != PageBuffer.Len())
		{
			PageEndIndex += LineTerminatorLen; // Skip the newline
			OverflowText.Reset();
			OverflowText.Append(PageBuffer.GetData() + PageEndIndex, PageBuffer.Len() - PageEndIndex);
			PageBuffer.Reset();
			PageBuffer.Append(OverflowText);
			PageBuffer.Append(LINE_TERMINATOR);
			NumLinesInPage = NumOverflowLines;
		}
		else
		{
			PageBuffer.Reset();
			NumLinesInPage = 0;
		}
	};
	auto AddLine = [&PageBuffer, LinesPerPage, &NumLinesInPage, &FinishPage, &OutPages]()
	{
		if (LinesPerPage == 1)
		{
			OutPages.Add(FString::ConstructFromPtrSize(PageBuffer.GetData(), PageBuffer.Len()));
			PageBuffer.Reset();
		}
		else
		{
			++NumLinesInPage;
			if (LinesPerPage == 0 || NumLinesInPage < LinesPerPage)
			{
				PageBuffer.Append(LINE_TERMINATOR);
			}
			else
			{
				FinishPage(false);
			}
		}
	};
	auto AddPageBreak = [LinesPerPage, &NumLinesInPage, &FinishPage]()
	{
		if (LinesPerPage > 1 && NumLinesInPage != 0)
		{
			FinishPage(true);
		}
	};

	if (bAllFields || Arguments.Contains(TEXT("ObjectPath")))
	{
		AddPageBreak();
		PageBuffer.Append(TEXT("--- Begin CachedAssetsByObjectPath ---"));
		AddLine();

		TArray<FCachedAssetKey> Keys;
		Keys.Reserve(CachedAssets.Num());
		EnumerateAllAssets([&Keys](const FAssetData& AssetData)
			{
				Keys.Emplace(FCachedAssetKey(AssetData));
			});
		Keys.Sort([](const FCachedAssetKey& A, const FCachedAssetKey& B) {
			return WriteToString<1024>(A).ToView().Compare(WriteToString<1024>(B).ToView(),
				ESearchCase::IgnoreCase) < 0;
		});

		for (const FCachedAssetKey& Key : Keys)
		{
			PageBuffer.Append(TEXT("	"));
			Key.AppendString(PageBuffer);
			AddLine();
		}

		PageBuffer.Appendf(TEXT("--- End CachedAssetsByObjectPath : %d entries ---"), CachedAssets.Num());
		AddLine();
	}

	if (bAllFields || Arguments.Contains(TEXT("PackageName")))
	{
		AddPageBreak();
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		PrintAssetDataMap(TEXT("CachedAssetsByPackageName"), CachedAssetsByPackageName, PageBuffer, AddLine, CachedAssets);
#else
		PrintPackageNameMap(TEXT("CachedAssetsByPackageName"), CachedAssetsByPackageName, PageBuffer, AddLine, CachedAssets);
#endif
	}

	if (bAllFields || Arguments.Contains(TEXT("Path")))
	{
		AddPageBreak();
		PrintAssetDataMap(TEXT("CachedAssetsByPath"), CachedAssetsByPath, PageBuffer, AddLine, CachedAssets);
	}

	if (bAllFields || Arguments.Contains(TEXT("Class")))
	{
		AddPageBreak();
		PrintAssetDataMap(TEXT("CachedAssetsByClass"), CachedAssetsByClass, PageBuffer, AddLine, CachedAssets);
	}

	// Only print this if it's requested specifically - '-all' will print tags-per-asset rather than assets-per-tag.
	if (Arguments.Contains(TEXT("Tag")))
	{
		AddPageBreak();
#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
		PrintAssetDataMap(TEXT("CachedAssetsByTag"), CachedAssetsByTag, PageBuffer, AddLine, CachedAssets,
			[&PageBuffer, &AddLine](const FName& TagName, const FAssetData& Data)
			{
				PageBuffer << TEXT(", ") << Data.TagsAndValues.FindTag(TagName).ToLoose();
			});
#else
		PrintClassDataMap(TEXT("CachedClassesByTag"), CachedClassesByTag, PageBuffer, AddLine);
#endif
	}

	TArray<const FAssetData*> SortedAssets;
	auto InitializeSortedAssets = [&SortedAssets, this]()
	{
		if (SortedAssets.Num() != CachedAssets.Num())
		{
			SortedAssets.Reserve(CachedAssets.Num());
			EnumerateAllAssets([&SortedAssets](const FAssetData& AssetData)
				{
					SortedAssets.Add(&AssetData);
				});
			Algo::Sort(SortedAssets, [](const FAssetData* A, const FAssetData* B)
				{ return A->GetSoftObjectPath().LexicalLess(B->GetSoftObjectPath()); }
			);
		}
	};

	if (bAllFields || Arguments.Contains(TEXT("AssetTags")))
	{
		int32 Counter = 0;
		AddPageBreak();
		PageBuffer.Append(TEXT("--- Begin AssetTags ---"));
		AddLine();

		InitializeSortedAssets();
		TArray<FName> SortedTagKeys;
		for (const FAssetData* AssetData : SortedAssets)
		{
			if (AssetData->TagsAndValues.Num() == 0)
			{
				continue;
			}
			++Counter;

			PageBuffer << TEXT("  ") << FCachedAssetKey(AssetData);
			AddLine();

			SortedTagKeys.Reset();
			AssetData->TagsAndValues.ForEach([&SortedTagKeys](const TPair<FName, FAssetTagValueRef>& TagPair)
			{
				SortedTagKeys.Add(TagPair.Key);
			});
			Algo::Sort(SortedTagKeys, FNameLexicalLess());
			for (FName TagKey : SortedTagKeys)
			{
				FAssetTagValueRef Value = AssetData->TagsAndValues.FindTag(TagKey);
				PageBuffer << TEXT("    ") << TagKey << TEXT(" : ") << *Value.AsString();
				AddLine();
			}
		}

		PageBuffer.Appendf(TEXT("--- End AssetTags : %d entries ---"), Counter);
		AddLine();
	}


	if ((bAllFields || Arguments.Contains(TEXT("Dependencies"))) && !bDumpDependencyDetails)
	{
		AddPageBreak();
		PageBuffer.Appendf(TEXT("--- Begin CachedDependsNodes ---"));
		AddLine();

		TArray<FDependsNode*> Nodes;
		CachedDependsNodes.GenerateValueArray(Nodes);
		Nodes.Sort([](const FDependsNode& A, const FDependsNode& B)
			{ return A.GetIdentifier().ToString() < B.GetIdentifier().ToString(); }
			);

		for (const FDependsNode* Node : Nodes)
		{
			PageBuffer.Append(TEXT("	"));
			Node->GetIdentifier().AppendString(PageBuffer);
			PageBuffer.Appendf(TEXT(" : %d connection(s)"), Node->GetConnectionCount());
			AddLine();
		}

		PageBuffer.Appendf(TEXT("--- End CachedDependsNodes : %d entries ---"), CachedDependsNodes.Num());
		AddLine();
	}

	if (bDumpDependencyDetails)
	{
		using namespace UE::AssetRegistry;
		AddPageBreak();
		PageBuffer.Append(TEXT("--- Begin CachedDependsNodes ---"));
		AddLine();

		auto SortByAssetID = [](const FDependsNode& A, const FDependsNode& B)
			{
				return A.GetIdentifier().ToString() < B.GetIdentifier().ToString();
			};
		TArray<FDependsNode*> Nodes;
		CachedDependsNodes.GenerateValueArray(Nodes);
		Nodes.Sort(SortByAssetID);

		if (Arguments.Contains(TEXT("LegacyDependencies"))) // LegacyDependencies are not show by all; they have to be directly requested
		{
			EDependencyCategory CategoryTypes[] =
			{
				EDependencyCategory::Package,
				EDependencyCategory::Package,
				EDependencyCategory::SearchableName,
				EDependencyCategory::Manage,
				EDependencyCategory::Manage,
				EDependencyCategory::None
			};
			EDependencyQuery CategoryQueries[] =
			{
				EDependencyQuery::Hard,
				EDependencyQuery::Soft,
				EDependencyQuery::NoRequirements,
				EDependencyQuery::Direct,
				EDependencyQuery::Indirect,
				EDependencyQuery::NoRequirements
			};
			const TCHAR* CategoryNames[] =
			{
				TEXT("Hard"),
				TEXT("Soft"),
				TEXT("SearchableName"),
				TEXT("HardManage"),
				TEXT("SoftManage"),
				TEXT("References")
			};
			const int NumCategories = UE_ARRAY_COUNT(CategoryTypes);
			check(NumCategories == UE_ARRAY_COUNT(CategoryNames) && NumCategories == UE_ARRAY_COUNT(CategoryQueries));

			TArray<FDependsNode*> Links;
			for (const FDependsNode* Node : Nodes)
			{
				PageBuffer.Append(TEXT("	"));
				Node->GetIdentifier().AppendString(PageBuffer);
				AddLine();
				for (int CategoryIndex = 0; CategoryIndex < NumCategories; ++CategoryIndex)
				{
					EDependencyCategory CategoryType = CategoryTypes[CategoryIndex];
					EDependencyQuery CategoryQuery = CategoryQueries[CategoryIndex];
					const TCHAR* CategoryName = CategoryNames[CategoryIndex];
					Links.Reset();
					if (CategoryType != EDependencyCategory::None)
					{
						Node->GetDependencies(Links, CategoryType, CategoryQuery);
					}
					else
					{
						Node->GetReferencers(Links);
					}
					if (Links.Num() > 0)
					{
						PageBuffer.Appendf(TEXT("		%s"), CategoryName);
						AddLine();
						Links.Sort(SortByAssetID);
						for (FDependsNode* LinkNode : Links)
						{
							PageBuffer.Append(TEXT("			"));
							LinkNode->GetIdentifier().AppendString(PageBuffer);
							AddLine();
						}
					}
				}
			}
		}
		else
		{
			EDependencyCategory CategoryTypes[] =
			{
				EDependencyCategory::Package,
				EDependencyCategory::SearchableName,
				EDependencyCategory::Manage,
				EDependencyCategory::None
			};
			const TCHAR* CategoryNames[] =
			{
				TEXT("Package"),
				TEXT("SearchableName"),
				TEXT("Manage"),
				TEXT("References")
			};
			const int NumCategories = UE_ARRAY_COUNT(CategoryTypes);
			check(NumCategories == UE_ARRAY_COUNT(CategoryNames));

			TArray<FAssetDependency> Dependencies;
			TArray<FDependsNode*> References;
			for (const FDependsNode* Node : Nodes)
			{
				PageBuffer.Append(TEXT("	"));
				Node->GetIdentifier().AppendString(PageBuffer);
				AddLine();
				for (int CategoryIndex = 0; CategoryIndex < NumCategories; ++CategoryIndex)
				{
					EDependencyCategory CategoryType = CategoryTypes[CategoryIndex];
					const TCHAR* CategoryName = CategoryNames[CategoryIndex];
					if (CategoryType != EDependencyCategory::None)
					{
						Dependencies.Reset();
						Node->GetDependencies(Dependencies, CategoryType);
						if (Dependencies.Num() > 0)
						{
							PageBuffer.Appendf(TEXT("		%s"), CategoryName);
							AddLine();
							Dependencies.Sort([](const FAssetDependency& A, const FAssetDependency& B)
								{
									FString AString = A.AssetId.ToString();
									FString BString = B.AssetId.ToString();
									if (AString != BString)
									{
										return AString < BString;
									}
									return A.Properties < B.Properties;
								});
							for (const FAssetDependency& AssetDependency : Dependencies)
							{
								PageBuffer.Append(TEXT("			"));
								AssetDependency.AssetId.AppendString(PageBuffer);
								PageBuffer.Append(TEXT("\t\t{"));
								PropertiesToString(AssetDependency.Properties, PageBuffer, AssetDependency.Category);
								PageBuffer.Append(TEXT("}"));
								AddLine();
							}
						}
					}
					else
					{
						References.Reset();
						Node->GetReferencers(References);
						if (References.Num() > 0)
						{
							PageBuffer.Appendf(TEXT("		%s"), CategoryName);
							AddLine();
							References.Sort(SortByAssetID);
							for (const FDependsNode* Reference : References)
							{
								PageBuffer.Append(TEXT("			"));
								Reference->GetIdentifier().AppendString(PageBuffer);
								AddLine();
							}
						}
					}
				}
			}
		}

		PageBuffer.Appendf(TEXT("--- End CachedDependsNodes : %d entries ---"), CachedDependsNodes.Num());
		AddLine();
	}
	if (bAllFields || Arguments.Contains(TEXT("PackageData")))
	{
		AddPageBreak();
		PageBuffer.Append(TEXT("--- Begin CachedPackageData ---"));
		AddLine();

		TArray<FName> Keys;
		CachedPackageData.GenerateKeyArray(Keys);
		Keys.Sort(FNameLexicalLess());

		for (const FName& Key : Keys)
		{
			const FAssetPackageData* PackageData = CachedPackageData.FindChecked(Key);
			PageBuffer.Append(TEXT("	"));
			Key.AppendString(PageBuffer);
			PageBuffer.Append(TEXT(" : "));
#if WITH_EDITORONLY_DATA
			PageBuffer << PackageData->GetPackageSavedHash();
#else
			PageBuffer << FIoHash();
#endif
			PageBuffer.Appendf(TEXT(" : %" INT64_FMT " bytes"), PackageData->DiskSize);
			AddLine();
		}

		PageBuffer.Appendf(TEXT("--- End CachedPackageData : %d entries ---"), CachedPackageData.Num());
		AddLine();
	}

	if (bAllFields || Arguments.Contains(TEXT("AssetBundles")))
	{
		int32 Counter = 0;
		AddPageBreak();
		PageBuffer.Append(TEXT("--- Begin AssetBundles ---"));
		AddLine();

		InitializeSortedAssets();
		for (const FAssetData* AssetData : SortedAssets)
		{
			if (AssetData->TaggedAssetBundles.IsValid())
			{
				++Counter;
				for (const FAssetBundleEntry& Entry : AssetData->TaggedAssetBundles->Bundles)
				{
					PageBuffer << TEXT("  Owner: ")
						<< FCachedAssetKey(AssetData) << TEXT(" BundleName: ") << Entry.BundleName;
					AddLine();

					for (const FTopLevelAssetPath& Path : Entry.AssetPaths)
					{
						PageBuffer << TEXT("    ") << Path;
						AddLine();
					}
				}
			}
		}

		PageBuffer.Appendf(TEXT("--- End AssetBundles : %d entries ---"), Counter);
		AddLine();
	}

	if (PageBuffer.Len() > 0)
	{
		if (LinesPerPage == 1)
		{
			AddLine();
		}
		else
		{
			FinishPage(true);
		}
	}
}

#endif // ASSET_REGISTRY_STATE_DUMPING_ENABLED


////////////////////////////////////////////////////////////////////////////

#if WITH_DEV_AUTOMATION_TESTS 

#include "Misc/AutomationTest.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAssetRegistryAssetPathStringsTest, "System.AssetRegistry.AssetPathStrings",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);

// Tests that we can produce correct paths for objects represented by FCachedAssetKey and FAssetData
// E.g	PackageName.AssetName
//		PackageName.TopLevel:Inner
//		Packagename.TopLevel:Inner.Innermost
bool FAssetRegistryAssetPathStringsTest::RunTest(const FString& Parameters)
{
	using namespace UE::AssetRegistry::Private;

	// Construct these FNames before creating FCachedAssetKey because the key tries not to create unused path names.
	const FName TopLevelOuter = "/Path/To/PackageName";
	const FName DirectSubObjectOuter = "/Path/To/PackageName.OuterName";
	const FName SubSubObjectOuter = "/Path/To/PackageName.OuterName:SubOuterName";
	const FName AssetName = "AssetName";

	const FString TopLevelPathString = FString::Printf(TEXT("%s.%s"),
		*TopLevelOuter.ToString(), *AssetName.ToString());
	const FString DirectSubObjectPathString = FString::Printf(TEXT("%s:%s"),
		*DirectSubObjectOuter.ToString(), *AssetName.ToString());
	const FString SubSubObjectPathString = FString::Printf(TEXT("%s.%s"),
		*SubSubObjectOuter.ToString(), *AssetName.ToString());

	const FSoftObjectPath TopLevelPath(TopLevelPathString);
	const FSoftObjectPath DirectSubObjectPath(DirectSubObjectPathString);
	const FSoftObjectPath SubSubObjectPath(SubSubObjectPathString);

	TestEqual(TEXT("SoftObjectPath::ToString() correct for top-level asset"),
		TopLevelPath.ToString(), TopLevelPathString);
	TestEqual(TEXT("SoftObjectPath::ToString() correct for subobject asset"),
		DirectSubObjectPath.ToString(), DirectSubObjectPathString);
	TestEqual(TEXT("SoftObjectPath::ToString() correct for sub-subobject asset"),
		SubSubObjectPath.ToString(), SubSubObjectPathString);

	// Construct FCachedAssetKey from FSoftObjectPath of various lengths + check they have the right components
	const FCachedAssetKey TopLevelAssetKey(TopLevelPath);
	TestEqual(TEXT("FCachedAssetKey::OuterPath correct for top-level asset"),
		TopLevelAssetKey.OuterPath.ToString(), TopLevelOuter.ToString());
	TestEqual(TEXT("FCachedAssetKey::ObjectName correct for top-level asset"),
		TopLevelAssetKey.ObjectName.ToString(), AssetName.ToString());

	const FCachedAssetKey DirectSubObjectKey(DirectSubObjectPath);
	TestEqual(TEXT("FCachedAssetKey::OuterPath correct for subobject asset"),
		DirectSubObjectKey.OuterPath.ToString(), DirectSubObjectOuter.ToString());
	TestEqual(TEXT("FCachedAssetKey::ObjectName correct for subobject asset"),
		DirectSubObjectKey.ObjectName.ToString(), AssetName.ToString());

	const FCachedAssetKey SubSubObjectKey(SubSubObjectPath);
	TestEqual(TEXT("FCachedAssetKey::OuterPath correct for sub-subobject asset"),
		SubSubObjectKey.OuterPath.ToString(), SubSubObjectOuter.ToString());
	TestEqual(TEXT("FCachedAssetKey::ObjectName correct for sub-subobject asset"),
		SubSubObjectKey.ObjectName.ToString(), AssetName.ToString());

	// Construct FCachedAssetKey from FSoftObjectPath of various lengths + check they give the right strings from AppendString
	TestEqual(TEXT("FCachedAssetKey::ToString() correct for top-level asset"),
		TopLevelAssetKey.ToString(), TopLevelPathString);
	TestEqual(TEXT("FCachedAssetKey::ToString() correct for subobject asset"),
		DirectSubObjectKey.ToString(), DirectSubObjectPathString);
	TestEqual(TEXT("FCachedAssetKey::ToString() correct for sub-subobject asset"),
		SubSubObjectKey.ToString(), SubSubObjectPathString);

	auto PathToAssetData = [](const FString& Path)
	{
		FString PackageName = FPackageName::ObjectPathToPackageName(Path);
		return FAssetData(PackageName, Path, FTopLevelAssetPath("/Script/CoreUObject.Object"));
	};

	FAssetData TopLevelAssetData = PathToAssetData(TopLevelPathString);
	FAssetData DirectSubObjectAssetData = PathToAssetData(DirectSubObjectPathString);
	FAssetData SubSubObjectAssetData = PathToAssetData(SubSubObjectPathString);

	// Test FAssetData::AppendPath for asset data with variable length of OptionalOuterPath
	TestEqual(TEXT("FAssetData::AppendPath() correct for top-level asset"),
		TopLevelAssetData.GetObjectPathString(), TopLevelPathString);

#if WITH_EDITORONLY_DATA
	// These tests are only enabled when WITH_EDITORONLY_DATA is active because only
	// then OuterPath is retained by FAssedData (see FAssetData::AppendObjectPath).
	TestEqual(TEXT("FAssetData::AppendPath() correct for subobject asset"),
		DirectSubObjectAssetData.GetObjectPathString(), DirectSubObjectPathString);
	TestEqual(TEXT("FAssetData::AppendPath() correct for sub-subobject asset"),
		SubSubObjectAssetData.GetObjectPathString(), SubSubObjectPathString);
#endif
	
	return true;
}
#endif // WITH_DEV_AUTOMATION_TESTS
