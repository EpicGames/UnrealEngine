// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/AssetRegistryState.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS
#include "AssetRegistry/AssetData.h"
#include "Misc/StringBuilder.h"
#endif


#if WITH_DEV_AUTOMATION_TESTS
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEnumerateAssetsPerformance, "System.AssetRegistry.AssetRegistryState.EnumerateAssetsPerformance",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::PerfFilter);

bool FEnumerateAssetsPerformance::RunTest(const FString& Parameters)
{
	constexpr int32 NumAssets = 1000000;
	constexpr FStringView PackageNameDirStr = TEXTVIEW("/Engine/AssetRegistryStateTest");
	FName PackageNameDirName(PackageNameDirStr);
	FTopLevelAssetPath BlueprintClassPath(TEXT("/Script/Engine.Blueprint"));

	FAssetRegistryState State;

	FName PackageNameToQuery;
	// Construct the state with N assets
	for (int32 Index = 0; Index < NumAssets; ++Index)
	{
		TStringBuilder<256> PackageNameStr(InPlace, PackageNameDirStr, TEXT("/Package"), Index);
		TStringBuilder<16> AssetName(InPlace, TEXT("Package"), Index);
		FName PackageNameFName(PackageNameStr);
		FAssetData* AssetData = new FAssetData(PackageNameFName, PackageNameDirName, FName(AssetName), BlueprintClassPath);
		State.AddAssetData(AssetData);
		PackageNameToQuery = PackageNameFName;
	}

	FARCompiledFilter Filter;
	Filter.PackageNames.Add(PackageNameToQuery);
	Filter.ClassPaths.Add(BlueprintClassPath);
	Filter.bIncludeOnlyOnDiskAssets = true;

	TSet<FName> PackageNamesToSkip;

	// Run the performance test K times to get more profile data
	// Expected cost is 10ms per call, make sure that NumIterations*Cost < 1 second so this test doesn't take too long
	constexpr int32 NumIterations = 10;

	for (int32 IterationIndex = 0; IterationIndex < NumIterations; ++IterationIndex)
	{
		int32 Count = 0;
		State.EnumerateAssets(Filter, PackageNamesToSkip,
			[&Count](const FAssetData& AssetData)
			{
				++Count;
				return true;
			});
		// Force Count to be used so that compiler cannot optimize away the entire EnumerateAssets call
		TestEqual(TEXT("EnumerateAssets: Expected number of assets was found."), Count, 1);
	}

	return true;
}

#endif