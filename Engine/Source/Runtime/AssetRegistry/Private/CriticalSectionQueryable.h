// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/CriticalSection.h"
#include "HAL/PlatformTLS.h"
#include "Misc/ScopeLock.h"

#include <atomic>

/** A Critical Section with additional function to query whether it is locked on the current thread. */
class FCriticalSectionQueryable
{
public:
	FCriticalSectionQueryable(const FCriticalSectionQueryable&) = delete;
	FCriticalSectionQueryable(FCriticalSectionQueryable&&) = delete;
	FCriticalSectionQueryable& operator=(const FCriticalSectionQueryable&) = delete;
	FCriticalSectionQueryable& operator=(FCriticalSectionQueryable&&) = delete;

	FCriticalSectionQueryable()
		: OwnerThreadId(0)
	{
	}

	void Lock()
	{
		Inner.Lock();
		SetOwner();
	}

	bool TryLock()
	{
		if (Inner.TryLock())
		{
			SetOwner();
			return true;
		}
		return false;
	}

	void Unlock()
	{
		check(IsLockedOnCurrentThread());
		ClearOwner();
		Inner.Unlock();
	}

	bool IsLockedOnCurrentThread() const
	{
		return OwnerThreadId.load(std::memory_order_relaxed) == FPlatformTLS::GetCurrentThreadId();
	}

private:
	void SetOwner()
	{
		OwnerThreadId.store(FPlatformTLS::GetCurrentThreadId(), std::memory_order_relaxed);
	}

	void ClearOwner()
	{
		OwnerThreadId.store(0, std::memory_order_relaxed);
	}

	FCriticalSection Inner;
	std::atomic<uint32> OwnerThreadId;
};

/**
 * Identical to FScopeLock but works with an FCriticalSectionQueryable
 */
class FScopeLockQueryable
{
public:
	FScopeLockQueryable() = delete;
	FScopeLockQueryable(const FScopeLockQueryable& InScopeLock) = delete;
	FScopeLockQueryable(FScopeLockQueryable&& InScopeLock) = delete;
	FScopeLockQueryable& operator=(FScopeLockQueryable& InScopeLock) = delete;
	FScopeLockQueryable& operator=(FScopeLockQueryable&& InScopeLock) = delete;

	FScopeLockQueryable(FCriticalSectionQueryable* InSynchObject)
		: SynchObject(InSynchObject)
	{
		check(SynchObject);
		SynchObject->Lock();
	}

	/** Destructor that performs a release on the synchronization object. */
	~FScopeLockQueryable()
	{
		Unlock();
	}

	void Unlock()
	{
		if (SynchObject)
		{
			SynchObject->Unlock();
			SynchObject = nullptr;
		}
	}

private:

	// Holds the synchronization object to aggregate and scope manage.
	FCriticalSectionQueryable* SynchObject;
};

/**
 * Like a critical section: only a single thread can posess it at once. But provides a different API than a critical
 * section to more flexibly interact with another critical section. With multiple normal critical sections they must
 * be entered in the same order to prevent deadlocks. A ThreadOwnerSection is instead tested and claimed for ownership
 * only while inside its collaborating critical section (called the AegisLock), and ownership can then continue to be
 * held even after releasing the AegisLock.
 */
class FThreadOwnerSection
{
public:
	bool TryTakeOwnership(FCriticalSectionQueryable& NotYetEnteredAegisLock)
	{
		FScopeLockQueryable ScopeLock(&NotYetEnteredAegisLock);
		return TryTakeOwnershipWithinLock();
	}
	bool TryTakeOwnership(FCriticalSection& NotYetEnteredAegisLock)
	{
		FScopeLock ScopeLock(&NotYetEnteredAegisLock);
		return TryTakeOwnershipWithinLock();
	}
	bool TryTakeOwnership(FScopeLockQueryable& AlreadyEnteredAegisScopeLock)
	{
		return TryTakeOwnershipWithinLock();
	}
	bool TryTakeOwnership(FScopeLock& AlreadyEnteredAegisScopeLock)
	{
		return TryTakeOwnershipWithinLock();
	}

	void ReleaseOwnershipChecked(FCriticalSectionQueryable& NotYetEnteredAegisLock)
	{
		FScopeLockQueryable ScopeLock(&NotYetEnteredAegisLock);
		ReleaseOwnershipCheckedWithinLock();
	}
	void ReleaseOwnershipChecked(FCriticalSection& NotYetEnteredAegisLock)
	{
		FScopeLock ScopeLock(&NotYetEnteredAegisLock);
		ReleaseOwnershipCheckedWithinLock();
	}
	void ReleaseOwnershipChecked(FScopeLockQueryable& AlreadyEnteredAegisScopeLock)
	{
		ReleaseOwnershipCheckedWithinLock();
	}
	void ReleaseOwnershipChecked(FScopeLock& AlreadyEnteredAegisScopeLock)
	{
		ReleaseOwnershipCheckedWithinLock();
	}

	bool IsOwned(FScopeLockQueryable& AlreadyEnteredAegisScopeLock) const
	{
		return bHasOwner;
	}
	bool IsOwned(FScopeLock& AlreadyEnteredAegisScopeLock) const
	{
		return bHasOwner;
	}

	bool IsOwnedByCurrentThread() const
	{
		return OwnerThreadId.load(std::memory_order_relaxed) == FPlatformTLS::GetCurrentThreadId();
	}

private:
	bool TryTakeOwnershipWithinLock()
	{
		if (!bHasOwner)
		{
			bHasOwner = true;
			OwnerThreadId.store(FPlatformTLS::GetCurrentThreadId(), std::memory_order_relaxed);
			return true;
		}
		return false;
	}
	void ReleaseOwnershipCheckedWithinLock()
	{
		check(IsOwnedByCurrentThread());
		bHasOwner = false;
		OwnerThreadId.store(0, std::memory_order_relaxed);
	}

private:
	/** OwnerThreadId can be checked while outside of the lock so it is an atomic to prevent dataraces. */
	std::atomic<uint32> OwnerThreadId{ 0 };
	bool bHasOwner = false;
};