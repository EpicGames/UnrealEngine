// Copyright Epic Games, Inc. All Rights Reserved.

#include "DependsNode.h"

#include "AssetRegistry/AssetRegistryState.h"
#include "AssetRegistryPrivate.h"
#include "Misc/StringBuilder.h"

template <uint32 FlagWidth>
void SortDependencyList(TArray<FDependsNode*>& Dependencies, TBitArray<>* Flags, EAllowShrinking ShrinkPolicy);

void FDependsNode::PrintNode() const
{
	UE_LOGF(LogAssetRegistry, Log, "*** Printing DependsNode: %ls ***", *GetIdentifier().ToString());
	UE_LOGF(LogAssetRegistry, Log, "*** Dependencies:");
	PrintDependencies();
	UE_LOGF(LogAssetRegistry, Log, "*** Referencers:");
	PrintReferencers();
}

void FDependsNode::PrintDependencies() const
{
	TSet<const FDependsNode*> VisitedNodes;

	PrintDependenciesRecursive(TEXT(""), VisitedNodes);
}

void FDependsNode::PrintReferencers() const
{
	TSet<const FDependsNode*> VisitedNodes;

	PrintReferencersRecursive(TEXT(""), VisitedNodes);
}

void FDependsNode::SetIdentifier(const FAssetIdentifier& InIdentifier)
{
	Identifier = InIdentifier;
	if (!InIdentifier.PackageName.IsNone())
	{
		TStringBuilder<FName::StringBufferSize> PackageNameStr(InPlace, InIdentifier.PackageName);
		bScriptPath = PackageNameStr.ToView().StartsWith(TEXT("/Script/"), ESearchCase::CaseSensitive) ? 1 : 0;
	}
	else
	{
		bScriptPath = 0;
	}
}

/**
 * Helper function for IterateDependencyList, which itself is a helper function for FDependsNode functions.
 * This function is given an index into one of the lists. All of the data for the list and how to interpret it is also
 * passed in.
 * The function looks of the set of DependencyProperty flag groups for the index, and iterates over that set, and
 * calls the callback on each flag group that matches the requested SearchFlags.
 */
template <uint32 FlagWidth, typename VisitorType>
void EnumerateMatchingDependencyFlags(const TBitArray<>* FlagBits, UE::AssetRegistry::EDependencyProperty CategoryMask,
	UE::AssetRegistry::EDependencyProperty(*ByteToProperties)(uint8),
	int32 ListIndex, const UE::AssetRegistry::FDependencyQuery& SearchFlags, VisitorType&& Visitor)
{
	using namespace UE::AssetRegistry;

	typedef TPropertyCombinationSet<FlagWidth> FCombinationSet;
	constexpr uint32 FlagSetWidth = FCombinationSet::StorageBitCount;

	EDependencyProperty RequiredProperties = SearchFlags.Required & CategoryMask;
	EDependencyProperty ExcludedProperties = SearchFlags.Excluded & CategoryMask;

	FCombinationSet DependsNodeFlagsSet(*FlagBits, ListIndex * FlagSetWidth);
	bool bDuplicate = false;
	for (uint32 DependencyFlagBits : DependsNodeFlagsSet)
	{
		EDependencyProperty DependencyProperties = ByteToProperties(static_cast<uint8>(DependencyFlagBits));
		if (!EnumHasAllFlags(DependencyProperties, RequiredProperties))
		{
			continue;
		}
		if (EnumHasAnyFlags(DependencyProperties, ExcludedProperties))
		{
			continue;
		}
		bool bPassesRequiredUnions = true;
		for (EDependencyProperty RequiredUnion : SearchFlags.RequiredUnions)
		{
			EDependencyProperty RequiredUnionProperty = RequiredUnion & CategoryMask;
			if (RequiredUnionProperty != EDependencyProperty::None &&
				!EnumHasAnyFlags(DependencyProperties, RequiredUnionProperty))
			{
				bPassesRequiredUnions = false;
				break;
			}
		}
		if (!bPassesRequiredUnions)
		{
			continue;
		}

		Visitor(DependencyProperties, bDuplicate);
		bDuplicate = true;
	}
}

/**
 * Helper function for GetDependencies, IterateOverDependencies, and others that need to find dependencies in
 * each of an FDependsNode's list of dependencies, while filtering by the list of DependencyProperties associated with
 * each dependency.
 * This function is given all of the data for the list and how to interpret it, and a requested SearchFlags.
 * The function iterates over all dependencies in the list, and iterates over the set of DependencyProperty flag
 * groups for each dependency, and calls the callback with the dependency and with each flag group on the dependency
 * that matches the requested SearchFlags.
 */
template <uint32 FlagWidth, typename CallbackType>
void IterateDependencyList(CallbackType&& InCallback,
	UE::AssetRegistry::EDependencyCategory SearchCategory,
	const UE::AssetRegistry::FDependencyQuery& SearchFlags,
	UE::AssetRegistry::EDependencyCategory ListCategory,
	UE::AssetRegistry::EDependencyProperty CategoryMask,
	const TArray<FDependsNode*>& Dependencies,
	const TBitArray<>* FlagBits,
	UE::AssetRegistry::EDependencyProperty(*ByteToProperties)(uint8),
	bool IsSorted)
{
	using namespace UE::AssetRegistry;
	if (!(SearchCategory & ListCategory))
	{
		return;
	}

	if (FlagWidth == 0)
	{
		for (int32 ListIndex = 0; ListIndex < Dependencies.Num(); ++ListIndex)
		{
			InCallback(Dependencies[ListIndex], ListCategory, EDependencyProperty::None, false /* bDuplicate */);
		}
	}
	else
	{
		for (int32 ListIndex = 0; ListIndex < Dependencies.Num(); ++ListIndex)
		{
			FDependsNode* DependsNode = Dependencies[ListIndex];
			EnumerateMatchingDependencyFlags<FlagWidth>(
				FlagBits, CategoryMask, ByteToProperties, ListIndex, SearchFlags,
				[&InCallback, DependsNode, ListCategory](EDependencyProperty MatchingDependencyProperties, bool bDuplicate)
				{
					InCallback(DependsNode, ListCategory, MatchingDependencyProperties, bDuplicate);
				});
		}
	}
}

/**
 * Helper function for IterateOverDependencies, ContainsDependency, and others that need to find a given dependency in
 * one of an FDependsNode's list of dependencies, while filtering by the list of DependencyProperties associated with
 * each dependency.
 * This function is given all of the data for the list and how to interpret it, and a requested SearchNode and
 * SearchFlags.
 * The function looks up the SearchNode in the list of dependencies, and iterates over the set of DependencyProperty flag
 * groups for each dependency, and calls the callback with each flag group on the dependency that matches the requested
 * SearchFlags.
 */
template <uint32 FlagWidth, typename CallbackType>
void IterateDependencyList(CallbackType&& InCallback,
	const FDependsNode* SearchNode,
	UE::AssetRegistry::EDependencyCategory SearchCategory,
	const UE::AssetRegistry::FDependencyQuery& SearchFlags,
	UE::AssetRegistry::EDependencyCategory ListCategory,
	UE::AssetRegistry::EDependencyProperty CategoryMask,
	const TArray<FDependsNode*>& Dependencies,
	const TBitArray<>* FlagBits,
	UE::AssetRegistry::EDependencyProperty(*ByteToProperties)(uint8),
	bool IsSorted)
{
	using namespace UE::AssetRegistry;
	if (!(SearchCategory & ListCategory))
	{
		return;
	}

	auto ReportIndex = [&](int32 ListIndex)
	{
		if (FlagWidth == 0)
		{
			InCallback(Dependencies[ListIndex], ListCategory, EDependencyProperty::None, false /* bDuplicate */);
		}
		else
		{
			FDependsNode* DependsNode = Dependencies[ListIndex];
			EnumerateMatchingDependencyFlags<FlagWidth>(
				FlagBits, CategoryMask, ByteToProperties, ListIndex, SearchFlags,
				[&InCallback, DependsNode, ListCategory](EDependencyProperty MatchingDependencyProperties, bool bDuplicate)
				{
					InCallback(DependsNode, ListCategory, MatchingDependencyProperties, bDuplicate);
				});
		}
	};

	if (IsSorted)
	{
		int32 ListIndex = Algo::BinarySearch(Dependencies, SearchNode);
		if (ListIndex != INDEX_NONE)
		{
			ReportIndex(ListIndex);
		}
	}
	else
	{
		for (int32 ListIndex = 0, Num = Dependencies.Num(); ListIndex < Num; ++ListIndex)
		{
			if (Dependencies[ListIndex] == SearchNode)
			{
				ReportIndex(ListIndex);
			}
		}
	}
}

void FDependsNode::IterateOverDependencies(const FIterateDependenciesCallback& InCallback,
	UE::AssetRegistry::EDependencyCategory Category,
	const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	using namespace UE::AssetRegistry;
	IterateDependencyList<PackageFlagWidth>(InCallback, Category, Flags, EDependencyCategory::Package,
		EDependencyProperty::PackageMask, PackageDependencies, &PackageFlags,
		ByteToPackageProperties, PackageIsSorted);
	IterateDependencyList<SearchableNameFlagWidth>(InCallback, Category, Flags, EDependencyCategory::SearchableName,
		EDependencyProperty::SearchableNameMask, NameDependencies, nullptr,
		nullptr, SearchableNameIsSorted);
	IterateDependencyList<ManageFlagWidth>(InCallback, Category, Flags, EDependencyCategory::Manage,
		EDependencyProperty::ManageMask, ManageDependencies, &ManageFlags,
		ByteToManageProperties, ManageIsSorted);
}

void FDependsNode::IterateOverDependencies(const FIterateDependenciesCallback& InCallback,
	const FDependsNode* DependsNode,
	UE::AssetRegistry::EDependencyCategory Category,
	const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	using namespace UE::AssetRegistry;
	IterateDependencyList<PackageFlagWidth>(InCallback, DependsNode, Category, Flags, EDependencyCategory::Package,
		EDependencyProperty::PackageMask, PackageDependencies, &PackageFlags,
		ByteToPackageProperties, PackageIsSorted);
	IterateDependencyList<SearchableNameFlagWidth>(InCallback, DependsNode, Category, Flags,
		EDependencyCategory::SearchableName,
		EDependencyProperty::SearchableNameMask, NameDependencies, nullptr, 
		nullptr, SearchableNameIsSorted);
	IterateDependencyList<ManageFlagWidth>(InCallback, DependsNode, Category, Flags, EDependencyCategory::Manage,
		EDependencyProperty::ManageMask, ManageDependencies, &ManageFlags,
		ByteToManageProperties, ManageIsSorted);
}

void FDependsNode::GetDependencies(TArray<FDependsNode*>& OutDependencies,
	UE::AssetRegistry::EDependencyCategory Category,
	const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	IterateOverDependencies([&OutDependencies](FDependsNode* InDependency,
		UE::AssetRegistry::EDependencyCategory InCategory,
		UE::AssetRegistry::EDependencyProperty InProperties,
		bool bDuplicate)
	{
		if (!bDuplicate)
		{
			OutDependencies.Add(InDependency);
		}
	}, 
	Category, Flags);
}

void FDependsNode::GetDependencies(TArray<FAssetIdentifier>& OutDependencies,
	UE::AssetRegistry::EDependencyCategory Category, const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	IterateOverDependencies([&OutDependencies](const FDependsNode* InDependency,
		UE::AssetRegistry::EDependencyCategory InCategory,
		UE::AssetRegistry::EDependencyProperty InProperties,
		bool bDuplicate)
	{
		if (!bDuplicate)
		{
			OutDependencies.Add(InDependency->GetIdentifier());
		}
	},
	Category, Flags);
}

void FDependsNode::GetDependencies(TArray<FAssetDependency>& OutDependencies,
	UE::AssetRegistry::EDependencyCategory Category, const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	IterateOverDependencies([&OutDependencies](const FDependsNode* InDependency,
		UE::AssetRegistry::EDependencyCategory InCategory,
		UE::AssetRegistry::EDependencyProperty InProperties,
		bool bDuplicate)
	{
		OutDependencies.Add(FAssetDependency{ InDependency->GetIdentifier(), InCategory, InProperties });
	},
	Category, Flags);
}

void FDependsNode::GetReferencers(TArray<FDependsNode*>& OutReferencers,
	UE::AssetRegistry::EDependencyCategory Category, const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	for (FDependsNode* Referencer : Referencers)
	{
		bool bShouldAdd = false;
		// If type specified, filter
		if (Category != UE::AssetRegistry::EDependencyCategory::All
			|| Flags.Required != UE::AssetRegistry::EDependencyProperty::None
			|| Flags.Excluded != UE::AssetRegistry::EDependencyProperty::None
			|| !Flags.RequiredUnions.IsEmpty())
		{
			Referencer->IterateOverDependencies([&bShouldAdd](const FDependsNode* InDependency,
				UE::AssetRegistry::EDependencyCategory InCategory,
				UE::AssetRegistry::EDependencyProperty InProperties,
				bool bDuplicate)
				{
					bShouldAdd = true;
				}, this, Category, Flags);
		}
		else
		{
			bShouldAdd = true;
		}

		if (bShouldAdd)
		{
			OutReferencers.Add(Referencer);
		}
	}
}

void FDependsNode::GetReferencers(TArray<FAssetDependency>& OutReferencers,
	UE::AssetRegistry::EDependencyCategory Category, const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	for (FDependsNode* Referencer : Referencers)
	{
		Referencer->IterateOverDependencies([&OutReferencers, Referencer](const FDependsNode* InDependency,
			UE::AssetRegistry::EDependencyCategory InCategory,
			UE::AssetRegistry::EDependencyProperty InProperties,
			bool bDuplicate)
			{
				OutReferencers.Add(FAssetDependency{ Referencer->GetIdentifier(), InCategory, InProperties });
			}, this, Category, Flags);
	}
}

void FDependsNode::GetPackageReferencers(TArray<TPair<FAssetIdentifier, FPackageFlagSet>>& OutReferencers)
{
	for (FDependsNode* Referencer : Referencers)
	{
		int32 Index = Algo::BinarySearch(Referencer->PackageDependencies, this);
		if (Index != INDEX_NONE)
		{
			FPackageFlagSet FlagList(Referencer->PackageFlags, Index * PackageFlagSetWidth);
			OutReferencers.Emplace(Referencer->GetIdentifier(), FlagList);
		}
	}
}

template <uint32 FlagWidth>
void AddDependency(FDependsNode* InDependency, UE::AssetRegistry::EDependencyProperty AddProperties,
	UE::AssetRegistry::EDependencyProperty CategoryMask, TArray<FDependsNode*>& Dependencies, TBitArray<>* FlagBits,
	uint8(*PropertiesToByte)(UE::AssetRegistry::EDependencyProperty), bool IsSorted)
{
	using namespace UE::AssetRegistry;

	bool bIsNew = false;
	int32 ListIndex = IsSorted ? Algo::LowerBound(Dependencies, InDependency) : Dependencies.Num();
	if (Dependencies.Num() <= ListIndex || Dependencies[ListIndex] != InDependency)
	{
		Dependencies.Insert(InDependency, ListIndex);
		bIsNew = true;
	}

	if (FlagWidth == 0)
	{
		return;
	}
	else
	{
		uint8 DependencyProperties = PropertiesToByte(CategoryMask & AddProperties);
		typedef TPropertyCombinationSet<FlagWidth> FCombinationSet;
		constexpr uint32 FlagSetWidth = FCombinationSet::StorageBitCount;
		FCombinationSet DependsNodeFlagsSet;
		if (bIsNew)
		{
			FlagBits->InsertUninitialized(ListIndex * FlagSetWidth, FlagSetWidth);
		}
		else
		{
			DependsNodeFlagsSet.Load(*FlagBits, ListIndex * FlagSetWidth);
		}
		DependsNodeFlagsSet.Add(DependencyProperties);
		DependsNodeFlagsSet.Save(*FlagBits, ListIndex * FlagSetWidth);
	}
}

void FDependsNode::AddDependency(FDependsNode* InDependency,
	UE::AssetRegistry::EDependencyCategory Category, UE::AssetRegistry::EDependencyProperty Properties)
{
	using namespace UE::AssetRegistry;
	if (EnumHasAnyFlags(Category, UE::AssetRegistry::EDependencyCategory::Package))
	{
		::AddDependency<PackageFlagWidth>(InDependency, Properties, EDependencyProperty::PackageMask,
			PackageDependencies, &PackageFlags, PackagePropertiesToByte, PackageIsSorted);
		// It is illegal to try to add a dependency as more than one category at a time
		check((Category & ~EDependencyCategory::Package) == EDependencyCategory::None);
	}
	else if (EnumHasAnyFlags(Category, EDependencyCategory::SearchableName))
	{
		::AddDependency<SearchableNameFlagWidth>(InDependency, Properties, EDependencyProperty::SearchableNameMask,
			NameDependencies, nullptr, nullptr, SearchableNameIsSorted);
		// It is illegal to try to add a dependency as more than one category at a time
		check((Category & ~EDependencyCategory::SearchableName) == EDependencyCategory::None);
	}
	else if (EnumHasAnyFlags(Category, UE::AssetRegistry::EDependencyCategory::Manage))
	{
		::AddDependency<ManageFlagWidth>(InDependency, Properties, EDependencyProperty::ManageMask,
			ManageDependencies, &ManageFlags, ManagePropertiesToByte, ManageIsSorted);
		// It is illegal to try to add a dependency as more than one category at a time
		check((Category & ~EDependencyCategory::Manage) == EDependencyCategory::None);
	}
	else
	{
		// It is illegal to try to add a dependency without a category
		check(false);
	}
}

void FDependsNode::AddPackageDependencySet(FDependsNode* InDependency, const FPackageFlagSet& PropertyCombinationSet)
{
	int32 Index = PackageIsSorted ? Algo::LowerBound(PackageDependencies, InDependency) : PackageDependencies.Num();
	if (PackageDependencies.Num() <= Index || PackageDependencies[Index] != InDependency)
	{
		PackageDependencies.Insert(InDependency, Index);
		PackageFlags.InsertUninitialized(Index * PackageFlagSetWidth, PackageFlagSetWidth);
	}
	PropertyCombinationSet.Save(PackageFlags, Index * PackageFlagSetWidth);
}

void FDependsNode::AddReferencer(FDependsNode* InReferencer)
{
	using namespace UE::AssetRegistry;
	::AddDependency<0>(InReferencer, EDependencyProperty::None, EDependencyProperty::None,
		Referencers, nullptr, nullptr, ReferencersIsSorted);
}

template <uint32 FlagWidth>
void RemoveDependency(FDependsNode* InDependency,
	TArray<FDependsNode*>& Dependencies,
	TBitArray<>* FlagBits,
	bool IsSorted,
	bool bAllowShrinking)
{
	const EAllowShrinking ShrinkPolicy = bAllowShrinking ? EAllowShrinking::Yes : EAllowShrinking::No;
	check(FlagWidth == 0 || FlagBits != nullptr);
	if (IsSorted)
	{
		int32 ListIndex = Algo::LowerBound(Dependencies, InDependency);
		if (Dependencies.Num() <= ListIndex || Dependencies[ListIndex] != InDependency)
		{
			return;
		}

		Dependencies.RemoveAt(ListIndex, ShrinkPolicy);
		if (FlagWidth != 0)
		{
			constexpr uint32 FlagSetWidth = TPropertyCombinationSet<FlagWidth>::StorageBitCount;
			FlagBits->RemoveAt(ListIndex * FlagSetWidth, FlagSetWidth);
		}
	}
	else
	{
		// When unsorted, in addition to be unsorted, the list may contain multiple elements of a single dependency
		for (int32 ListIndex = 0; ListIndex < Dependencies.Num(); )
		{
			if (Dependencies[ListIndex] != InDependency)
			{
				++ListIndex;
			}
			else
			{
				Dependencies.RemoveAtSwap(ListIndex, ShrinkPolicy);
				if (FlagWidth != 0)
				{
					constexpr uint32 FlagSetWidth = TPropertyCombinationSet<FlagWidth>::StorageBitCount;
					FlagBits->RemoveAtSwap(ListIndex * FlagSetWidth, FlagSetWidth);
				}
			}
		}
	}
}

void FDependsNode::RemoveDependency(FDependsNode* InDependency, UE::AssetRegistry::EDependencyCategory Category)
{
	if (!!(Category & UE::AssetRegistry::EDependencyCategory::Package))
	{
		::RemoveDependency<PackageFlagWidth>(InDependency, PackageDependencies, &PackageFlags, PackageIsSorted, bAllowShrinking);
	}

	if (!!(Category & UE::AssetRegistry::EDependencyCategory::SearchableName))
	{
		::RemoveDependency<SearchableNameFlagWidth>(InDependency, NameDependencies, nullptr, SearchableNameIsSorted, bAllowShrinking);
	}

	if (!!(Category & UE::AssetRegistry::EDependencyCategory::Manage))
	{
		::RemoveDependency<ManageFlagWidth>(InDependency, ManageDependencies, &ManageFlags, ManageIsSorted, bAllowShrinking);
	}
}

void FDependsNode::RemoveReferencer(FDependsNode* InReferencer)
{
	::RemoveDependency<0>(InReferencer, Referencers, nullptr, ReferencersIsSorted, bAllowShrinking);
}

void FDependsNode::RemoveReferencers(const TSet<FDependsNode*>& InReferencers)
{
	for (FDependsNodeList::TIterator It = Referencers.CreateIterator(); It; ++It)
	{
		if (InReferencers.Contains(*It))
		{
			It.RemoveCurrentSwap();
		}
	}

	if (ReferencersIsSorted)
	{
		Algo::Sort(Referencers);
	}

	if (bAllowShrinking)
	{
		Referencers.Shrink();
	}
}

void FDependsNode::RefreshReferencers()
{
	if (IsReferencersSorted())
	{
		Referencers.RemoveAll([this](FDependsNode* Referencer)
			{
				return !Referencer->ContainsDependency(this);
			});
	}
	else
	{
		Referencers.RemoveAllSwap([this](FDependsNode* Referencer)
			{
				return !Referencer->ContainsDependency(this);
			});
	}
}

void FDependsNode::ClearDependencies(UE::AssetRegistry::EDependencyCategory Category)
{
	using namespace UE::AssetRegistry;
	if (!!(Category & EDependencyCategory::Package))
	{
		if (bAllowShrinking)
		{
			PackageDependencies.Empty();
			PackageFlags.Empty();
		}
		else
		{
			PackageDependencies.Empty(PackageDependencies.Max());
			PackageFlags.Reset();
		}
	}
	if (!!(Category & EDependencyCategory::SearchableName))
	{
		NameDependencies.Empty(bAllowShrinking ? 0 : NameDependencies.Max());
	}
	if (!!(Category & EDependencyCategory::Manage))
	{
		if (bAllowShrinking)
		{
			ManageDependencies.Empty();
			ManageFlags.Empty();
		}
		else
		{
			ManageDependencies.Empty(ManageDependencies.Max());
			ManageFlags.Reset();
		}
	}
}

void FDependsNode::ClearReferencers()
{
	Referencers.Empty(bAllowShrinking ? 0 : Referencers.Max());
}

void FDependsNode::RemoveManageReferencesToNode()
{
	const EAllowShrinking ShrinkPolicy = bAllowShrinking ? EAllowShrinking::Yes : EAllowShrinking::No;
	UE::AssetRegistry::EDependencyCategory InCategory = UE::AssetRegistry::EDependencyCategory::Manage;

	// Iterate referencers array, possibly removing 
	for (int32 i = Referencers.Num() - 1; i >= 0; i--)
	{
		Referencers[i]->RemoveDependency(this, InCategory);
		if (!Referencers[i]->ContainsDependency(this, UE::AssetRegistry::EDependencyCategory::All & ~InCategory))
		{
			if (ReferencersIsSorted)
			{
				Referencers.RemoveAt(i, ShrinkPolicy);
			}
			else
			{
				Referencers.RemoveAtSwap(i, ShrinkPolicy);
			}
		}
	}
}

template <uint32 FlagWidth>
void RemoveAll(const TUniqueFunction<bool(const FDependsNode*)>& ShouldRemove,
	TArray<FDependsNode*>& Dependencies,
	TBitArray<>* FlagBits,
	bool IsSorted,
	EAllowShrinking ShrinkPolicy)
{
	check(FlagWidth == 0 || FlagBits != nullptr);
	if (IsSorted)
	{
		if (FlagWidth == 0)
		{
			Dependencies.RemoveAll(ShouldRemove);
		}
		else
		{
			// This block is the same functionality as TArray::RemoveAll,
			// but it needs to handle removing the corresponding FlagBits
			const int32 OriginalNum = Dependencies.Num();
			if (!OriginalNum)
			{
				return; // nothing to do, loop assumes one item so need to deal with this edge case here
			}

			constexpr uint32 FlagSetWidth = TPropertyCombinationSet<FlagWidth>::StorageBitCount;
			int32 WriteIndex = 0;
			int32 ReadIndex = 0;
			FDependsNode** DependencyData = Dependencies.GetData();
			// When calling ShouldRemove, use a ! to guarantee it can't be anything other than zero or one
			bool bKeep = !ShouldRemove(DependencyData[ReadIndex]);
			do
			{
				int32 RunStartIndex = ReadIndex++;
				while (ReadIndex < OriginalNum && bKeep == !ShouldRemove(DependencyData[ReadIndex]))
				{
					ReadIndex++;
				}
				int32 RunLength = ReadIndex - RunStartIndex;
				checkSlow(RunLength > 0);
				if (bKeep)
				{
					// this was a keep run, we need to move it
					if (WriteIndex != RunStartIndex)
					{
						FMemory::Memmove(&DependencyData[WriteIndex], &DependencyData[RunStartIndex],
							sizeof(DependencyData[0]) * RunLength);
						FlagBits->SetRangeFromRange(WriteIndex * FlagSetWidth,
							FlagSetWidth * RunLength, FlagBits->GetData(), RunStartIndex * FlagSetWidth);
					}
					WriteIndex += RunLength;
				}
				bKeep = !bKeep;
			} while (ReadIndex < OriginalNum);

			Dependencies.SetNum(WriteIndex, ShrinkPolicy);
			FlagBits->SetNumUninitialized(WriteIndex * FlagSetWidth);
		}
	}
	else
	{
		if (FlagWidth == 0)
		{
			Dependencies.RemoveAllSwap(ShouldRemove, ShrinkPolicy);
		}
		else
		{
			constexpr uint32 FlagSetWidth = TPropertyCombinationSet<FlagWidth>::StorageBitCount;
			for (int32 ListIndex = 0; ListIndex < Dependencies.Num(); )
			{
				if (ShouldRemove(Dependencies[ListIndex]))
				{
					Dependencies.RemoveAtSwap(ListIndex, ShrinkPolicy);
					FlagBits->RemoveAtSwap(ListIndex * FlagSetWidth, FlagSetWidth);
				}
				else
				{
					++ListIndex;
				}
			}
		}
	}
}

void FDependsNode::RemoveLinks(const TUniqueFunction<bool(const FDependsNode*)>& ShouldRemove)
{
	const EAllowShrinking ShrinkPolicy = bAllowShrinking ? EAllowShrinking::Yes : EAllowShrinking::No;
	::RemoveAll<PackageFlagWidth>(ShouldRemove, PackageDependencies, &PackageFlags, PackageIsSorted, ShrinkPolicy);
	::RemoveAll<SearchableNameFlagWidth>(ShouldRemove, NameDependencies, nullptr, SearchableNameIsSorted, ShrinkPolicy);
	::RemoveAll<ManageFlagWidth>(ShouldRemove, ManageDependencies, &ManageFlags, ManageIsSorted, ShrinkPolicy);
	::RemoveAll<0>(ShouldRemove, Referencers, nullptr, ReferencersIsSorted, ShrinkPolicy);
}

template <uint32 FlagWidth>
void RemapLinks(const TUniqueFunction<FDependsNode* (FDependsNode*)>& Remap,
	TArray<FDependsNode*>& Dependencies,
	TBitArray<>* FlagBits,
	bool IsSorted,
	EAllowShrinking ShrinkPolicy)
{
	for (FDependsNode*& OldNode : Dependencies)
	{
		OldNode = Remap(OldNode);
	}
	::RemoveAll<FlagWidth>([](const FDependsNode* Node)
		{
			return Node == nullptr;
		},
		Dependencies, FlagBits, false /* IsSorted */, ShrinkPolicy);
	if (IsSorted)
	{
		::SortDependencyList<FlagWidth>(Dependencies, FlagBits, ShrinkPolicy);
	}
}

void FDependsNode::RemapLinks(const TUniqueFunction<FDependsNode* (FDependsNode*)>& Remap)
{
	RemapDependencies(Remap);
	RemapReferencers(Remap);
}

void FDependsNode::RemapDependencies(const TUniqueFunction<FDependsNode* (FDependsNode*)>& Remap)
{
	const EAllowShrinking ShrinkPolicy = bAllowShrinking ? EAllowShrinking::Yes : EAllowShrinking::No;
	::RemapLinks<PackageFlagWidth>(Remap, PackageDependencies, &PackageFlags, PackageIsSorted, ShrinkPolicy);
	::RemapLinks<SearchableNameFlagWidth>(Remap, NameDependencies, nullptr, SearchableNameIsSorted, ShrinkPolicy);
	::RemapLinks<ManageFlagWidth>(Remap, ManageDependencies, &ManageFlags, ManageIsSorted, ShrinkPolicy);
}

void FDependsNode::RemapReferencers(const TUniqueFunction<FDependsNode* (FDependsNode*)>& Remap)
{
	const EAllowShrinking ShrinkPolicy = bAllowShrinking ? EAllowShrinking::Yes : EAllowShrinking::No;
	::RemapLinks<0>(Remap, Referencers, nullptr, ReferencersIsSorted, ShrinkPolicy);
}

bool FDependsNode::ContainsDependency(const FDependsNode* InDependency,
	UE::AssetRegistry::EDependencyCategory Category, const UE::AssetRegistry::FDependencyQuery& Flags) const
{
	using namespace UE::AssetRegistry;

	bool bExists = false;
	if (!!(Category & EDependencyCategory::Package))
	{
		IterateDependencyList<PackageFlagWidth>([&bExists]
		(const FDependsNode* DependsNode, EDependencyCategory MatchingCategory,
			EDependencyProperty MatchingDependencyProperties, bool bDuplicate)
			{
				bExists = true;
			},
			InDependency, Category, Flags, EDependencyCategory::Package,
			EDependencyProperty::PackageMask, PackageDependencies, &PackageFlags,
			ByteToPackageProperties, PackageIsSorted);
		if (bExists)
		{
			return true;
		}
	}
	if (!!(Category & EDependencyCategory::SearchableName))
	{
		IterateDependencyList<SearchableNameFlagWidth>([&bExists]
		(const FDependsNode* DependsNode, EDependencyCategory MatchingCategory,
			EDependencyProperty MatchingDependencyProperties, bool bDuplicate)
			{
				bExists = true;
			},
			InDependency, Category, Flags, EDependencyCategory::SearchableName,
			EDependencyProperty::SearchableNameMask, NameDependencies, nullptr,
			nullptr, SearchableNameIsSorted);
		if (bExists)
		{
			return true;
		}
	}
	if (!!(Category & EDependencyCategory::Manage))
	{
		IterateDependencyList<ManageFlagWidth>([&bExists]
		(const FDependsNode* DependsNode, EDependencyCategory MatchingCategory,
			EDependencyProperty MatchingDependencyProperties, bool bDuplicate)
			{
				bExists = true;
			},
			InDependency, Category, Flags, EDependencyCategory::Manage,
			EDependencyProperty::ManageMask, ManageDependencies, &ManageFlags,
			ByteToManageProperties, ManageIsSorted);
		if (bExists)
		{
			return true;
		}
	}
	return false;
}

void FDependsNode::PrintDependenciesRecursive(const FString& Indent, TSet<const FDependsNode*>& VisitedNodes) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!IsThisNotNull(this, "FDependsNode::PrintDependenciesRecursive"))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		UE_LOGF(LogAssetRegistry, Log, "%lsNULL", *Indent);
	}
	else if ( VisitedNodes.Contains(this) )
	{
		UE_LOGF(LogAssetRegistry, Log, "%ls[CircularReferenceTo]%ls", *Indent, *GetIdentifier().ToString());
	}
	else
	{
		UE_LOGF(LogAssetRegistry, Log, "%ls%ls", *Indent, *GetIdentifier().ToString());
		VisitedNodes.Add(this);

		IterateOverDependencies([&Indent, &VisitedNodes](FDependsNode* InDependency,
			UE::AssetRegistry::EDependencyCategory InCategory,
			UE::AssetRegistry::EDependencyProperty InProperties,
			bool bDuplicate)
		{
			if (!bDuplicate)
			{
				InDependency->PrintDependenciesRecursive(Indent + TEXT("  "), VisitedNodes);
			}
		});
	}
}

void FDependsNode::PrintReferencersRecursive(const FString& Indent, TSet<const FDependsNode*>& VisitedNodes) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!IsThisNotNull(this, "FDependsNode::PrintReferencersRecursive"))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		UE_LOGF(LogAssetRegistry, Log, "%lsNULL", *Indent);
	}
	else if ( VisitedNodes.Contains(this) )
	{
		UE_LOGF(LogAssetRegistry, Log, "%ls[CircularReferenceTo]%ls", *Indent, *GetIdentifier().ToString());
	}
	else
	{
		UE_LOGF(LogAssetRegistry, Log, "%ls%ls", *Indent, *GetIdentifier().ToString());
		VisitedNodes.Add(this);

		for (FDependsNode* Node : Referencers)
		{
			Node->PrintReferencersRecursive(Indent + TEXT("  "), VisitedNodes);
		}
	}
}

int32 FDependsNode::GetConnectionCount() const
{
	return PackageDependencies.Num() + NameDependencies.Num() + ManageDependencies.Num() + Referencers.Num();
}

void FDependsNode::SerializeSave(FArchive& Ar,
	const TUniqueFunction<int32(FDependsNode*, bool)>& GetSerializeIndexFromNode,
	FSaveScratch& Scratch,
	const FAssetRegistrySerializationOptions& Options) const
{
	Ar << const_cast<FAssetIdentifier&>(Identifier);

	auto WriteDependencies = [&Ar, &GetSerializeIndexFromNode, &Scratch](const TArray<FDependsNode*>& InDependencies,
		const TBitArray<>* InFlagBits,
		int FlagSetWidth,
		bool bAsReferencer)
	{
		TArray<FSaveScratch::FSortInfo>& SortInfos = Scratch.SortInfos;
		TArray<int32>& OutDependencies = Scratch.OutDependencies;
		TBitArray<>& OutFlagBits = Scratch.OutFlagBits;

		SortInfos.Reset(InDependencies.Num());
		for (int32 ListIndex = 0, End = InDependencies.Num(); ListIndex < End; ++ListIndex)
		{
			int32 SerializeIndex = GetSerializeIndexFromNode(InDependencies[ListIndex], bAsReferencer);
			if (SerializeIndex < 0)
			{
				continue;
			}
			SortInfos.Add({ SerializeIndex, ListIndex });
		}
		// Sort the serialized dependencies to make the output deterministic.
		// OutDependencies and OutFlagBits (if present) are associated arrays and have to be sorted together
		Algo::Sort(SortInfos, [](const FSaveScratch::FSortInfo& A, const FSaveScratch::FSortInfo& B)
			{
				return A.SerializeIndex < B.SerializeIndex;
			});

		int32 NumOutDependencies = SortInfos.Num();
		OutDependencies.Reset(NumOutDependencies);
		for (const FSaveScratch::FSortInfo& SortInfo : SortInfos)
		{
			OutDependencies.Add(SortInfo.SerializeIndex);
		}
		Ar << OutDependencies;
		if (InFlagBits)
		{
			OutFlagBits.Reset();
			for (const FSaveScratch::FSortInfo& SortInfo : SortInfos)
			{
				OutFlagBits.AddRange(*InFlagBits, FlagSetWidth, SortInfo.ListIndex * FlagSetWidth);
			}

			// We don't use BitArray::operator<< because we want to avoid the reallocation that operator<< does on load
			// and we want to avoid saving BitArray.Num when it can be derived from NumDependencies.
			int32 NumFlagBits = FlagSetWidth * NumOutDependencies;
			check(OutFlagBits.Num() == NumFlagBits);
			int32 NumFlagWords = FBitSet::CalculateNumWords(NumFlagBits);
			Ar.Serialize(OutFlagBits.GetData(), NumFlagWords * sizeof(uint32));
		}
	};

	WriteDependencies(PackageDependencies, &PackageFlags, PackageFlagSetWidth, false);
	WriteDependencies(Options.bSerializeSearchableNameDependencies ? NameDependencies : FDependsNodeList(),
		nullptr, 0, false);
	WriteDependencies(Options.bSerializeManageDependencies ? ManageDependencies : FDependsNodeList(),
		Options.bSerializeManageDependencies ? &ManageFlags : nullptr, ManageFlagSetWidth, false);
	WriteDependencies(Referencers, nullptr, 0, true);
}

template <bool bLatestVersion>
void FDependsNode::SerializeLoad_PackageDependencies(FArchive& Ar,
	const TUniqueFunction<FDependsNode* (int32)>& GetNodeFromSerializeIndex,
	FLoadScratch& Scratch, FAssetRegistryVersion::Type Version)
{
	// This code is duplicated-with-modifications in SerializeLoad_PackageDependencies,
	// and SerializeLoad_ManageDependencies, because sharing code makes it more difficult to read.
	TArray<FDependsNode*>& OutDependencies = this->PackageDependencies;
	TBitArray<>& OutFlagBits = this->PackageFlags;
	int FlagSetWidth = PackageFlagSetWidth;

	TArray<int32>& InDependencies = Scratch.InDependencies;
	TArray<uint32>& InFlagBits = Scratch.InFlagBits;
	TArray<FDependsNode*>& PointerDependencies = Scratch.PointerDependencies;
	TArray<int32>& SortIndexes = Scratch.SortIndexes;
	int32 NumFlagBits = 0;

	InDependencies.Reset();
	Ar << InDependencies;
	int32 NumDependencies = InDependencies.Num();

	// We don't use BitArray::operator<< because we want to avoid the reallocation that operator<< does on load
	// and we want to avoid saving BitArray.Num when it can be derived from NumDependencies.
	NumFlagBits = FlagSetWidth * NumDependencies;
	const int32 NumFlagWords = FBitSet::CalculateNumWords(NumFlagBits);
	InFlagBits.SetNumUninitialized(NumFlagWords);
	Ar.Serialize(InFlagBits.GetData(), NumFlagWords * sizeof(uint32));

	PointerDependencies.Reset(NumDependencies);
	for (int32 SerializeIndex : InDependencies)
	{
		FDependsNode* DependsNode = GetNodeFromSerializeIndex(SerializeIndex);
		if (!DependsNode)
		{
			Ar.SetError();
			return;
		}
		PointerDependencies.Add(DependsNode);
	}

	SortIndexes.Reset(NumDependencies);
	for (int Index = 0; Index < NumDependencies; ++Index)
	{
		SortIndexes.Add(Index);
	}

	Algo::Sort(SortIndexes, [&PointerDependencies](int32 A, int32 B)
		{
			return PointerDependencies[A] < PointerDependencies[B];
		});

	OutDependencies.Empty(NumDependencies);
	for (int32 SortIndex : SortIndexes)
	{
		OutDependencies.Add(PointerDependencies[SortIndex]);
	}

	OutFlagBits.SetNumUninitialized(NumFlagBits);
	uint32* InFlagBitsData = InFlagBits.GetData();
	for (int32 WriteIndex = 0; WriteIndex < NumDependencies; ++WriteIndex)
	{
		int32 ReadIndex = SortIndexes[WriteIndex];
		OutFlagBits.SetRangeFromRange(WriteIndex * FlagSetWidth, FlagSetWidth,
			InFlagBitsData, ReadIndex * FlagSetWidth);
	}
}

template <bool bLatestVersion>
void FDependsNode::SerializeLoad_NameDependencies(FArchive& Ar,
	const TUniqueFunction<FDependsNode* (int32)>& GetNodeFromSerializeIndex,
	FLoadScratch& Scratch, FAssetRegistryVersion::Type Version)
{
	// This code is duplicated-with-modifications in SerializeLoad_NameDependencies
	// and SerializeLoad_Referencers, because sharing code makes it more difficult to read.
	TArray<FDependsNode*>& OutDependencies = this->NameDependencies;

	TArray<int32>& InDependencies = Scratch.InDependencies;
	InDependencies.Reset();
	Ar << InDependencies;
	int32 NumDependencies = InDependencies.Num();

	OutDependencies.Reset(NumDependencies);
	for (int32 SerializeIndex : InDependencies)
	{
		FDependsNode* DependsNode = GetNodeFromSerializeIndex(SerializeIndex);
		if (!DependsNode)
		{
			Ar.SetError();
			return;
		}
		OutDependencies.Add(DependsNode);
	}

	Algo::Sort(OutDependencies, [](FDependsNode* A, FDependsNode* B)
		{
			return A < B;
		});
}

template <bool bLatestVersion>
void FDependsNode::SerializeLoad_ManageDependencies(FArchive& Ar,
	const TUniqueFunction<FDependsNode* (int32)>& GetNodeFromSerializeIndex,
	FLoadScratch& Scratch, FAssetRegistryVersion::Type Version)
{
	using namespace UE::AssetRegistry;

	// This code is duplicated-with-modifications in SerializeLoad_PackageDependencies,
	// and SerializeLoad_ManageDependencies, because sharing code makes it more difficult to read.
	TArray<FDependsNode*>& OutDependencies = this->ManageDependencies;
	TBitArray<>& OutFlagBits = this->ManageFlags;
	int OutFlagSetWidth = ManageFlagSetWidth;
	int InFlagSetWidth;
	if (bLatestVersion || Version >= FAssetRegistryVersion::ManageDependenciesCookRule)
	{
		InFlagSetWidth = OutFlagSetWidth;
	}
	else
	{
		// Dependencies before ManageDependenciesCookRule were a single bit.
		InFlagSetWidth = 1;
	}

	TArray<int32>& InDependencies = Scratch.InDependencies;
	TArray<uint32>& InFlagBits = Scratch.InFlagBits;
	TArray<FDependsNode*>& PointerDependencies = Scratch.PointerDependencies;
	TArray<int32>& SortIndexes = Scratch.SortIndexes;
	int32 InNumFlagBits = 0;

	InDependencies.Reset();
	Ar << InDependencies;
	int32 NumDependencies = InDependencies.Num();

	// We don't use BitArray::operator<< because we want to avoid the reallocation that operator<< does on load
	// and we want to avoid saving BitArray.Num when it can be derived from NumDependencies.
	InNumFlagBits = InFlagSetWidth * NumDependencies;
	const int32 InNumFlagWords = FBitSet::CalculateNumWords(InNumFlagBits);
	InFlagBits.SetNumUninitialized(InNumFlagWords);
	Ar.Serialize(InFlagBits.GetData(), InNumFlagWords * sizeof(uint32));

	PointerDependencies.Reset(NumDependencies);
	for (int32 SerializeIndex : InDependencies)
	{
		FDependsNode* DependsNode = GetNodeFromSerializeIndex(SerializeIndex);
		if (!DependsNode)
		{
			Ar.SetError();
			return;
		}
		PointerDependencies.Add(DependsNode);
	}

	SortIndexes.Reset(NumDependencies);
	for (int Index = 0; Index < NumDependencies; ++Index)
	{
		SortIndexes.Add(Index);
	}

	Algo::Sort(SortIndexes, [&PointerDependencies](int32 A, int32 B)
		{
			return PointerDependencies[A] < PointerDependencies[B];
		});

	OutDependencies.Empty(NumDependencies);
	for (int32 SortIndex : SortIndexes)
	{
		OutDependencies.Add(PointerDependencies[SortIndex]);
	}

	if (bLatestVersion || Version >= FAssetRegistryVersion::ManageDependenciesCookRule)
	{
		OutFlagBits.SetNumUninitialized(InNumFlagBits);
		uint32* InFlagBitsData = InFlagBits.GetData();
		for (int32 WriteIndex = 0; WriteIndex < NumDependencies; ++WriteIndex)
		{
			int32 ReadIndex = SortIndexes[WriteIndex];
			OutFlagBits.SetRangeFromRange(WriteIndex * OutFlagSetWidth, OutFlagSetWidth,
				InFlagBitsData, ReadIndex * OutFlagSetWidth);
		}
	}
	else
	{
		// Manage dependency sets before ManageDependenciesCookRule were lists of exactly 1 property value, with a
		// bit specifying EDependencyProperty::Direct or EDependencyProperty::None. They were all implicitly
		// EDependencyProperty::CookRule. We stored each as a single bit rather than storing them as a
		// TPropertyCombinationSet.
		// So to store the upgraded version as a TPropertyCombinationSet, upgrade the original bit into a single
		// EDependencyProperty, and add that single property value to the output TPropertyCombinationSet.
		TBitArray<> InFlagBitsAsBitArray;
		InFlagBitsAsBitArray.SetNumUninitialized(InNumFlagBits);
		InFlagBitsAsBitArray.SetRangeFromRange(0, InNumFlagBits, InFlagBits.GetData(), 0);

		int32 OutNumFlagBits = OutFlagSetWidth * NumDependencies;
		OutFlagBits.SetNumUninitialized(OutNumFlagBits);
		for (int32 WriteIndex = 0; WriteIndex < NumDependencies; ++WriteIndex)
		{
			int32 ReadIndex = SortIndexes[WriteIndex];
			bool bInputIsDirect = InFlagBitsAsBitArray[ReadIndex*InFlagSetWidth];
			EDependencyProperty UpgradedInputProperty =
				(bInputIsDirect ? EDependencyProperty::Direct : EDependencyProperty::None)
				| EDependencyProperty::CookRule;
			FManageFlagSet UpgradedInputFlagSet;
			UpgradedInputFlagSet.Add(ManagePropertiesToByte(UpgradedInputProperty));
			
			// Write the bits of the upgraded TPropertyCombinationSet into the properly sorted location in OutFlagBits.
			UpgradedInputFlagSet.Save(OutFlagBits, WriteIndex * OutFlagSetWidth);
		}
	}
}

template <bool bLatestVersion>
void FDependsNode::SerializeLoad_Referencers(FArchive& Ar,
	const TUniqueFunction<FDependsNode* (int32)>& GetNodeFromSerializeIndex,
	FLoadScratch& Scratch, FAssetRegistryVersion::Type Version)
{
	// This code is duplicated-with-modifications in SerializeLoad_NameDependencies
	// and SerializeLoad_Referencers, because sharing code makes it more difficult to read.
	TArray<FDependsNode*>& OutDependencies = this->Referencers;

	TArray<int32>& InDependencies = Scratch.InDependencies;
	InDependencies.Reset();
	Ar << InDependencies;
	int32 NumDependencies = InDependencies.Num();

	OutDependencies.Reset(NumDependencies);
	for (int32 SerializeIndex : InDependencies)
	{
		FDependsNode* DependsNode = GetNodeFromSerializeIndex(SerializeIndex);
		if (!DependsNode)
		{
			Ar.SetError();
			return;
		}
		OutDependencies.Add(DependsNode);
	}

	Algo::Sort(OutDependencies, [](FDependsNode* A, FDependsNode* B)
		{
			return A < B;
		});
}

template <bool bLatestVersion>
void FDependsNode::SerializeLoad(FArchive& Ar,
	const TUniqueFunction<FDependsNode* (int32)>& GetNodeFromSerializeIndex,
	FLoadScratch& Scratch, FAssetRegistryVersion::Type Version)
{
	using namespace UE::AssetRegistry;

	Ar << Identifier;
	SetIdentifier(Identifier);

	SerializeLoad_PackageDependencies<bLatestVersion>(Ar, GetNodeFromSerializeIndex, Scratch, Version);
	SerializeLoad_NameDependencies<bLatestVersion>(Ar, GetNodeFromSerializeIndex, Scratch, Version);
	SerializeLoad_ManageDependencies<bLatestVersion>(Ar, GetNodeFromSerializeIndex, Scratch, Version);
	SerializeLoad_Referencers<bLatestVersion>(Ar, GetNodeFromSerializeIndex, Scratch, Version);

	SetIsScriptDependenciesInitialized(IsScriptPath());
}

// We need Explicit instantiations of the template function because they are only called from other translation units.
template void FDependsNode::SerializeLoad<true>(FArchive& Ar,
	const TUniqueFunction<FDependsNode* (int32)>& GetNodeFromSerializeIndex,
	FLoadScratch& Scratch, FAssetRegistryVersion::Type Version);
template void FDependsNode::SerializeLoad<false>(FArchive& Ar,
	const TUniqueFunction<FDependsNode* (int32)>& GetNodeFromSerializeIndex,
	FLoadScratch& Scratch, FAssetRegistryVersion::Type Version);

void FDependsNode::LegacySerializeLoad_BeforeFlags(FArchive& Ar,
	FAssetRegistryVersion::Type Version,
	FDependsNode* PreallocatedDependsNodeDataBuffer,
	int32 NumDependsNodes,
	bool bSerializeDependencies)
{
	Ar << Identifier;
	SetIdentifier(Identifier);

	int32 NumHard, NumSoft, NumName, NumSoftManage, NumHardManage, NumReferencers;
	Ar << NumHard;
	Ar << NumSoft;
	Ar << NumName;
	Ar << NumSoftManage;
	if (Version < FAssetRegistryVersion::AddedHardManage)
	{
		NumHardManage = 0;
	}
	else
	{
		Ar << NumHardManage;
	}
	Ar << NumReferencers;

	// Empty dependency arrays and reserve space
	PackageDependencies.Empty(NumHard + NumSoft);
	NameDependencies.Empty(bSerializeDependencies ? NumName : 0);
	ManageDependencies.Empty(bSerializeDependencies ? NumSoftManage + NumHardManage : 0);
	Referencers.Empty(NumReferencers);

	auto SerializeNodeArray = [&Ar, PreallocatedDependsNodeDataBuffer, NumDependsNodes](int32 Num,
		TArray<FDependsNode*>& OutNodes,
		TBitArray<>* OutFlagBits,
		uint32 FlagSetWidth,
		uint32 FlagSetBits,
		bool bShouldOverwriteFlag,
		bool bAllowWrite)
	{
		for (int32 DependencyIndex = 0; DependencyIndex < Num; ++DependencyIndex)
		{
			int32 Index = 0;
			Ar << Index;
			if (Index < 0 || Index >= NumDependsNodes)
			{
				Ar.SetError();
				return;
			}
			if (bAllowWrite)
			{
				FDependsNode* DependsNode = &PreallocatedDependsNodeDataBuffer[Index];
				int32 OutNodeIndex = Algo::LowerBound(OutNodes, DependsNode);
				if (OutNodes.Num() <= OutNodeIndex || OutNodes[OutNodeIndex] != DependsNode)
				{
					OutNodes.Insert(DependsNode, OutNodeIndex);
					if (OutFlagBits)
					{
						OutFlagBits->InsertRange(&FlagSetBits, OutNodeIndex * FlagSetWidth, FlagSetWidth);
					}
				}
				else if (bShouldOverwriteFlag)
				{
					if (OutFlagBits)
					{
						OutFlagBits->SetRangeFromRange(OutNodeIndex * FlagSetWidth, FlagSetWidth, &FlagSetBits);
					}
				}
			}
		}
	};

	// Read the bits for each type, but don't write anything if serializing that type isn't allowed
	if (NumHard > 0)
	{
		// All legacy PackageDependencies were Game and Build. Hard had the same meaning as EDependencyProperty::Hard.
		uint32 HardBits = 0;
		FDependsNode::FPackageFlagSet FlagSet;
		FlagSet.Add(PackagePropertiesToByte(
			UE::AssetRegistry::EDependencyProperty::Hard
			| UE::AssetRegistry::EDependencyProperty::Game | UE::AssetRegistry::EDependencyProperty::Build));
		FlagSet.Save(&HardBits);
		SerializeNodeArray(NumHard, PackageDependencies, &PackageFlags, PackageFlagSetWidth, HardBits, true, true);
	}
	if (NumSoft > 0)
	{
		// All legacy PackageDependencies were Game and Build. Soft had the same meaning as !EDependencyProperty::Hard.
		uint32 SoftBits = 0;
		FDependsNode::FPackageFlagSet FlagSet;
		FlagSet.Add(PackagePropertiesToByte(
			UE::AssetRegistry::EDependencyProperty::Game | UE::AssetRegistry::EDependencyProperty::Build));
		FlagSet.Save(&SoftBits);
		SerializeNodeArray(NumSoft, PackageDependencies, &PackageFlags, PackageFlagSetWidth, SoftBits, false, true);
	}
	if (NumName > 0)
	{
		SerializeNodeArray(NumName, NameDependencies, nullptr, 0, 0, true, bSerializeDependencies);
	}
	if (NumSoftManage > 0)
	{
		// All legacy ManageDependencies were CookRule. Soft meant Indirect.
		uint32 SoftManageBits = 0;
		FDependsNode::FManageFlagSet FlagSet;
		FlagSet.Add(ManagePropertiesToByte(
			UE::AssetRegistry::EDependencyProperty::CookRule));
		FlagSet.Save(&SoftManageBits);
		SerializeNodeArray(NumSoftManage, ManageDependencies, &ManageFlags, ManageFlagSetWidth, SoftManageBits,
			true, bSerializeDependencies);
	}
	if (NumHardManage > 0)
	{
		// All legacy ManageDependencies were CookRule. Hard meant Direct.
		uint32 HardManageBits = 0;
		FDependsNode::FManageFlagSet FlagSet;
		FlagSet.Add(ManagePropertiesToByte(
			UE::AssetRegistry::EDependencyProperty::Direct
			| UE::AssetRegistry::EDependencyProperty::CookRule));
		FlagSet.Save(&HardManageBits);
		SerializeNodeArray(NumHardManage, ManageDependencies, &ManageFlags, ManageFlagSetWidth, HardManageBits,
			true, bSerializeDependencies);
	}
	if (NumReferencers > 0)
	{
		SerializeNodeArray(NumReferencers, Referencers, nullptr, 0, 0, true, true);
	}

	SetIsScriptDependenciesInitialized(IsScriptPath());
}

template <uint32 FlagWidth>
void SortDependencyList(TArray<FDependsNode*>& Dependencies, TBitArray<>* Flags, EAllowShrinking ShrinkPolicy)
{
	const int32 Num = Dependencies.Num();
	if (Num == 0)
	{
		return;
	}

	FDependsNode** const DependencyData = Dependencies.GetData();

	if (FlagWidth == 0)
	{
		// Sort dependencies+
		Algo::Sort(TArrayView<FDependsNode*>(DependencyData, Num));

		// Remove duplicates, which are now adjacent
		int32 WriteIndex;
		for (WriteIndex = 0; WriteIndex < Num - 1; ++WriteIndex)
		{
			if (DependencyData[WriteIndex] == DependencyData[WriteIndex + 1])
			{
				break;
			}
		}
		if (WriteIndex < Num - 1)
		{
			++WriteIndex;
			for (int32 ReadIndex = WriteIndex + 1; ReadIndex < Num; ++ReadIndex)
			{
				if (DependencyData[ReadIndex] != DependencyData[WriteIndex-1])
				{
					DependencyData[WriteIndex++] = DependencyData[ReadIndex];
				}
			}
			Dependencies.SetNum(WriteIndex, ShrinkPolicy);
		}
	}
	else
	{
		check(Flags);
		// Create an index array for sorting
		TArray<int32> Order;
		Order.SetNum(Num);
		for (int n = 0; n < Num; ++n)
		{
			Order[n] = n;
		}

		// Sort the index array
		Algo::Sort(Order, [&DependencyData](int32 A, int32 B)
			{
				return DependencyData[A] < DependencyData[B];
			});

		// Remove duplicate in the dependency array, which are now adjacent, and merge their corresponding flags
		typedef TPropertyCombinationSet<FlagWidth> FCombinationSet;
		constexpr uint32 FlagSetWidth = FCombinationSet::StorageBitCount;
		TArray<FDependsNode*> NewDependencies;
		TBitArray<> NewFlags;
		NewDependencies.Reserve(Num);
		NewFlags.Reserve(Num*FlagSetWidth);

		int ReadIndex = Order[0];
		NewDependencies.Add(DependencyData[ReadIndex]);
		NewFlags.AddRange(*Flags, FlagSetWidth, ReadIndex * FlagSetWidth);

		for (int OrderIndex = 1; OrderIndex < Num; ++OrderIndex)
		{
			ReadIndex = Order[OrderIndex];
			FDependsNode* DependencyToAdd = DependencyData[ReadIndex];
			if (DependencyToAdd != NewDependencies.Last())
			{
				NewDependencies.Add(DependencyToAdd);
				NewFlags.AddRange(*Flags, FlagSetWidth, ReadIndex * FlagSetWidth);
			}
			else
			{
				const int ExistingBitIndex = (NewDependencies.Num() - 1) * FlagSetWidth;
				FCombinationSet ExistingSet;
				ExistingSet.Load(NewFlags, ExistingBitIndex);
				FCombinationSet NewSet;
				NewSet.Load(*Flags, ReadIndex * FlagSetWidth);
				ExistingSet.AddRange(NewSet);
				ExistingSet.Save(NewFlags, ExistingBitIndex);
			}
		}
		Swap(Dependencies, NewDependencies);
		Swap(*Flags, NewFlags);
	}
}

bool FDependsNode::IsDependencyListSorted(UE::AssetRegistry::EDependencyCategory Category) const
{
	using namespace UE::AssetRegistry;
	if (Category == EDependencyCategory::Package)
	{
		return PackageIsSorted;
	}
	else if (Category == EDependencyCategory::SearchableName)
	{
		return SearchableNameIsSorted;
	}
	else if (Category == EDependencyCategory::Manage)
	{
		return ManageIsSorted;
	}
	else
	{
		check(false);
		return true;
	}
}

void FDependsNode::SetIsDependencyListSorted(UE::AssetRegistry::EDependencyCategory Category, bool bValue)
{
	using namespace UE::AssetRegistry;
	const EAllowShrinking ShrinkPolicy = bAllowShrinking ? EAllowShrinking::Yes : EAllowShrinking::No;
	if (!!(Category & EDependencyCategory::Package))
	{
		if (bValue && PackageIsSorted == 0)
		{
			SortDependencyList<PackageFlagWidth>(PackageDependencies, &PackageFlags, ShrinkPolicy);
		}
		PackageIsSorted = bValue ? 1 : 0;
	}
	if (!!(Category & EDependencyCategory::SearchableName))
	{
		if (bValue && SearchableNameIsSorted == 0)
		{
			SortDependencyList<SearchableNameFlagWidth>(NameDependencies, nullptr, ShrinkPolicy);
		}
		SearchableNameIsSorted = bValue ? 1 : 0;
	}
	if (!!(Category & EDependencyCategory::Manage))
	{
		if (bValue && ManageIsSorted == 0)
		{
			SortDependencyList<ManageFlagWidth>(ManageDependencies, &ManageFlags, ShrinkPolicy);
		}
		ManageIsSorted = bValue ? 1 : 0;
	}
}

bool FDependsNode::IsReferencersSorted() const
{
	return ReferencersIsSorted;
}

void FDependsNode::SetIsReferencersSorted(bool bValue)
{
	if (bValue && ReferencersIsSorted == 0)
	{
		const EAllowShrinking ShrinkPolicy = bAllowShrinking ? EAllowShrinking::Yes : EAllowShrinking::No;
		SortDependencyList<0>(Referencers, nullptr, ShrinkPolicy);
	}
	ReferencersIsSorted = bValue ? 1 : 0;
}

bool FDependsNode::IsScriptDependenciesInitialized() const
{
	return DependenciesInitialized;
}

void FDependsNode::SetIsScriptDependenciesInitialized(bool bValue)
{
	DependenciesInitialized = bValue ? 1 : 0;
}
