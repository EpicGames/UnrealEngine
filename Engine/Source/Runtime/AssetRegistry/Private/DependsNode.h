// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/BinarySearch.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Misc/AssetRegistryInterface.h"
#include "PropertyCombinationSet.h"

struct FDependsNodeReservations;

/** Implementation of IDependsNode */
class FDependsNode
{
public:
	typedef TArray<FDependsNode*> FDependsNodeList;
	static constexpr uint32 PackageFlagWidth = 3;
	static constexpr uint32 SearchableNameFlagWidth = 0;
	static constexpr uint32 ManageFlagWidth = 2;
	typedef TPropertyCombinationSet<PackageFlagWidth> FPackageFlagSet;
	static constexpr uint32 PackageFlagSetWidth = FPackageFlagSet::StorageBitCount;
	static constexpr uint32 SearchableNameFlagSetWidth = 0;
	typedef TPropertyCombinationSet<ManageFlagWidth> FManageFlagSet;
	static constexpr uint32 ManageFlagSetWidth = FManageFlagSet::StorageBitCount;

public:
	FDependsNode();
	FDependsNode(const FAssetIdentifier& InIdentifier);
	void Empty();

	/** Prints the dependencies and referencers for this node to the log */
	void PrintNode() const;
	/** Prints the dependencies for this node to the log */
	void PrintDependencies() const;
	/** Prints the referencers to this node to the log */
	void PrintReferencers() const;
	/** Gets the list of dependencies for this node */
	void GetDependencies(TArray<FDependsNode*>& OutDependencies,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const;
	/** Gets the list of dependency names for this node */
	void GetDependencies(TArray<FAssetIdentifier>& OutDependencies,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const;
	void GetDependencies(TArray<FAssetDependency>& OutDependencies,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const;
	/** Gets the list of referencers to this node */
	void GetReferencers(TArray<FDependsNode*>& OutReferencers,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const;
	void GetReferencers(TArray<FAssetDependency>& OutReferencers,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const;
	/** Helper function to return GetIdentifier.PackageName. */
	FName GetPackageName() const;
	/** Returns the entire identifier */
	const FAssetIdentifier& GetIdentifier() const;
	bool IsScriptPath() const;
	/** Sets the entire identifier */
	void SetIdentifier(const FAssetIdentifier& InIdentifier);
	/** Add a dependency to this node */
	void AddDependency(FDependsNode* InDependency,
		UE::AssetRegistry::EDependencyCategory InDependencyType, UE::AssetRegistry::EDependencyProperty InProperties);
	void GetPackageReferencers(TArray<TPair<FAssetIdentifier, FPackageFlagSet>>& OutReferencers);
	void AddPackageDependencySet(FDependsNode* InDependency, const FPackageFlagSet& PropertyCombinationSet);
	/** Add a referencer to this node */
	void AddReferencer(FDependsNode* InReferencer);
	/** Remove a dependency from this node */
	void RemoveDependency(FDependsNode* InDependency,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All);
	/** Remove a referencer from this node */
	void RemoveReferencer(FDependsNode* InReferencer);
	/** Remove a set of referencers from this node */
	void RemoveReferencers(const TSet<FDependsNode*>& InReferencers);
	/** Removes any referencers that no longer have this node as a dependency */
	void RefreshReferencers();
	bool ContainsDependency(const FDependsNode* InDependency,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const;
	/** Clear all dependency records from this node */
	void ClearDependencies(UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All);
	void ClearReferencers();
	/**
	 * Removes Manage dependencies on this node and clean up referencers array.
	 * Manage references are the only ones safe to remove at runtime.
	 */
	void RemoveManageReferencesToNode();
	/** Remove all nodes from referencers and dependencies for which ShouldRemove returns true */
	void RemoveLinks(const TUniqueFunction<bool(const FDependsNode*)>& ShouldRemove);
	/** Replace nodes in referencers and dependencies with OldNodeToNewNode[Target]; remove any that remap to null. */
	void RemapLinks(const TMap<FDependsNode*, FDependsNode*>& OldNodeToNewNode)
	{
		RemapLinks([&OldNodeToNewNode](FDependsNode* Target)
			{
				FDependsNode*const* NewNodePtr = OldNodeToNewNode.Find(Target);
				return NewNodePtr ? *NewNodePtr : nullptr;
			});
	}
	/** Replace nodes in referencers and dependencies with the Remapper[Target]; remove any that Remap to null. */
	void RemapLinks(const TUniqueFunction<FDependsNode* (FDependsNode*)>& Remapper);
	/** Replace nodes in dependencies with the Remapper[Target]; remove any that Remap to null. */
	void RemapDependencies(const TUniqueFunction<FDependsNode* (FDependsNode*)>& Remapper);
	/** Replace nodes in referencers with the Remapper[Target]; remove any that Remap to null. */
	void RemapReferencers(const TUniqueFunction<FDependsNode* (FDependsNode*)>& Remapper);

	/** Returns number of connections this node has, both references and dependencies */
	int32 GetConnectionCount() const;
	/** Returns amount of memory used by the arrays */
	SIZE_T GetAllocatedSize(void) const;

	typedef TUniqueFunction<void(
		/** The other node the source node has a link to. */
		FDependsNode* Dependency,
		/**
		 * The category of the dependency; dependencies are divided into high level categories,
		 * @see EDependencyCategory.
		 */
		UE::AssetRegistry::EDependencyCategory Category,
		/**
		 * The properties of the dependency within its category. Each category has properties for dependencies in that category.
		 * @see EDependencyProperty.
		 */
		UE::AssetRegistry::EDependencyProperty Properties,
		/**
		 * The source node may have multiple links to the targetnode, in different categories or with different
		 * property combinations within the category. e.g. A package might have a Soft Game reference to another
		 * package, but also a hard EditorOnly reference to that same package. When this occurs, and multiple links to
		 * to the same Dependency are reported to an FIterateDependenciesCallback, all of the links to the same 
		 * node are iterated consecutively, and bDuplicate=true for each of the reports after the first.
		 * For the first or only occurrence of a Dependency in the iteration, bDuplicate=false.
		 */
		bool bDuplicate
		)> FIterateDependenciesCallback;
	/**
	 * Iterate over all the dependencies of this node, optionally filtered by the target node, category and query,
	 * and call the supplied lambda parameter on the record.
	 */
	void IterateOverDependencies(const FIterateDependenciesCallback& InCallback,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const;
	void IterateOverDependencies(const FIterateDependenciesCallback& InCallback,
		const FDependsNode* DependsNode,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const;

	/** Iterate over all the referencers of this node and call the supplied lambda parameter on the referencer */
	template <class T>
	void IterateOverReferencers(const T& InCallback) const;

	void Reserve(int32 InNumPackageDependencies, int32 InNumNameDependencies,
		int32 InNumManageDependencies, int32 InNumReferencers);
	void Reserve(const FDependsNodeReservations& InReservations);
	void Reserve(const FDependsNode* Other);

	bool GetAllowShrinking() const;
	void SetAllowShrinking(bool bAllowShrinking);

	static uint8 PackagePropertiesToByte(UE::AssetRegistry::EDependencyProperty Properties);
	static UE::AssetRegistry::EDependencyProperty ByteToPackageProperties(uint8 Bits);
	static uint8 ManagePropertiesToByte(UE::AssetRegistry::EDependencyProperty Properties);
	static UE::AssetRegistry::EDependencyProperty ByteToManageProperties(uint8 Bits);

	struct FSaveScratch
	{
		struct FSortInfo
		{
			int32 SerializeIndex;
			int32 ListIndex;
		};
		TArray<FSortInfo> SortInfos;
		TArray<int32> OutDependencies;
		TBitArray<> OutFlagBits;
	};
	void SerializeSave(FArchive& Ar, const TUniqueFunction<int32(FDependsNode*, bool)>& GetSerializeIndexFromNode,
		FSaveScratch& Scratch, const FAssetRegistrySerializationOptions& Options) const;
	struct FLoadScratch
	{
		TArray<int32> InDependencies;
		TArray<uint32> InFlagBits;
		TArray<FDependsNode*> PointerDependencies;
		TArray<int32> SortIndexes;
	};
	template <bool bLatestVersion>
	void SerializeLoad(FArchive& Ar,
		const TUniqueFunction<FDependsNode* (int32)>& GetNodeFromSerializeIndex,
		FLoadScratch& Scratch, FAssetRegistryVersion::Type Version);

	void LegacySerializeLoad_BeforeFlags(FArchive& Ar, FAssetRegistryVersion::Type Version,
		FDependsNode* PreallocatedDependsNodeDataBuffer, int32 NumDependsNodes, bool bSerializeDependencies);

	bool IsDependencyListSorted(UE::AssetRegistry::EDependencyCategory Category) const;
	void SetIsDependencyListSorted(UE::AssetRegistry::EDependencyCategory Category, bool bValue);
	bool IsReferencersSorted() const;
	void SetIsReferencersSorted(bool bValue);
	/** 
	* Returns true if an IsScriptPath() FDependsNode has been initialized. FDependsNodes 
	* that are not IsScriptPath() will always return false. 
	*/
	bool IsScriptDependenciesInitialized() const;
	void SetIsScriptDependenciesInitialized(bool bValue);

private:

	/**
	 * Recursively prints dependencies of the node starting with the specified indent.
	 * VisitedNodes should be an empty set at first which is populated recursively.
	 */
	void PrintDependenciesRecursive(const FString& Indent, TSet<const FDependsNode*>& VisitedNodes) const;
	/**
	 * Recursively prints referencers to the node starting with the specified indent.
	 * VisitedNodes should be an empty set at first which is populated recursively.
	 */
	void PrintReferencersRecursive(const FString& Indent, TSet<const FDependsNode*>& VisitedNodes) const;

	template <bool bLatestVersion>
	void SerializeLoad_PackageDependencies(FArchive& Ar,
		const TUniqueFunction<FDependsNode* (int32)>& GetNodeFromSerializeIndex,
		FLoadScratch& Scratch, FAssetRegistryVersion::Type Version);
	template <bool bLatestVersion>
	void SerializeLoad_NameDependencies(FArchive& Ar,
		const TUniqueFunction<FDependsNode* (int32)>& GetNodeFromSerializeIndex,
		FLoadScratch& Scratch, FAssetRegistryVersion::Type Version);
	template <bool bLatestVersion>
	void SerializeLoad_ManageDependencies(FArchive& Ar,
		const TUniqueFunction<FDependsNode* (int32)>& GetNodeFromSerializeIndex,
		FLoadScratch& Scratch, FAssetRegistryVersion::Type Version);
	template <bool bLatestVersion>
	void SerializeLoad_Referencers(FArchive& Ar,
		const TUniqueFunction<FDependsNode* (int32)>& GetNodeFromSerializeIndex,
		FLoadScratch& Scratch, FAssetRegistryVersion::Type Version);

	/** The name of the package/object this node represents */
	FAssetIdentifier Identifier;
	FDependsNodeList PackageDependencies;
	FDependsNodeList NameDependencies;
	FDependsNodeList ManageDependencies;
	FDependsNodeList Referencers;
	TBitArray<> PackageFlags;
	TBitArray<> ManageFlags;

	// Transient flags that are not serialized
	uint32 PackageIsSorted : 1 = 1;
	uint32 SearchableNameIsSorted : 1 = 1;
	uint32 ManageIsSorted : 1 = 1;
	uint32 ReferencersIsSorted : 1 = 1;
	uint32 DependenciesInitialized : 1 = 0;
	uint32 bScriptPath : 1 = 0;
	/** Used to control FDependsNodeList shrinking behaviour */
	uint32 bAllowShrinking : 1 = 1;

	friend FDependsNodeReservations;
};

struct FDependsNodeReservations
{
	int32 PackageDependenciesSize = 0;
	int32 NameDependenciesSize = 0;
	int32 ManageDependenciesSize = 0;
	int32 ReferencersSize = 0;
	
	FDependsNodeReservations() = default;
	FDependsNodeReservations(const FDependsNode& Node)
	{
		PackageDependenciesSize = Node.PackageDependencies.Num();
		NameDependenciesSize = Node.NameDependencies.Num();
		ManageDependenciesSize = Node.ManageDependencies.Num();
		ReferencersSize = Node.Referencers.Num();
	}

	friend FArchive& operator<<(FArchive& Ar, FDependsNodeReservations& Reservations)
	{
		Ar << Reservations.PackageDependenciesSize << Reservations.NameDependenciesSize << Reservations.ManageDependenciesSize << Reservations.ReferencersSize;
		return Ar;
	}
};

///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////

inline FDependsNode::FDependsNode()
{
}

inline FDependsNode::FDependsNode(const FAssetIdentifier& InIdentifier)
{
	SetIdentifier(InIdentifier);
}

inline void FDependsNode::Empty()
{
	SetAllowShrinking(true);
	ClearDependencies();
	ClearReferencers();
}

inline FName FDependsNode::GetPackageName() const
{
	return Identifier.PackageName;
}

inline const FAssetIdentifier& FDependsNode::GetIdentifier() const
{
	return Identifier;
}

inline bool FDependsNode::IsScriptPath() const
{
	return bScriptPath != 0;
}

inline SIZE_T FDependsNode::GetAllocatedSize(void) const
{
	return PackageDependencies.GetAllocatedSize() + PackageFlags.GetAllocatedSize()
		+ NameDependencies.GetAllocatedSize() + ManageDependencies.GetAllocatedSize() + ManageFlags.GetAllocatedSize()
		+ Referencers.GetAllocatedSize();
}

template <class T>
inline void FDependsNode::IterateOverReferencers(const T& InCallback) const
{
	for (FDependsNode* Referencer : Referencers)
	{
		InCallback(Referencer);
	}
}

inline void FDependsNode::Reserve(int32 InNumPackageDependencies, int32 InNumNameDependencies,
	int32 InNumManageDependencies, int32 InNumReferencers)
{
	PackageDependencies.Reserve(InNumPackageDependencies);
	PackageFlags.Reserve(InNumPackageDependencies * PackageFlagSetWidth);
	NameDependencies.Reserve(InNumNameDependencies);
	ManageDependencies.Reserve(InNumManageDependencies);
	ManageFlags.Reserve(InNumManageDependencies * ManageFlagSetWidth);
	Referencers.Reserve(InNumReferencers);
}

inline void FDependsNode::Reserve(const FDependsNodeReservations& InReservations)
{
	Reserve(InReservations.PackageDependenciesSize, InReservations.NameDependenciesSize,
		InReservations.ManageDependenciesSize, InReservations.ReferencersSize);
}

inline void FDependsNode::Reserve(const FDependsNode* Other)
{
	Reserve(Other->PackageDependencies.Num(), Other->NameDependencies.Num(),
		Other->ManageDependencies.Num(), Other->Referencers.Num());
}

inline bool FDependsNode::GetAllowShrinking() const
{
	return bAllowShrinking;
}

inline void FDependsNode::SetAllowShrinking(bool InAllowShrinking)
{
	bAllowShrinking = InAllowShrinking;
}

inline uint8 FDependsNode::PackagePropertiesToByte(UE::AssetRegistry::EDependencyProperty Properties)
{
	return (0x01 * (static_cast<uint8>(Properties & UE::AssetRegistry::EDependencyProperty::Hard) != 0))
		| (0x02 * (static_cast<uint8>(Properties & UE::AssetRegistry::EDependencyProperty::Game) != 0))
		| (0x04 * (static_cast<uint8>(Properties & UE::AssetRegistry::EDependencyProperty::Build) != 0));
}

inline UE::AssetRegistry::EDependencyProperty FDependsNode::ByteToPackageProperties(uint8 Bits)
{
	return static_cast<UE::AssetRegistry::EDependencyProperty>(
		(static_cast<uint32>(UE::AssetRegistry::EDependencyProperty::Hard) * ((Bits & 0x01) != 0))
		| (static_cast<uint32>(UE::AssetRegistry::EDependencyProperty::Game) * ((Bits & 0x02) != 0))
		| (static_cast<uint32>(UE::AssetRegistry::EDependencyProperty::Build) * ((Bits & 0x04) != 0))
		);
}
inline uint8 FDependsNode::ManagePropertiesToByte(UE::AssetRegistry::EDependencyProperty Properties)
{
	return (0x01 * (static_cast<uint8>(Properties & UE::AssetRegistry::EDependencyProperty::Direct) != 0))
		| (0x02 * (static_cast<uint8>(Properties & UE::AssetRegistry::EDependencyProperty::CookRule) != 0));
}

inline UE::AssetRegistry::EDependencyProperty FDependsNode::ByteToManageProperties(uint8 Bits)
{
	return static_cast<UE::AssetRegistry::EDependencyProperty>(
		(static_cast<uint32>(UE::AssetRegistry::EDependencyProperty::Direct) * ((Bits & 0x01) != 0))
		| (static_cast<uint32>(UE::AssetRegistry::EDependencyProperty::CookRule) * ((Bits & 0x02) != 0))
		);
}
