// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "AssetRegistry/AssetRegistryState.h"
#include "DependsNode.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDependsNodeTest, "System.AssetRegistry.DependsNode", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);

namespace
{
	template <typename ElementType>
	bool CombinationEquals(TArrayView<const ElementType> A, TArrayView<const ElementType> B)
	{
		if (A.Num() != B.Num())
		{
			return false;
		}

		for (const ElementType& AObj : A)
		{
			int ACount = 0;
			for (const ElementType& AObj2 : A)
			{
				if (AObj2 == AObj)
				{
					++ACount;
				}
			}
			int BCount = 0;
			for (const ElementType& BObj : B)
			{
				if (BObj == AObj)
				{
					++BCount;
				}
			}
			if (ACount != BCount)
			{
				return false;
			}
		}
		return true;
	}
	template <typename ElementType>
	static bool CombinationEquals(TArrayView<const ElementType> A, const ElementType& B)
	{
		return CombinationEquals(A, TArrayView<const ElementType>(&B, 1));
	}
}

bool FDependsNodeTest::RunTest(const FString& Parameters)
{
	using namespace UE::AssetRegistry;

	TArray<FDependsNode*> Dependencies;
	TArray<FDependsNode*> Referencers;
	EDependencyCategory AllCategories[] = { EDependencyCategory::Package, EDependencyCategory::SearchableName, EDependencyCategory::Manage };
	constexpr int CategoryNum = UE_ARRAY_COUNT(AllCategories);
	FDependsNode ScratchNodes[10];

	// Added dependency is found by GetDependencies and ContainsDependency and GetReferencers
	{
		for (EDependencyCategory Category : {EDependencyCategory::Package, EDependencyCategory::SearchableName, EDependencyCategory::Manage})
		{
			FDependsNode A;
			FDependsNode* PointerA = &A;
			FDependsNode* B = &ScratchNodes[0];
			A.AddDependency(B, Category, EDependencyProperty::None);
			B->AddReferencer(PointerA);

			Dependencies.Reset();
			A.GetDependencies(Dependencies);
			Referencers.Reset();
			B->GetReferencers(Referencers);
			TestTrue(TEXT("GetDependencies single"), CombinationEquals<FDependsNode*>(Dependencies, B));
			TestTrue(TEXT("ContainsDependency AllCategory single"), A.ContainsDependency(B, EDependencyCategory::All));
			TestTrue(TEXT("ContainsDependency SpecificCategory single"), A.ContainsDependency(B, Category));
			TestTrue(TEXT("GetReferencers single"), CombinationEquals<FDependsNode*>(Referencers, PointerA));

			A.RemoveDependency(B, Category);
			B->RemoveReferencer(PointerA);
			Dependencies.Reset();
			A.GetDependencies(Dependencies);
			Referencers.Reset();
			B->GetReferencers(Referencers);
			TestTrue(TEXT("RemoveDependency"), Dependencies.Num() == 0);
			TestTrue(TEXT("ContainsDependency AllCategory single"), Referencers.Num() == 0);
		}
	}

	// Added dependency is found by GetDependencies and ContainsDependency when multiple dependencies are present
	{
		int NodeIndex = 0;
		FDependsNode A;
		FDependsNode* PointerA = &A;
		TArray<FDependsNode*> Expected[CategoryNum];
		int CategoryIndex = 0;
		for (EDependencyCategory Category : AllCategories)
		{
			check(NodeIndex < UE_ARRAY_COUNT(ScratchNodes));
			Expected[CategoryIndex].Add(&ScratchNodes[NodeIndex++]);
			Expected[CategoryIndex].Add(&ScratchNodes[NodeIndex++]);
			A.AddDependency(Expected[CategoryIndex][0], Category, EDependencyProperty::None);
			A.AddDependency(Expected[CategoryIndex][1], Category, EDependencyProperty::None);
			Expected[CategoryIndex][0]->AddReferencer(&A);
			Expected[CategoryIndex][1]->AddReferencer(&A);
			++CategoryIndex;
		}

		CategoryIndex = 0;
		for (EDependencyCategory Category : AllCategories)
		{
			Dependencies.Reset();
			A.GetDependencies(Dependencies, Category);
			TestTrue(TEXT("GetDependencies multiple"), CombinationEquals<FDependsNode*>(Dependencies, Expected[CategoryIndex]));
			for (FDependsNode* ExpectedNode : Expected[CategoryIndex])
			{
				TestTrue(TEXT("ContainsDependency AllCategory multiple"), A.ContainsDependency(ExpectedNode, EDependencyCategory::All));
				TestTrue(TEXT("ContainsDependency SpecificCategory multiple"), A.ContainsDependency(ExpectedNode, Category));
				Referencers.Reset();
				ExpectedNode->GetReferencers(Referencers, Category);
				TestTrue(TEXT("GetReferencers multiple"), CombinationEquals<FDependsNode*>(Referencers, PointerA));
			}
			++CategoryIndex;
		}

		CategoryIndex = 0;
		for (EDependencyCategory Category : AllCategories)
		{
			Expected[CategoryIndex][0]->RemoveReferencer(&A);
			Expected[CategoryIndex][1]->RemoveReferencer(&A);
			++CategoryIndex;
		}
	}

	// Added dependency and referencer is found when filtering by Flags
	enum class ESortTestType
	{
		AlwaysSorted,
		AlwaysUnsorted,
		UnsortedThenSorted,
	};
	for (ESortTestType SortTestType : { ESortTestType::AlwaysSorted, ESortTestType::AlwaysUnsorted, ESortTestType::UnsortedThenSorted })
	{
		TArray<FAssetDependency> ReferencersStructs;
		int NodeIndex = 0;
		FDependsNode A;
		FName IdentifierA("IdentifierA");
		A.SetIdentifier(IdentifierA);
		FDependsNode* PointerA = &A;

		struct FTestDependency
		{
			FDependsNode* Node;
			EDependencyCategory Category;
			EDependencyProperty Properties;

			FTestDependency(const FTestDependency& Other, FDependsNode* NewNode)
				:Node(NewNode), Category(Other.Category), Properties(Other.Properties)
			{}
			FTestDependency(FDependsNode* InNode, EDependencyCategory InCategory, EDependencyProperty InProperties)
				:Node(InNode), Category(InCategory), Properties(InProperties)
			{}
			bool operator==(const FTestDependency& Other) const
			{
				return Node == Other.Node && Category == Other.Category && Properties == Other.Properties;
			}
			FAssetDependency AsAssetDependency() const
			{
				return FAssetDependency{ Node->GetIdentifier(), Category, Properties };
			}
		};
		FTestDependency TestDependencies[] = {
			{&ScratchNodes[0], EDependencyCategory::Package, EDependencyProperty::Hard | EDependencyProperty::Game | EDependencyProperty::Build},
			{&ScratchNodes[1], EDependencyCategory::Package, EDependencyProperty::Hard},
			{&ScratchNodes[2], EDependencyCategory::Package, EDependencyProperty::None},
			{&ScratchNodes[3], EDependencyCategory::SearchableName, EDependencyProperty::None},
			{&ScratchNodes[4], EDependencyCategory::Manage, EDependencyProperty::Direct | EDependencyProperty::CookRule},
			{&ScratchNodes[5], EDependencyCategory::Manage, EDependencyProperty::Direct},
			{&ScratchNodes[6], EDependencyCategory::Manage, EDependencyProperty::None }
		};

		constexpr int NumUsedScratchNodes = UE_ARRAY_COUNT(TestDependencies);
		static_assert(NumUsedScratchNodes <= UE_ARRAY_COUNT(ScratchNodes));
		if (SortTestType == ESortTestType::AlwaysUnsorted || SortTestType == ESortTestType::UnsortedThenSorted)
		{
			A.SetIsDependencyListSorted(EDependencyCategory::All, false);
			for (int n = 0; n < NumUsedScratchNodes; ++n)
			{
				ScratchNodes[n].SetIsReferencersSorted(false);
			}
		}
		for (EDependencyCategory Category : { EDependencyCategory::Package, EDependencyCategory::SearchableName, EDependencyCategory::Manage})
		{
			TestEqual(TEXT("IsDependencyListSorted"), A.IsDependencyListSorted(Category), SortTestType == ESortTestType::AlwaysSorted);
		}
		TestEqual(TEXT("IsReferencersSorted"), ScratchNodes[0].IsReferencersSorted(), SortTestType == ESortTestType::AlwaysSorted);


		for (FTestDependency& TestDependency : TestDependencies)
		{
			A.AddDependency(TestDependency.Node, TestDependency.Category, TestDependency.Properties);
		}
		for (int n = 0; n < NumUsedScratchNodes; ++n)
		{
			ScratchNodes[n].AddReferencer(&A);
		}

		if (SortTestType == ESortTestType::UnsortedThenSorted)
		{
			A.SetIsDependencyListSorted(UE::AssetRegistry::EDependencyCategory::All, true);
			for (int n = 0; n < NumUsedScratchNodes; ++n)
			{
				ScratchNodes[n].SetIsReferencersSorted(true);
			}
		}

		// Package All
		{
			FTestDependency MatchingDeps[] = { TestDependencies[0], TestDependencies[1], TestDependencies[2] };
			FDependsNode* Matching[] = { &ScratchNodes[0], &ScratchNodes[1], &ScratchNodes[2] };
			Dependencies.Reset();
			A.GetDependencies(Dependencies, EDependencyCategory::Package);
			TestTrue(TEXT("Package dependency with no flags"), CombinationEquals<FDependsNode*>(Dependencies, Matching));
			for (int MatchingIndex = 0; MatchingIndex < UE_ARRAY_COUNT(Matching); ++MatchingIndex)
			{
				FDependsNode* Dependency = Matching[MatchingIndex];
				FTestDependency& TestDependency = MatchingDeps[MatchingIndex];
				Referencers.Reset();
				ReferencersStructs.Reset();
				Dependency->GetReferencers(Referencers, EDependencyCategory::Package);
				Dependency->GetReferencers(ReferencersStructs, EDependencyCategory::Package);
				TestTrue(TEXT("Package referencers with no flags"), CombinationEquals<FDependsNode*>(Referencers, PointerA));
				TestTrue(TEXT("Package referencers with no flags"), CombinationEquals<FAssetDependency>(ReferencersStructs, FTestDependency(TestDependency, PointerA).AsAssetDependency()));
				int IterateCount = 0;
				Dependency->IterateOverReferencers([this, &IterateCount, &PointerA](FDependsNode* Referencer)
					{
						++IterateCount;
						if (Referencer != PointerA)
						{
							AddError(TEXT("IterateReferencers Package, no flags"));
						}
					});
				TestEqual(TEXT("IterateReferencers Package no flags, Count"), IterateCount, 1);
			}
			{
				int IterateCount = 0;
				A.IterateOverDependencies([this, &IterateCount, &ScratchNodes](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
					{
						TestTrue(TEXT("IterateDependencies Package, no flags, Category"), Category == EDependencyCategory::Package);
						++IterateCount;
						if (Dependency == &ScratchNodes[0])
						{
							TestTrue(TEXT("IterateDependencies Package, no flags"), Properties == (EDependencyProperty::Hard | EDependencyProperty::Game | EDependencyProperty::Build) && !bDuplicate);
						}
						else if (Dependency == &ScratchNodes[1])
						{
							TestTrue(TEXT("IterateDependencies Package, no flags"), Properties == EDependencyProperty::Hard && !bDuplicate);
						}
						else if (Dependency == &ScratchNodes[2])
						{
							TestTrue(TEXT("IterateDependencies Package, no flags"), Properties == EDependencyProperty::None && !bDuplicate);
						}
						else
						{
							AddError(TEXT("IterateDependencies Package, no flags"));
						}
					}, EDependencyCategory::Package);
				TestEqual(TEXT("IterateDependencies Package, no flags, Count"), IterateCount, 3);
			}
		}
		// Package Hard
		{
			FTestDependency MatchingDeps[] = { TestDependencies[0], TestDependencies[1] };
			FDependsNode* Matching[] = { &ScratchNodes[0], &ScratchNodes[1] };
			FDependsNode* NonMatching[] = { &ScratchNodes[2] };
			Dependencies.Reset();
			A.GetDependencies(Dependencies, EDependencyCategory::Package, EDependencyQuery::Hard);
			TestTrue(TEXT("Package dependency with flags, Hard"), CombinationEquals<FDependsNode*>(Dependencies, Matching));
			for (int MatchingIndex = 0; MatchingIndex < UE_ARRAY_COUNT(Matching); ++MatchingIndex)
			{
				FDependsNode* Dependency = Matching[MatchingIndex];
				FTestDependency TestDependency = MatchingDeps[MatchingIndex];
				Referencers.Reset();
				Dependency->GetReferencers(Referencers, EDependencyCategory::Package, EDependencyQuery::Hard);
				ReferencersStructs.Reset();
				Dependency->GetReferencers(ReferencersStructs, EDependencyCategory::Package, EDependencyQuery::Hard);
				TestTrue(TEXT("Package referencers with flags, Hard, matching"), CombinationEquals<FDependsNode*>(Referencers, PointerA));
				TestTrue(TEXT("Package referencers with flags, Hard, matching"), CombinationEquals<FAssetDependency>(ReferencersStructs, FTestDependency(TestDependency, PointerA).AsAssetDependency()));
				int IterateCount = 0;
				Dependency->IterateOverReferencers([this, &IterateCount, PointerA](FDependsNode* Referencer)
					{
						++IterateCount;
						if (Referencer != PointerA)
						{
							AddError(TEXT("IterateReferencers Package, Hard, matching"));
						}
					});
				TestEqual(TEXT("IterateReferencers Package Hard, matching, Count"), IterateCount, 1);
			}
			for (FDependsNode* Dependency : NonMatching)
			{
				Referencers.Reset();
				Dependency->GetReferencers(Referencers, EDependencyCategory::Package, EDependencyQuery::Hard);
				TestTrue(TEXT("Package referencers with flags, Hard, nonmatching"), Referencers.Num() == 0);
				int IterateCount = 0;
				Dependency->IterateOverReferencers([this, &IterateCount, PointerA](FDependsNode* Referencer)
					{
						++IterateCount;
						if (Referencer != PointerA)
						{
							AddError(TEXT("IterateReferencers Package, Hard, nonmatching"));
						}
					});
				TestEqual(TEXT("IterateReferencers Package Hard, nonmatching, Count"), IterateCount, 1);
			}
			{
				int IterateCount = 0;
				A.IterateOverDependencies([this, &IterateCount, &ScratchNodes](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
					{
						TestTrue(TEXT("IterateDependencies Package, Hard, Category"), Category == EDependencyCategory::Package);
						++IterateCount;
						if (Dependency == &ScratchNodes[0])
						{
							TestTrue(TEXT("IterateDependencies Package, Hard"), Properties == (EDependencyProperty::Hard | EDependencyProperty::Game | EDependencyProperty::Build) && !bDuplicate);
						}
						else if (Dependency == &ScratchNodes[1])
						{
							TestTrue(TEXT("IterateDependencies Package, Hard"), Properties == EDependencyProperty::Hard && !bDuplicate);
						}
						else
						{
							AddError(TEXT("IterateDependencies Package, Hard"));
						}
					}, EDependencyCategory::Package, EDependencyQuery::Hard);
				TestEqual(TEXT("IterateDependencies Package, Hard, Count"), IterateCount, 2);
			}
		}
		// Package Soft
		{
			FTestDependency MatchingDeps[] = { TestDependencies[2] };
			FDependsNode* Matching[] = { &ScratchNodes[2] };
			FDependsNode* NonMatching[] = { &ScratchNodes[0], &ScratchNodes[1] };
			Dependencies.Reset();
			A.GetDependencies(Dependencies, EDependencyCategory::Package, EDependencyQuery::Soft);
			TestTrue(TEXT("Package dependency with flags, Soft"), CombinationEquals<FDependsNode*>(Dependencies, Matching));
			for (int MatchingIndex = 0; MatchingIndex < UE_ARRAY_COUNT(Matching); ++MatchingIndex)
			{
				FDependsNode* Dependency = Matching[MatchingIndex];
				FTestDependency TestDependency = MatchingDeps[MatchingIndex];
				Referencers.Reset();
				Dependency->GetReferencers(Referencers, EDependencyCategory::Package, EDependencyQuery::Soft);
				ReferencersStructs.Reset();
				Dependency->GetReferencers(ReferencersStructs, EDependencyCategory::Package, EDependencyQuery::Soft);
				TestTrue(TEXT("Package referencers with flags, Soft, matching"), CombinationEquals<FDependsNode*>(Referencers, PointerA));
				TestTrue(TEXT("Package referencers with flags, Soft, matching"), CombinationEquals<FAssetDependency>(ReferencersStructs, FTestDependency(TestDependency, PointerA).AsAssetDependency()));
				int IterateCount = 0;
				Dependency->IterateOverReferencers([this, &IterateCount, PointerA](FDependsNode* Referencer)
					{
						++IterateCount;
						if (Referencer != PointerA)
						{
							AddError(TEXT("IterateReferencers Package, Soft, matching"));
						}
					});
				TestEqual(TEXT("IterateReferencers Package Soft, matching, Count"), IterateCount, 1);
			}
			for (FDependsNode* Dependency : NonMatching)
			{
				Referencers.Reset();
				Dependency->GetReferencers(Referencers, EDependencyCategory::Package, EDependencyQuery::Soft);
				TestTrue(TEXT("Package referencers with flags, Soft, nonmatching"), Referencers.Num() == 0);
				int IterateCount = 0;
				Dependency->IterateOverReferencers([this, &IterateCount, PointerA](FDependsNode* Referencer)
					{
						++IterateCount;
						if (Referencer != PointerA)
						{
							AddError(TEXT("IterateReferencers Package, Soft, nonmatching"));
						}
					});
				TestEqual(TEXT("IterateReferencers Package Soft, nonmatching, Count"), IterateCount, 1);
			}
			{
				int IterateCount = 0;
				A.IterateOverDependencies([this, &IterateCount, &ScratchNodes](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
					{
						TestTrue(TEXT("IterateDependencies Package, Soft, Category"), Category == EDependencyCategory::Package);
						++IterateCount;
						if (Dependency == &ScratchNodes[2])
						{
							TestTrue(TEXT("IterateDependencies Package, Soft"), Properties == EDependencyProperty::None && !bDuplicate);
						}
						else
						{
							AddError(TEXT("IterateDependencies Package, Soft"));
						}
					}, EDependencyCategory::Package, EDependencyQuery::Soft);
				TestEqual(TEXT("IterateDependencies Package, Soft, Count"), IterateCount, 1);
			}
		}
		// SearchableName All
		{
			FTestDependency MatchingDeps[] = { TestDependencies[3] };
			FDependsNode* Matching[] = { &ScratchNodes[3] };
			Dependencies.Reset();
			A.GetDependencies(Dependencies, EDependencyCategory::SearchableName);
			TestTrue(TEXT("SearchableName dependency with no flags"), CombinationEquals<FDependsNode*>(Dependencies, Matching));
			for (int MatchingIndex = 0; MatchingIndex < UE_ARRAY_COUNT(Matching); ++MatchingIndex)
			{
				FDependsNode* Dependency = Matching[MatchingIndex];
				FTestDependency TestDependency = MatchingDeps[MatchingIndex];
				Referencers.Reset();
				Dependency->GetReferencers(Referencers, EDependencyCategory::SearchableName);
				ReferencersStructs.Reset();
				Dependency->GetReferencers(ReferencersStructs, EDependencyCategory::SearchableName);
				TestTrue(TEXT("SearchableName referencers with no flags"), CombinationEquals<FDependsNode*>(Referencers,PointerA));
				TestTrue(TEXT("SearchableName referencers with no flags"), CombinationEquals<FAssetDependency>(ReferencersStructs, FTestDependency(TestDependency, PointerA).AsAssetDependency()));
				int IterateCount = 0;
				Dependency->IterateOverReferencers([this, &IterateCount, PointerA](FDependsNode* Referencer)
					{
						++IterateCount;
						if (Referencer != PointerA)
						{
							AddError(TEXT("IterateReferencers SearchableName, no flags"));
						}
					});
				TestEqual(TEXT("IterateReferencers SearchableName no flags, Count"), IterateCount, 1);
			}
			int IterateCount = 0;
			A.IterateOverDependencies([this, &IterateCount, &ScratchNodes](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
				{
					TestTrue(TEXT("IterateDependencies SearchableName Category"), Category == EDependencyCategory::SearchableName);
					++IterateCount;
					if (Dependency == &ScratchNodes[3])
					{
						TestTrue(TEXT("IterateDependencies SearchableName"), Properties == EDependencyProperty::None && !bDuplicate);
					}
					else
					{
						AddError(TEXT("IterateDependencies SearchableName"));
					}
				}, EDependencyCategory::SearchableName);
			TestEqual(TEXT("IterateDependencies SearchableName Count"), IterateCount, 1);
		}
		// Manage All
		{
			FTestDependency MatchingDeps[] = { TestDependencies[4], TestDependencies[5], TestDependencies[6] };
			FDependsNode* Matching[] = { &ScratchNodes[4], &ScratchNodes[5], &ScratchNodes[6] };
			Dependencies.Reset();
			A.GetDependencies(Dependencies, EDependencyCategory::Manage);
			TestTrue(TEXT("Manage dependency with no flags"), CombinationEquals<FDependsNode*>(Dependencies, Matching));
			for (int MatchingIndex = 0; MatchingIndex < UE_ARRAY_COUNT(Matching); ++MatchingIndex)
			{
				FDependsNode* Dependency = Matching[MatchingIndex];
				FTestDependency TestDependency = MatchingDeps[MatchingIndex];
				Referencers.Reset();
				Dependency->GetReferencers(Referencers, EDependencyCategory::Manage);
				ReferencersStructs.Reset();
				Dependency->GetReferencers(ReferencersStructs, EDependencyCategory::Manage);
				TestTrue(TEXT("Manage referencers with no flags"), CombinationEquals<FDependsNode*>(Referencers, PointerA));
				TestTrue(TEXT("Manage referencers with no flags"), CombinationEquals<FAssetDependency>(ReferencersStructs, FTestDependency(TestDependency, PointerA).AsAssetDependency()));
				int IterateCount = 0;
				Dependency->IterateOverReferencers([this, &IterateCount, PointerA](FDependsNode* Referencer)
					{
						++IterateCount;
						if (Referencer != PointerA)
						{
							AddError(TEXT("IterateReferencers Manage, no flags"));
						}
					});
				TestEqual(TEXT("IterateReferencers Manage no flags, Count"), IterateCount, 1);
			}
			int IterateCount = 0;
			A.IterateOverDependencies([this, &IterateCount, &ScratchNodes](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
				{
					TestTrue(TEXT("IterateDependencies Manage, no flags, Category"), Category == EDependencyCategory::Manage);
					++IterateCount;
					if (Dependency == &ScratchNodes[4])
					{
						TestTrue(TEXT("IterateDependencies Manage, no flags"), Properties == (EDependencyProperty::Direct | EDependencyProperty::CookRule) && !bDuplicate);
					}
					else if (Dependency == &ScratchNodes[5])
					{
						TestTrue(TEXT("IterateDependencies Manage, no flags"), Properties == EDependencyProperty::Direct && !bDuplicate);
					}
					else if (Dependency == &ScratchNodes[6])
					{
						TestTrue(TEXT("IterateDependencies Manage, no flags"), Properties == EDependencyProperty::None && !bDuplicate);
					}
					else
					{
						AddError(TEXT("IterateDependencies Manage, no flags"));
					}
				}, EDependencyCategory::Manage);
			TestEqual(TEXT("IterateDependencies Manage, no flags, Count"), IterateCount, 3);
		}
		// Manage Direct
		{
			FTestDependency MatchingDeps[] = { TestDependencies[4], TestDependencies[5] };
			FDependsNode* Matching[] = { &ScratchNodes[4], &ScratchNodes[5] };
			FDependsNode* NonMatching[] = { &ScratchNodes[6] };
			Dependencies.Reset();
			A.GetDependencies(Dependencies, EDependencyCategory::Manage, EDependencyQuery::Direct);
			TestTrue(TEXT("Manage dependency with flags, Direct"), CombinationEquals<FDependsNode*>(Dependencies, Matching));
			for (int MatchingIndex = 0; MatchingIndex < UE_ARRAY_COUNT(Matching); ++MatchingIndex)
			{
				FDependsNode* Dependency = Matching[MatchingIndex];
				FTestDependency TestDependency = MatchingDeps[MatchingIndex];
				Referencers.Reset();
				Dependency->GetReferencers(Referencers, EDependencyCategory::Manage, EDependencyQuery::Direct);
				ReferencersStructs.Reset();
				Dependency->GetReferencers(ReferencersStructs, EDependencyCategory::Manage, EDependencyQuery::Direct);
				TestTrue(TEXT("Manage referencers with flags, Direct, matching"), CombinationEquals<FDependsNode*>(Referencers, PointerA));
				TestTrue(TEXT("Manage referencers with flags, Direct, matching"), CombinationEquals<FAssetDependency>(ReferencersStructs, FTestDependency(TestDependency, PointerA).AsAssetDependency()));
				int IterateCount = 0;
				Dependency->IterateOverReferencers([this, &IterateCount, PointerA](FDependsNode* Referencer)
					{
						++IterateCount;
						if (Referencer != PointerA)
						{
							AddError(TEXT("IterateReferencers Manage, Direct, matching"));
						}
					});
				TestEqual(TEXT("IterateReferencers Manage Direct, matching, Count"), IterateCount, 1);
			}
			for (FDependsNode* Dependency : NonMatching)
			{
				Referencers.Reset();
				Dependency->GetReferencers(Referencers, EDependencyCategory::Manage, EDependencyQuery::Direct);
				TestTrue(TEXT("Manage referencers with flags, Direct, nonmatching"), Referencers.Num() == 0);
				int IterateCount = 0;
				Dependency->IterateOverReferencers([this, &IterateCount, PointerA](FDependsNode* Referencer)
					{
						++IterateCount;
						if (Referencer != PointerA)
						{
							AddError(TEXT("IterateReferencers Manage, Direct, nonmatching"));
						}
					});
				TestEqual(TEXT("IterateReferencers Manage Direct, nonmatching, Count"), IterateCount, 1);
			}
			int IterateCount = 0;
			A.IterateOverDependencies([this, &IterateCount, &ScratchNodes](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
				{
					TestTrue(TEXT("IterateDependencies Manage, Direct, Category"), Category == EDependencyCategory::Manage);
					++IterateCount;
					if (Dependency == &ScratchNodes[4])
					{
						TestTrue(TEXT("IterateDependencies Manage, Direct"), Properties == (EDependencyProperty::Direct | EDependencyProperty::CookRule) && !bDuplicate);
					}
					else if (Dependency == &ScratchNodes[5])
					{
						TestTrue(TEXT("IterateDependencies Manage, Direct"), Properties == EDependencyProperty::Direct && !bDuplicate);
					}
					else
					{
						AddError(TEXT("IterateDependencies Manage, Direct"));
					}
				}, EDependencyCategory::Manage, EDependencyQuery::Direct);
			TestEqual(TEXT("IterateDependencies Manage, Direct, Count"), IterateCount, 2);
		}
		// Manage Indirect
		{
			FTestDependency MatchingDeps[] = { TestDependencies[6] };
			FDependsNode* Matching[] = { &ScratchNodes[6] };
			FDependsNode* NonMatching[] = { &ScratchNodes[4], &ScratchNodes[5] };
			Dependencies.Reset();
			A.GetDependencies(Dependencies, EDependencyCategory::Manage, EDependencyQuery::Indirect);
			TestTrue(TEXT("Manage dependency with flags, Indirect"), CombinationEquals<FDependsNode*>(Dependencies, Matching));
			for (int MatchingIndex = 0; MatchingIndex < UE_ARRAY_COUNT(Matching); ++MatchingIndex)
			{
				FDependsNode* Dependency = Matching[MatchingIndex];
				FTestDependency TestDependency = MatchingDeps[MatchingIndex];
				Referencers.Reset();
				Dependency->GetReferencers(Referencers, EDependencyCategory::Manage, EDependencyQuery::Indirect);
				ReferencersStructs.Reset();
				Dependency->GetReferencers(ReferencersStructs, EDependencyCategory::Manage, EDependencyQuery::Indirect);
				TestTrue(TEXT("Manage referencers with flags, Indirect, matching"), CombinationEquals<FDependsNode*>(Referencers, PointerA));
				TestTrue(TEXT("Manage referencers with flags, Indirect, matching"), CombinationEquals<FAssetDependency>(ReferencersStructs, FTestDependency(TestDependency, PointerA).AsAssetDependency()));
				int IterateCount = 0;
				Dependency->IterateOverReferencers([this, &IterateCount, PointerA](FDependsNode* Referencer)
					{
						++IterateCount;
						if (Referencer != PointerA)
						{
							AddError(TEXT("IterateReferencers Manage, Indirect, matching"));
						}
					});
				TestEqual(TEXT("IterateReferencers Manage Indirect, matching, Count"), IterateCount, 1);
			}
			for (FDependsNode* Dependency : NonMatching)
			{
				Referencers.Reset();
				Dependency->GetReferencers(Referencers, EDependencyCategory::Manage, EDependencyQuery::Indirect);
				TestTrue(TEXT("Manage referencers with flags, Indirect, nonmatching"), Referencers.Num() == 0);
				int IterateCount = 0;
				Dependency->IterateOverReferencers([this, &IterateCount, PointerA](FDependsNode* Referencer)
					{
						++IterateCount;
						if (Referencer != PointerA)
						{
							AddError(TEXT("IterateReferencers Manage, Indirect, nonmatching"));
						}
					});
				TestEqual(TEXT("IterateReferencers Manage Indirect, nonmatching, Count"), IterateCount, 1);
			}
			int IterateCount = 0;
			A.IterateOverDependencies([this, &IterateCount, &ScratchNodes](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
				{
					TestTrue(TEXT("IterateDependencies Manage, Indirect, Category"), Category == EDependencyCategory::Manage);
					++IterateCount;
					if (Dependency == &ScratchNodes[6])
					{
						TestTrue(TEXT("IterateDependencies Manage, Indirect"), Properties == EDependencyProperty::None && !bDuplicate);
					}
					else
					{
						AddError(TEXT("IterateDependencies Manage, Indirect"));
					}
				}, EDependencyCategory::Manage, EDependencyQuery::Indirect);
			TestEqual(TEXT("IterateDependencies Manage, Indirect, Count"), IterateCount, 1);
		}

		TestEqual(TEXT("RemoveDependency in dependency and referencer filtering loop"), A.GetConnectionCount(), NumUsedScratchNodes);
		for (int n = 0; n < NumUsedScratchNodes; ++n)
		{
			ScratchNodes[n].RemoveReferencer(&A);
			ScratchNodes[n].SetIsReferencersSorted(true);
			A.RemoveDependency(&ScratchNodes[n]);
			TestEqual(TEXT("RemoveDependency in dependency and referencer filtering loop"), A.GetConnectionCount(), NumUsedScratchNodes - 1 - n);
		}
	}

	// Unsorted Dependency lists with duplicate copies of a node
	{
		FDependsNode A;
		FDependsNode B;
		FDependsNode C;
		for (int Order = 0; Order < 2; ++Order)
		{
			A.ClearDependencies();
			B.ClearReferencers();
			A.SetIsDependencyListSorted(EDependencyCategory::Package, false);
			B.SetIsReferencersSorted(false);
			if (Order == 0)
			{
				A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::None);
				A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::None);
				A.AddDependency(&C, EDependencyCategory::Package, EDependencyProperty::None);
				B.AddReferencer(&A);
				B.AddReferencer(&A);
				B.AddReferencer(&C);
			}
			else
			{
				A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::None);
				A.AddDependency(&C, EDependencyCategory::Package, EDependencyProperty::None);
				A.AddDependency(&C, EDependencyCategory::Package, EDependencyProperty::None);
				B.AddReferencer(&A);
				B.AddReferencer(&C);
				B.AddReferencer(&C);
			}
			A.SetIsDependencyListSorted(EDependencyCategory::Package, true);
			B.SetIsReferencersSorted(true);
			Dependencies.Reset(0);
			A.GetDependencies(Dependencies);
			TestTrue(TEXT("Sorting a list removes duplicates"), CombinationEquals<FDependsNode* const>(Dependencies, TArrayView<FDependsNode* const>({ &B, &C })));
			Referencers.Reset();
			B.GetReferencers(Referencers);
			TestTrue(TEXT("Sorting a list removes duplicates"), CombinationEquals<FDependsNode* const>(Referencers, TArrayView<FDependsNode* const>({ &A, &C })));
		}
	}
	{
		FDependsNode A;
		FDependsNode B;
		FDependsNode C;
		A.SetIsDependencyListSorted(EDependencyCategory::Package, false);
		B.SetIsReferencersSorted(false);
		A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::None);
		A.AddDependency(&C, EDependencyCategory::Package, EDependencyProperty::None);
		A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::None);
		B.AddReferencer(&A);
		B.AddReferencer(&C);
		B.AddReferencer(&A);
		B.AddReferencer(&C);
		C.AddReferencer(&A);
		A.SetIsDependencyListSorted(EDependencyCategory::Package, true);
		B.SetIsReferencersSorted(true);
		int IterationIndex = 0;
		A.IterateOverDependencies([&IterationIndex](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
			{
				++IterationIndex;
			});
		TestTrue(TEXT("Sorting a list removes duplicates"), IterationIndex == 2);
		Referencers.Reset();
		B.GetReferencers(Referencers);
		TestTrue(TEXT("Sorting a list removes duplicates"), Referencers.Num() == 2 && Referencers[0] != Referencers[1]);
	}
	{
		FDependsNode A;
		FDependsNode B;
		FDependsNode C;
		A.SetIsDependencyListSorted(EDependencyCategory::Package, false);
		B.SetIsReferencersSorted(false);
		A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::Hard);
		A.AddDependency(&C, EDependencyCategory::Package, EDependencyProperty::None);
		A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::Game);
		A.AddDependency(&C, EDependencyCategory::Package, EDependencyProperty::None);
		A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::Hard);
		B.AddReferencer(&A);
		B.AddReferencer(&A);
		B.AddReferencer(&A);
		C.AddReferencer(&A);
		C.AddReferencer(&A);
		A.SetIsDependencyListSorted(EDependencyCategory::Package, true);
		B.SetIsReferencersSorted(true);
		int IterationIndex = 0;
		int DuplicateCount = 0;
		A.IterateOverDependencies([&IterationIndex, &DuplicateCount](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
			{
				++IterationIndex;
				DuplicateCount += bDuplicate != 0;
			});
		TestTrue(TEXT("Sorting a list removes duplicates but keeps nonredundant corners"), IterationIndex == 3 && DuplicateCount == 1);
		Referencers.Reset();
		B.GetReferencers(Referencers);
		TestTrue(TEXT("Sorting a list removes duplicates"), Referencers.Num() == 1);
	}

	// GetPackageReferencers
	{
		FDependsNode A;
		FName AName("FDependsNodeA");
		A.SetIdentifier(AName);
		FDependsNode B;
		A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::None);
		B.AddReferencer(&A);
		TArray<TPair<FAssetIdentifier, FDependsNode::FPackageFlagSet>> ReferencersResult;
		B.GetPackageReferencers(ReferencersResult);
		TestTrue(TEXT("GetPackageReferencers expecting 1"), ReferencersResult.Num() == 1 && ReferencersResult[0].Key.PackageName == AName);
		ReferencersResult.Reset();
		A.GetPackageReferencers(ReferencersResult);
		TestTrue(TEXT("GetPackageReferencers expecting 0"), ReferencersResult.Num() == 0);
	}

	// iterating over the same dependency with two nonredundant property combinations
	{
		FDependsNode A;
		FDependsNode B;
		A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::Game);
		A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::Hard);
		B.AddReferencer(&A);
		int IterationIndex = 0;
		bool bHasGame = false;
		bool bHasHard = false;
		A.IterateOverDependencies([this, &B, &IterationIndex, &bHasGame, &bHasHard](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
			{
				TestTrue(TEXT("IterateOverDependencies two property corners only two elements"), IterationIndex < 2);
				TestTrue(TEXT("IterateOverDependencies two property corners expected values"), Dependency == &B && Category == EDependencyCategory::Package && (Properties == EDependencyProperty::Game || Properties == EDependencyProperty::Hard));
				TestTrue(TEXT("IterateOverDependencies two property corners bDuplicate is set correctly"), IterationIndex == 0 || bDuplicate);
				bHasGame = bHasGame || Properties == EDependencyProperty::Game;
				bHasHard = bHasHard || Properties == EDependencyProperty::Hard;
				IterationIndex++;
			});
		TestTrue(TEXT("IterateOverDependencies two property corners hits both corners"), bHasGame && bHasHard);

		IterationIndex = 0;
		bHasGame = false;
		bHasHard = false;
		A.IterateOverDependencies([this, &B, &IterationIndex, &bHasGame, &bHasHard](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
			{
				TestTrue(TEXT("IterateOverDependencies two property corners, node-specific, only two elements"), IterationIndex < 2);
				TestTrue(TEXT("IterateOverDependencies two property corners, node-specific, expected values"), Dependency == &B && Category == EDependencyCategory::Package && (Properties == EDependencyProperty::Game || Properties == EDependencyProperty::Hard));
				TestTrue(TEXT("IterateOverDependencies two property corners, node-specific, bDuplicate is set correctly"), IterationIndex == 0 || bDuplicate);
				bHasGame = bHasGame || Properties == EDependencyProperty::Game;
				bHasHard = bHasHard || Properties == EDependencyProperty::Hard;
				IterationIndex++;
			}, &B);
		TestTrue(TEXT("IterateOverDependencies two property corners, node-specific, hits both corners"), bHasGame&& bHasHard);

		TArray<FAssetIdentifier> Assets;
		A.GetDependencies(Assets);
		TestEqual(TEXT("GetDependencies with a lists of assets removes duplicates"), Assets.Num(), 1);

		TArray<FAssetDependency> StructDependencies;
		A.GetDependencies(StructDependencies);
		TestEqual(TEXT("GetDependencies with a lists of dependency structs keeps duplicates"), StructDependencies.Num(), 2);
	}

	// Redundant property combinations are skipped when iterating
	{
		FDependsNode A;
		FDependsNode B;
		A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::Game);
		A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::Game | EDependencyProperty::Hard);
		B.AddReferencer(&A);
		int IterationIndex = 0;
		A.IterateOverDependencies([this, &B, &IterationIndex](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
			{
				TestTrue(TEXT("IterateOverDependencies redundant corner only one element"), IterationIndex < 1);
				TestTrue(TEXT("IterateOverDependencies redundant corner expected values"), Dependency == &B && Category == EDependencyCategory::Package && (Properties == (EDependencyProperty::Game | EDependencyProperty::Hard)));
				TestTrue(TEXT("IterateOverDependencies redundant corner bDuplicate is set correctly"), IterationIndex == 0 || bDuplicate);
				IterationIndex++;
			});
		TestTrue(TEXT("IterateOverDependencies two property corners hits the one non-redundant corners"), IterationIndex == 1);
	}

	// AddPackageDependencySet
	{
		FDependsNode A;
		FDependsNode B;
		FDependsNode::FPackageFlagSet PackageFlagSet;
		PackageFlagSet.Add(FDependsNode::PackagePropertiesToByte(EDependencyProperty::Game));
		PackageFlagSet.Add(FDependsNode::PackagePropertiesToByte(EDependencyProperty::Hard));
		A.AddPackageDependencySet(&B, PackageFlagSet);
		B.AddReferencer(&A);
		int IterationIndex = 0;
		bool bHasGame = false;
		bool bHasHard = false;
		A.IterateOverDependencies([this, &B, &IterationIndex, &bHasGame, &bHasHard](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
			{
				TestTrue(TEXT("AddPackageDependencySet only two elements"), IterationIndex < 2);
				TestTrue(TEXT("AddPackageDependencySets expected values"), Dependency == &B && Category == EDependencyCategory::Package && (Properties == EDependencyProperty::Game || Properties == EDependencyProperty::Hard));
				bHasGame = bHasGame || Properties == EDependencyProperty::Game;
				bHasHard = bHasHard || Properties == EDependencyProperty::Hard;
				IterationIndex++;
			});
		TestTrue(TEXT("IterateOverDependencies two property corners hits both corners"), bHasGame && bHasHard);
	}

	// ClearDependencies
	{
		FDependsNode A;
		FDependsNode B;
		A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::Game);
		A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::Hard);
		B.AddReferencer(&A);
		A.ClearDependencies(EDependencyCategory::All);
		int IterationIndex = 0;
		A.IterateOverDependencies([&IterationIndex](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
			{
				++IterationIndex;
			});
		TestTrue(TEXT("ClearDependencies(All) results in 0 iteration"), IterationIndex == 0);

		A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::None);
		A.AddDependency(&B, EDependencyCategory::Manage, EDependencyProperty::None);
		A.AddDependency(&B, EDependencyCategory::SearchableName, EDependencyProperty::None);

		IterationIndex = 0;
		A.IterateOverDependencies([&IterationIndex](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
			{
				++IterationIndex;
			});
		TestTrue(TEXT("ClearDependencies partial, setup"), IterationIndex == 3);

		A.ClearDependencies(EDependencyCategory::Package);
		IterationIndex = 0;
		A.IterateOverDependencies([this, &IterationIndex](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
			{
				TestTrue(TEXT("ClearDependencies partial, removed Package"), Category != EDependencyCategory::Package);
				++IterationIndex;
			});
		TestTrue(TEXT("ClearDependencies partial, removed one"), IterationIndex == 2);

		A.ClearDependencies(EDependencyCategory::Manage);
		IterationIndex = 0;
		A.IterateOverDependencies([this, &IterationIndex](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
			{
				TestTrue(TEXT("ClearDependencies partial, removed Package+Manage"), Category != EDependencyCategory::Package && Category != EDependencyCategory::Manage);
				++IterationIndex;
			});
		TestTrue(TEXT("ClearDependencies partial, removed two"), IterationIndex == 1);

		A.ClearDependencies(EDependencyCategory::SearchableName);
		IterationIndex = 0;
		A.IterateOverDependencies([this, &IterationIndex](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
			{
				++IterationIndex;
			});
		TestTrue(TEXT("ClearDependencies partial, removed three"), IterationIndex == 0);
	}

	// RemoveManageReferencesToNode
	for (ESortTestType SortTestType : { ESortTestType::AlwaysSorted, ESortTestType::AlwaysUnsorted, ESortTestType::UnsortedThenSorted })
	{
		FDependsNode A;
		FDependsNode B;
		FDependsNode C;
		FDependsNode D;
		if (SortTestType == ESortTestType::AlwaysUnsorted || SortTestType == ESortTestType::UnsortedThenSorted)
		{
			A.SetIsDependencyListSorted(UE::AssetRegistry::EDependencyCategory::All, false);
			B.SetIsReferencersSorted(false);
			C.SetIsDependencyListSorted(UE::AssetRegistry::EDependencyCategory::All, false);
			D.SetIsReferencersSorted(false);
		}
		A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::None);
		A.AddDependency(&B, EDependencyCategory::Manage, EDependencyProperty::None);
		A.AddDependency(&B, EDependencyCategory::SearchableName, EDependencyProperty::None);
		B.AddReferencer(&A);
		C.AddDependency(&D, EDependencyCategory::Manage, EDependencyProperty::None);
		D.AddReferencer(&C);
		if (SortTestType == ESortTestType::UnsortedThenSorted)
		{
			A.SetIsDependencyListSorted(UE::AssetRegistry::EDependencyCategory::All, true);
			B.SetIsReferencersSorted(true);
			C.SetIsDependencyListSorted(UE::AssetRegistry::EDependencyCategory::All, true);
			D.SetIsReferencersSorted(true);
		}

		B.RemoveManageReferencesToNode();
		int IterationIndex = 0;
		A.IterateOverDependencies([this, &IterationIndex](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
			{
				TestTrue(TEXT("RemoveManageReferencesToNode removed the manage dependency"), Category != EDependencyCategory::Manage);
				++IterationIndex;
			});
		TestTrue(TEXT("RemoveManageReferencesToNode removed just the manage dependency"), IterationIndex == 2);
		Referencers.Reset();
		B.GetReferencers(Referencers);
		TestTrue(TEXT("RemoveManageReferencesToNode did not remove the referencer when dependencies still existed"), Referencers.Num() == 1);

		D.RemoveManageReferencesToNode();
		IterationIndex = 0;
		C.IterateOverDependencies([this, &IterationIndex](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
			{
				++IterationIndex;
			});
		TestTrue(TEXT("RemoveManageReferencesToNode only manage reference - removed the manage dependency"), IterationIndex == 0);
		Referencers.Reset();
		D.GetReferencers(Referencers);
		TestTrue(TEXT("RemoveManageReferencesToNode only manage reference - removed the referencer"), Referencers.Num() == 0);
	}

	// RemoveLinks
	{
		constexpr int NumLinkNodes = 5;
		EDependencyProperty PackageProperties[NumLinkNodes];
		EDependencyProperty ManageProperties[NumLinkNodes];
		int UnsortedOrder[] = { 4,1,3,0,2 };
		for (int n = 0; n < NumLinkNodes; ++n)
		{
			PackageProperties[n] = (n & 0x1) ? EDependencyProperty::Hard : EDependencyProperty::None;
			PackageProperties[n] |= (n & 0x2) ? EDependencyProperty::Game : EDependencyProperty::None;
			PackageProperties[n] |= (n & 0x4) ? EDependencyProperty::Build : EDependencyProperty::None;
			ManageProperties[n] = (n & 0x1) ? EDependencyProperty::Direct : EDependencyProperty::None;
			ManageProperties[n] |= (n & 0x2) ? EDependencyProperty::CookRule : EDependencyProperty::None;
		}

		for (ESortTestType SortTestType : { ESortTestType::AlwaysSorted, ESortTestType::UnsortedThenSorted, ESortTestType::AlwaysUnsorted})
		{
			for (int FirstRemove = 0; FirstRemove < NumLinkNodes; ++FirstRemove)
			{
				for (int SecondRemove = -1; SecondRemove < NumLinkNodes; ++SecondRemove)
				{
					if (SecondRemove == FirstRemove)
					{
						continue;
					}
					int NumRemoves = SecondRemove == -1 ? 1 : 2;

					for (int NumElements : { 1, 2, NumLinkNodes})
					{
						if (NumElements == 2 && SecondRemove != -1)
						{
							continue;
						}

						FDependsNode LinkHaver;
						FDependsNode LinkNodes[NumLinkNodes];
						auto AddLink = [&LinkHaver, PackageProperties, ManageProperties, &LinkNodes](int NodeIndex)
						{
							FDependsNode* LinkNode = &LinkNodes[NodeIndex];
							LinkHaver.AddReferencer(LinkNode);
							LinkHaver.AddDependency(LinkNode, EDependencyCategory::Package, PackageProperties[NodeIndex]);
							LinkHaver.AddDependency(LinkNode, EDependencyCategory::SearchableName, EDependencyProperty::None);
							LinkHaver.AddDependency(LinkNode, EDependencyCategory::Manage, ManageProperties[NodeIndex]);
						};
						auto GetNodeIndex = [&LinkNodes](const FDependsNode* Existing) { return UE_PTRDIFF_TO_INT32(Existing - &LinkNodes[0]); };

						if (SortTestType != ESortTestType::AlwaysSorted)
						{
							for (EDependencyCategory Category : { EDependencyCategory::Manage, EDependencyCategory::SearchableName, EDependencyCategory::Package})
							{
								LinkHaver.SetIsDependencyListSorted(Category, false);
							}
							LinkHaver.SetIsReferencersSorted(false);
						}
						if (NumElements <= 2)
						{
							AddLink(FirstRemove);
							if (NumElements > 1)
							{
								AddLink(SecondRemove);
							}
						}
						else
						{
							if (SortTestType == ESortTestType::AlwaysUnsorted)
							{
								for (int n = 0; n < NumLinkNodes; ++n)
								{
									AddLink(UnsortedOrder[n]);
								}
							}
							else
							{
								for (int n = 0; n < NumLinkNodes; ++n)
								{
									AddLink(n);
								}
							}
						}

						TUniqueFunction<bool(const FDependsNode*)> ShouldRemove = [FirstRemove, SecondRemove, &LinkNodes, &GetNodeIndex](const FDependsNode* ExistingNode)
						{
							int NodeIndex = GetNodeIndex(ExistingNode);
							return NodeIndex == FirstRemove || NodeIndex == SecondRemove;
						};

						LinkHaver.RemoveLinks(ShouldRemove);
						if (NumElements <= 2)
						{
							TestTrue(TEXT("RemoveLinks with NumElements removes all"), LinkHaver.GetConnectionCount() == 0);
						}
						else
						{
							int ExpectedCount = NumElements - NumRemoves;
							int Count = 0;
							LinkHaver.IterateOverReferencers([this, &ShouldRemove, &Count](const FDependsNode* ExistingLink)
								{
									TestTrue(TEXT("RemoveLinks removes all requested referencers"), !ShouldRemove(ExistingLink));
									++Count;
								});
							TestEqual(TEXT("RemoveLinks keeps the proper number of referencers"), ExpectedCount, Count);

							for (EDependencyCategory Category : {EDependencyCategory::Package, EDependencyCategory::SearchableName, EDependencyCategory::Manage})
							{
								Count = 0;
								LinkHaver.IterateOverDependencies([this, &ShouldRemove, &GetNodeIndex, Category, &Count, &PackageProperties, &ManageProperties](const FDependsNode* ExistingLink, EDependencyCategory ExistingCategory, EDependencyProperty ExistingProperty, bool bIsDuplicate)
									{
										TestTrue(TEXT("RemoveLinks removes all requested referencers"), !ShouldRemove(ExistingLink));
										EDependencyProperty* Properties = nullptr;
										if (Category == EDependencyCategory::Package) Properties = PackageProperties;
										else if (Category == EDependencyCategory::Manage) Properties = ManageProperties;
										if (Properties)
										{
											TestTrue(TEXT("RemoveLinks keeps the proper properties"), ExistingProperty == Properties[GetNodeIndex(ExistingLink)]);
										}
										++Count;
									}, Category);
								TestEqual(TEXT("RemoveLinks keeps the proper number of dependencies"), ExpectedCount, Count);
							}
						}
					}
				}
			}
		}
	}

	// GetConnectionCount, GetAllocatedSize
	{
		FDependsNode A;
		FDependsNode B;
		FDependsNode C;
		A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::None);
		A.AddDependency(&B, EDependencyCategory::Manage, EDependencyProperty::None);
		A.AddDependency(&B, EDependencyCategory::SearchableName, EDependencyProperty::None);
		A.AddDependency(&C, EDependencyCategory::Package, EDependencyProperty::None);
		B.AddDependency(&C, EDependencyCategory::Package, EDependencyProperty::None);
		B.AddReferencer(&A);
		C.AddReferencer(&A);
		C.AddReferencer(&B);
		TestEqual(TEXT("GetConnectionCount - Referencer - A"), A.GetConnectionCount(), 4);
		TestEqual(TEXT("GetConnectionCount - Referencer - B"), B.GetConnectionCount(), 2);
		TestEqual(TEXT("GetConnectionCount - Referencer - C"), C.GetConnectionCount(), 2);
		TestTrue(TEXT("GetAllocatedSize"), A.GetAllocatedSize() > 0 && C.GetAllocatedSize() > 0);
	}

	// Serialize
	enum class EKeepNode2
	{
		KeepAlways,
		KeepNever,
		KeepSaveButNotLoad,
	};
	for (EKeepNode2 KeepNode2 : { EKeepNode2::KeepAlways, EKeepNode2::KeepNever, EKeepNode2::KeepSaveButNotLoad })
	{
		for (bool bKeepManageDependencies : { true, false})
		{
			for (bool bKeepNameDependencies : { true, false})
			{
				FDependsNode::FSaveScratch SaveScratch;
				FDependsNode::FLoadScratch LoadScratch;
				FDependsNode A;
				FDependsNode B;
				A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::None);
				A.AddDependency(&B, EDependencyCategory::Manage, EDependencyProperty::None);
				A.AddDependency(&B, EDependencyCategory::SearchableName, EDependencyProperty::None);
				B.AddReferencer(&A);

				TUniqueFunction<int32(FDependsNode*, bool bAsReferencer)> GetSerializeIndexFromNode = [&A, &B, KeepNode2](FDependsNode* DependsNode, bool bAsReferencer) -> int32
				{
					if (DependsNode == &A) return 0;
					if (KeepNode2 == EKeepNode2::KeepAlways || KeepNode2 == EKeepNode2::KeepSaveButNotLoad)
					{
						if (DependsNode == &B) return 1;
					}
					return -1;
				};
				FDependsNode LoadedA;
				FDependsNode LoadedB;
				TUniqueFunction<FDependsNode* (int32)> GetNodeFromSerializeIndex = [&LoadedA, &LoadedB, KeepNode2](int32 Index) -> FDependsNode*
				{
					if (Index == 0) return &LoadedA;
					if (KeepNode2 == EKeepNode2::KeepAlways)
					{
						if (Index == 1) return &LoadedB;
					}
					return nullptr;
				};

				FAssetRegistrySerializationOptions Options(UE::AssetRegistry::ESerializationTarget::ForDevelopment);
				Options.bSerializeSearchableNameDependencies = bKeepNameDependencies;
				Options.bSerializeManageDependencies = bKeepManageDependencies;

				TArray<uint8> Bytes;
				{
					FMemoryWriter Writer(Bytes);
					A.SerializeSave(Writer, GetSerializeIndexFromNode, SaveScratch, Options);
					if (KeepNode2 == EKeepNode2::KeepAlways || KeepNode2 == EKeepNode2::KeepSaveButNotLoad)
					{
						B.SerializeSave(Writer, GetSerializeIndexFromNode, SaveScratch, Options);
					}
				}

				{
					FMemoryReader Reader(Bytes);
					LoadedA.SerializeLoad<true>(Reader, GetNodeFromSerializeIndex, LoadScratch, FAssetRegistryVersion::LatestVersion);
					if (KeepNode2 == EKeepNode2::KeepAlways)
					{
						LoadedB.SerializeLoad<true>(Reader, GetNodeFromSerializeIndex, LoadScratch, FAssetRegistryVersion::LatestVersion);
					}
				}

				int AIterationIndex = 0;
				bool bHasPackage = false;
				bool bHasManage = false;
				bool bHasName = false;
				LoadedA.IterateOverDependencies([this, &AIterationIndex, &bHasPackage, &bHasManage, &bHasName, &LoadedB](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
					{
						bHasPackage = bHasPackage || Category == EDependencyCategory::Package;
						bHasName = bHasName || Category == EDependencyCategory::SearchableName;
						bHasManage = bHasManage || Category == EDependencyCategory::Manage;
						TestTrue(TEXT("Serialize - ADependencies - Category one at a time"), Category == EDependencyCategory::Package || Category == EDependencyCategory::SearchableName || Category == EDependencyCategory::Manage);
						TestTrue(TEXT("Serialize - ADependencies - Expected values"), Dependency == &LoadedB && Properties == EDependencyProperty::None);
						++AIterationIndex;
					});

				TArray<FDependsNode*> AReferencers, BReferencers;
				LoadedA.GetReferencers(AReferencers);
				LoadedB.GetReferencers(BReferencers);
				int BIterationIndex = 0;
				LoadedB.IterateOverDependencies([&BIterationIndex](FDependsNode* Dependency, EDependencyCategory Category, EDependencyProperty Properties, bool bDuplicate)
					{
						++BIterationIndex;
					});

				TestTrue(TEXT("Serialize - AReferencers"), AReferencers.Num() == 0);
				TestTrue(TEXT("Serialize - BDependencies"), BIterationIndex == 0);

				if (KeepNode2 == EKeepNode2::KeepAlways)
				{
					TestTrue(TEXT("Serialize - ADependencies - All expected types"), bHasPackage && bHasName == bKeepNameDependencies && bHasManage == bKeepManageDependencies);
					TestTrue(TEXT("Serialize - BReferencers"), BReferencers.Num() == 1 && BReferencers[0] == &LoadedA);
				}
				else
				{
					TestTrue(TEXT("Serialize - ADependencies - filtered out node2"), AIterationIndex == 0);
				}
			}
		}
	}

	// RefreshReferencers
	for (ESortTestType SortTestType : { ESortTestType::AlwaysSorted, ESortTestType::AlwaysUnsorted, ESortTestType::UnsortedThenSorted })
	{
		FDependsNode Nodes[8];
		FDependsNode* NodeWithReferencers = &Nodes[7];
		if (SortTestType != ESortTestType::AlwaysSorted)
		{
			NodeWithReferencers->SetIsReferencersSorted(false);
		}
		Nodes[0].AddDependency(&Nodes[7], EDependencyCategory::Package, EDependencyProperty::None);
		Nodes[6].AddDependency(&Nodes[7], EDependencyCategory::Package, EDependencyProperty::None);
		for (int Count = 0; Count < 2; ++Count)
		{
			for (int n = 0; n < UE_ARRAY_COUNT(Nodes); ++n)
			{
				if (&Nodes[n] != NodeWithReferencers)
				{
					NodeWithReferencers->AddReferencer(&Nodes[n]);
				}
			}
		}
		NodeWithReferencers->RefreshReferencers();
		if (SortTestType != ESortTestType::AlwaysUnsorted)
		{
			NodeWithReferencers->SetIsReferencersSorted(true);
		}
		Referencers.Reset();
		NodeWithReferencers->GetReferencers(Referencers);
		if (SortTestType != ESortTestType::AlwaysUnsorted)
		{
			TestTrue(TEXT("RefreshReferencers"), CombinationEquals<FDependsNode* const>(Referencers, TArrayView<FDependsNode* const>({ &Nodes[0], &Nodes[6] })));
		}
		else
		{
			TestTrue(TEXT("RefreshReferencers"), CombinationEquals<FDependsNode* const>(Referencers, TArrayView<FDependsNode* const>({ &Nodes[0], &Nodes[6], &Nodes[0], &Nodes[6] })));
		}
	}

	// RemapLinks
	{
		FDependsNode A;
		FDependsNode B;
		FDependsNode C;
		FDependsNode D;
		FDependsNode RemapA;
		FDependsNode RemapC;
		A.AddDependency(&B, EDependencyCategory::Package, EDependencyProperty::None);
		A.AddDependency(&B, EDependencyCategory::Manage, EDependencyProperty::None);
		A.AddDependency(&B, EDependencyCategory::SearchableName, EDependencyProperty::None);
		A.AddDependency(&C, EDependencyCategory::Package, EDependencyProperty::None);
		A.AddDependency(&D, EDependencyCategory::Package, EDependencyProperty::None);
		B.AddDependency(&C, EDependencyCategory::Package, EDependencyProperty::None);
		B.AddReferencer(&A);
		C.AddReferencer(&A);
		C.AddReferencer(&B);
		D.AddReferencer(&A);

		TUniqueFunction<FDependsNode* (FDependsNode*)> Remapper = 
			[&A, &B, &C, &RemapA, &RemapC](FDependsNode* Target)
			{
				if (Target == &A)
				{
					return &RemapA;
				}
				if (Target == &B)
				{
					return (FDependsNode*)nullptr;
				}
				if (Target == &C)
				{
					return &RemapC;
				}
				return Target;
			};
		A.RemapLinks(Remapper);
		B.RemapLinks(Remapper);
		C.RemapLinks(Remapper);
		D.RemapLinks(Remapper);

		Dependencies.Reset();
		Referencers.Reset();
		A.GetDependencies(Dependencies);
		A.GetReferencers(Referencers);
		TestTrue(TEXT("RemapLinks.1"), CombinationEquals<FDependsNode*>(Dependencies, TArrayView<FDependsNode* const>({ &RemapC, &D })));
		TestTrue(TEXT("RemapLinks.2"), CombinationEquals<FDependsNode*>(Referencers, TArrayView<FDependsNode* const>()));

		Dependencies.Reset();
		Referencers.Reset();
		B.GetDependencies(Dependencies);
		B.GetReferencers(Referencers);
		TestTrue(TEXT("RemapLinks.3"), CombinationEquals<FDependsNode*>(Dependencies, TArrayView<FDependsNode* const>({ &RemapC })));
		TestTrue(TEXT("RemapLinks.4"), CombinationEquals<FDependsNode*>(Referencers, TArrayView<FDependsNode* const>({ &RemapA })));

		Dependencies.Reset();
		Referencers.Reset();
		C.GetDependencies(Dependencies);
		C.GetReferencers(Referencers);
		TestTrue(TEXT("RemapLinks.5"), CombinationEquals<FDependsNode*>(Dependencies, TArrayView<FDependsNode* const>()));
		TestTrue(TEXT("RemapLinks.6"), CombinationEquals<FDependsNode*>(Referencers, TArrayView<FDependsNode* const>({ &RemapA })));

		Dependencies.Reset();
		Referencers.Reset();
		D.GetDependencies(Dependencies);
		D.GetReferencers(Referencers);
		TestTrue(TEXT("RemapLinks.7"), CombinationEquals<FDependsNode*>(Dependencies, TArrayView<FDependsNode* const>()));
		TestTrue(TEXT("RemapLinks.8"), CombinationEquals<FDependsNode*>(Referencers, TArrayView<FDependsNode* const>({ &RemapA })));

		// Remap links does not handle adding the reverse links, so RemapA and RemapC are unmodified.
		TestTrue(TEXT("RemapLinks.9"), RemapA.GetConnectionCount() == 0);
		TestTrue(TEXT("RemapLinks.10"), RemapC.GetConnectionCount() == 0);
	}

	return true;
}

#endif
