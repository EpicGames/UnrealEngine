// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "PackageDependencyData.h"

namespace UE::AssetRegistry::Private
{

enum class EOptionalBool : uint8
{
	Unset,
	False,
	True,
	Max = EOptionalBool::True,
};

}
FArchive& operator<<(FArchive& Ar, UE::AssetRegistry::Private::EOptionalBool& Value);
UE::AssetRegistry::Private::EOptionalBool BoolToOptionalBool(bool bValue);

class FDiskCachedAssetData
{
public:
	FDateTime ModificationTime; // Modification time of the asset last time it was discovered
	FDateTime LastGatheredTime;  // The last time this asset was discovered during a scan 
	FName Extension;
	UE::AssetRegistry::Private::EOptionalBool HasMarkOfTheWeb = UE::AssetRegistry::Private::EOptionalBool::Unset;
	TArray<FAssetData> AssetDataList;
	FPackageDependencyData DependencyData;

	FDiskCachedAssetData()
	{}

	FDiskCachedAssetData(const FDateTime& InModificationTime, const FDateTime& InLastSeen, FName InExtension)
		: ModificationTime(InModificationTime)
		, LastGatheredTime(InLastSeen)
		, Extension(InExtension)
	{}

	/**
	 * Serialize as part of the registry cache. This is not meant to be serialized as part of a package so  it does not handle versions normally
	 * To version this data change FAssetRegistryVersion or AssetDataGathererConstants::CacheSerializationMagic
	 */
	template<class Archive>
	void SerializeForCache(Archive&& Ar)
	{
		Ar << ModificationTime;
		Ar << LastGatheredTime;
		Ar << Extension;
		Ar << HasMarkOfTheWeb;
	
		int32 AssetDataCount = AssetDataList.Num();
		Ar << AssetDataCount;

		if (Ar.IsLoading())
		{
			AssetDataList.SetNum(AssetDataCount);
		}

		for (int32 i = 0; i < AssetDataCount; i++)
		{
			AssetDataList[i].SerializeForCache(Ar);
		}

		DependencyData.SerializeForCache(Ar);
	}

	/** Returns the amount of memory allocated by this container, not including sizeof(*this). */
	SIZE_T GetAllocatedSize() const
	{
		SIZE_T Result = 0;
		Result += AssetDataList.GetAllocatedSize();
		Result += DependencyData.GetAllocatedSize();
		return Result;
	}
};

inline FArchive& operator<<(FArchive& Ar, UE::AssetRegistry::Private::EOptionalBool& Value)
{
	using namespace UE::AssetRegistry::Private;
	uint8 ValueInt = static_cast<uint8>(Value);
	Ar << ValueInt;
	if (Ar.IsLoading())
	{
		Value = ValueInt <= (uint8)EOptionalBool::Max ? (EOptionalBool)ValueInt : EOptionalBool::Unset;
	}
	return Ar;
}

inline UE::AssetRegistry::Private::EOptionalBool BoolToOptionalBool(bool bValue)
{
	using namespace UE::AssetRegistry::Private;
	return bValue ? EOptionalBool::True : EOptionalBool::False;
}
