// Copyright Epic Games, Inc. All Rights Reserved.

#include "NameTableArchive.h"

#include "AssetRegistry/AssetRegistryState.h"
#include "AssetRegistryPrivate.h"
#include "UObject/NameTypes.h"


class FNameTableErrorArchive : public FArchive
{
public:
	FNameTableErrorArchive()
	{
		SetError();
	}
};
static FNameTableErrorArchive GNameTableErrorArchive;

FNameTableArchiveReader::FNameTableArchiveReader(FArchive& WrappedArchive)
	: FArchive()
	, ProxyAr(&WrappedArchive)
{
	this->SetIsLoading(true);

	if (!SerializeNameMap())
	{
		ProxyAr = &GNameTableErrorArchive;
		SetError();
	}
}

bool FNameTableArchiveReader::SerializeNameMap()
{
	int64 NameOffset = 0;
	*this << NameOffset;

	if (NameOffset > TotalSize())
	{
		// The file was corrupted. Return false to fail to load the cache and thus regenerate it.
		return false;
	}

	if( NameOffset > 0 )
	{
		int64 OriginalOffset = Tell();
		ProxyAr->Seek( NameOffset );

		int32 NameCount = 0;
		*this << NameCount;
		if (IsError() || NameCount < 0)
		{
			return false;
		}
		
		const int32 MinFNameEntrySize = sizeof(int32);
		const int64 MaxReservation = ((TotalSize() - Tell()) / MinFNameEntrySize);
		NameMap.Reserve((int32)FMath::Min<int64>(NameCount, MaxReservation));
		for ( int32 NameMapIdx = 0; NameMapIdx < NameCount; ++NameMapIdx )
		{
			// Read the name entry from the file.
			FNameEntrySerialized NameEntry(ENAME_LinkerConstructor);
			*this << NameEntry;

			if (IsError())
			{
				return false;
			}

			NameMap.Add(FName(NameEntry).GetDisplayIndex());
		}

		ProxyAr->Seek(OriginalOffset);
	}

	return true;
}

void FNameTableArchiveReader::Serialize(void* V, int64 Length)
{
	ProxyAr->Serialize(V, Length);

	if (ProxyAr->IsError())
	{
		ProxyAr = &GNameTableErrorArchive;
		SetError();
	}
}

bool FNameTableArchiveReader::Precache(int64 PrecacheOffset, int64 PrecacheSize)
{
	if (!IsError())
	{
		return ProxyAr->Precache(PrecacheOffset, PrecacheSize);
	}

	return false;
}

void FNameTableArchiveReader::Seek(int64 InPos)
{
	if (!IsError())
	{
		ProxyAr->Seek(InPos);
	}
}

int64 FNameTableArchiveReader::Tell()
{
	return ProxyAr->Tell();
}

int64 FNameTableArchiveReader::TotalSize()
{
	return ProxyAr->TotalSize();
}

const FCustomVersionContainer& FNameTableArchiveReader::GetCustomVersions() const
{
	return ProxyAr->GetCustomVersions();
}

void FNameTableArchiveReader::SetCustomVersions(const FCustomVersionContainer& NewVersions)
{
	ProxyAr->SetCustomVersions(NewVersions);
}

void FNameTableArchiveReader::ResetCustomVersions()
{
	ProxyAr->ResetCustomVersions();
}

FArchive& FNameTableArchiveReader::operator<<(FName& OutName)
{
	int32 NameIndex;
	FArchive& Ar = *this;
	Ar << NameIndex;

	if (NameMap.IsValidIndex(NameIndex))
	{
		FNameEntryId MappedName = NameMap[NameIndex];

		int32 Number;
		Ar << Number;

		OutName = FName::CreateFromDisplayId(MappedName, MappedName ? Number : 0);
	}
	else
	{
		UE_LOGF(LogAssetRegistry, Error, "Bad name index reading cache %i/%i", NameIndex, NameMap.Num());

		ProxyAr = &GNameTableErrorArchive;
		SetError();

		OutName = FName();
	}

	return *this;
}

void FNameTableArchiveReader::SerializeTagsAndBundles(FAssetData& Out)
{
	FAssetDataTagMap Map;
	*this << Map;
	Out.SetTagsAndAssetBundles(MoveTemp(Map));
}
	
void FNameTableArchiveReader::SerializeTagsAndBundlesOldVersion(FAssetData& Out, int32 Version)
{
	FAssetDataTagMap Map;
	*this << Map;
	Out.SetTagsAndAssetBundles(MoveTemp(Map));
}