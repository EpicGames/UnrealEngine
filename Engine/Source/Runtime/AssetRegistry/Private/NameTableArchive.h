// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Serialization/Archive.h"

struct FAssetData;
struct FNameEntryId;

/** 
 * Legacy name table reader replaced by FAssetRegistryReader.
 *
 * Has an FName table and names are serialized as table indices.
*/
class FNameTableArchiveReader : public FArchive
{
public:	
	/** Create a reader that wraps around an existing archive. Existing archive must support seek */
	FNameTableArchiveReader(FArchive& WrappedArchive);
	
	// Farchive implementation to redirect requests to the wrapped archive
	virtual void Serialize(void* V, int64 Length) override;
	virtual bool Precache(int64 PrecacheOffset, int64 PrecacheSize) override;
	virtual void Seek(int64 InPos) override;
	virtual int64 Tell() override;
	virtual int64 TotalSize() override;
	virtual const FCustomVersionContainer& GetCustomVersions() const override;
	virtual void SetCustomVersions(const FCustomVersionContainer& NewVersions) override;
	virtual void ResetCustomVersions() override;
	virtual FArchive& operator<<(FName& Name);

	void SerializeTagsAndBundles(FAssetData& Out);
	void SerializeTagsAndBundlesOldVersion(FAssetData& Out, int32 Version);

private:
	/** Serializers for different package maps */
	bool SerializeNameMap();

	FArchive* ProxyAr;
	TArray<FNameEntryId> NameMap;
};
