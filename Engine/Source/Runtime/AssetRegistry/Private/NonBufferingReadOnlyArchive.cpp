// Copyright Epic Games, Inc. All Rights Reserved.

#include "NonBufferingReadOnlyArchive.h"

#include "AssetRegistryPrivate.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Logging/MessageLog.h"
#include "Logging/StructuredLog.h"
#include "Misc/PathViews.h"

FNonBufferingReadOnlyArchive::FNonBufferingReadOnlyArchive(int64 InBufferSize, uint32 InFlags)
	: Filename()
	, Size(0)
	, Pos(0)
	, BufferBase(0)
	, Handle()
	, Flags(InFlags)
{
	// Todo: The buffer must be aligned to at least the size of the physical sector of the disk used.
	// We don't have a generic API for this so use the page size as that will cover most common cases
	BufferReadAlignment = FPlatformMemory::GetConstants().BinnedPageSize;
	check(FMath::IsPowerOfTwo(BufferReadAlignment));
	BufferSize = FMath::Max(InBufferSize, BufferReadAlignment);
	check(FMath::IsPowerOfTwo(BufferSize));
	Buffer = (uint8*) FPlatformMemory::BinnedAllocFromOS(BufferSize);
	check(Buffer);
}

FNonBufferingReadOnlyArchive::~FNonBufferingReadOnlyArchive()
{
	Close();
	FPlatformMemory::BinnedFreeToOS(Buffer, BufferSize);
}

void FNonBufferingReadOnlyArchive::Seek(int64 InPos)
{
	checkf(InPos >= 0, TEXT("Attempted to seek to a negative location (%lld/%lld), file: %s. The file is most likely corrupt."), InPos, Size, *Filename);
	checkf(InPos <= Size, TEXT("Attempted to seek past the end of file (%lld/%lld), file: %s. The file is most likely corrupt."), InPos, Size, *Filename);

	// Note, we don't do any platform specific seeking when asked. We instead defer until a read attempt is made where we will
	// be forced to read in a new buffer's amount of data.
	Pos = InPos;
}

void FNonBufferingReadOnlyArchive::ReadLowLevel(uint8* Dest, int64 CountToRead, int64& OutBytesRead)
{
	int64 StartPos = Handle->Tell();
	if (Handle->Read(Dest, CountToRead))
	{
		OutBytesRead = Handle->Tell() - StartPos;
	}
	else
	{
		OutBytesRead = 0;
	}
}

bool FNonBufferingReadOnlyArchive::SeekLowLevel(int64 InPos)
{
	return Handle->Seek(InPos);
}

void FNonBufferingReadOnlyArchive::OpenFileLowLevel(const TCHAR* InFilename)
{
	Close();

	FFileOpenResult FileResult = FPlatformFileManager::Get().GetPlatformFile().OpenReadNoBuffering(InFilename, IPlatformFile::EOpenReadFlags::NoBuffering);
	if (!FileResult.HasError())
	{
		Handle = FileResult.StealValue();
	}
	else
	{
		UE_CLOGF(!IsSilent(), LogAssetRegistry, Warning, "OpenFile failed: File:%ls, Error:%ls",
			InFilename, *FileResult.StealError().GetMessage());
	}
}

void FNonBufferingReadOnlyArchive::CloseLowLevel()
{
	Handle.Reset();
}

bool FNonBufferingReadOnlyArchive::Close()
{
	CloseLowLevel();
	return !IsError();
}

bool FNonBufferingReadOnlyArchive::Precache(int64 PrecacheOffset, int64 PrecacheSize)
{
	// Archives not based on async I/O should always return true, so we return true whether or not the precache was successful.
	// Returning false would imply that the caller should continue calling until it returns true.
	if (PrecacheSize < 0)
	{
		return true;
	}
	InternalPrecache(PrecacheOffset, PrecacheSize);
	return true;
}

bool FNonBufferingReadOnlyArchive::OpenFile(const TCHAR* InFilename)
{
	OpenFileLowLevel(InFilename);
	if (!Handle.IsValid())
	{
		return false;
	}
	Size = Handle->Size();
	Filename = InFilename;
	Pos = 0;

	// Until we read in data we need a value that will force us to precache
	BufferBase = -(BufferSize + 1);

	this->SetIsLoading(true);
	this->SetIsPersistent(true);
	return true;
}

bool FNonBufferingReadOnlyArchive::InternalPrecache(int64 PrecacheOffset, int64 PrecacheSize)
{
	check(Buffer);

	// Only precache at current position and avoid work if precaching same offset twice.
	if (Pos != PrecacheOffset)
	{
		// We are refusing to precache, but return true if at least one byte after the requested PrecacheOffset is in our existing buffer.
		return BufferBase <= PrecacheOffset && PrecacheOffset < BufferBase + BufferSize;
	}

	bool bPosWithinBuffer = BufferBase <= Pos && Pos < BufferBase + BufferSize;
	int64 ReadCount = BufferSize;
	if (bPosWithinBuffer)
	{
		// At least one byte after the requested PrecacheOffset is in our existing buffer so do nothing and return true
		return true;
	}

	// We must always read at a sector aligned address so move the position backwards as needed
	BufferBase = AlignDown(Pos, BufferReadAlignment);
	if (BufferBase + ReadCount < Pos)
	{
		UE_CLOGF(!IsSilent(), LogAssetRegistry, Warning, 
			"Buffer is not large enough to satisfy the requested read. BufferBase=%lld BufferSize=%lld BufferReadAlignment=%lld ReadCount=%lld Pos=%lld", BufferBase, BufferSize, BufferReadAlignment, ReadCount, Pos);
		return false;
	}

	int64 Count = 0;
	SeekLowLevel(BufferBase);
	ReadLowLevel(Buffer, ReadCount, Count);

	if (Count != ReadCount && Count != (Size - BufferBase))
	{
		uint32 LastError = FPlatformMisc::GetLastError();
		TCHAR ErrorBuffer[1024];
		UE_CLOGF(!IsSilent(), LogAssetRegistry, Warning, "ReadFile failed: Count=%lld Pos=%lld Size=%lld BufferBase=%lld ReadCount=%lld BufferReadAlignment=%lld LastError=%u: %ls",
			Count, Pos, Size, BufferBase, ReadCount, BufferReadAlignment, LastError, FPlatformMisc::GetSystemErrorMessage(ErrorBuffer, 1024, LastError));

		return BufferBase + Count >= Pos;
	}

	return true;
}

void FNonBufferingReadOnlyArchive::Serialize(void* V, int64 Length)
{
	if (Pos + Length > Size)
	{
		SetError();
		UE_CLOG(!IsSilent(), LogAssetRegistry, Error, TEXT("Requested read of %" INT64_FMT " bytes when %" INT64_FMT " bytes remain (file=%s, size=%" INT64_FMT ")"), Length, Size - Pos, *Filename, Size);
		return;
	}

	bool bReadUncached = false;
	const bool bIsOutsideBufferWindow = (Pos < BufferBase) || (Pos >= (BufferBase + BufferSize));
	if (bIsOutsideBufferWindow)
	{
		bReadUncached = true;
	}

	while (Length > 0)
	{
		int64 Copy = FMath::Min(Length, BufferBase + BufferSize - Pos);
		if (Copy <= 0 || bReadUncached)
		{
			if (!InternalPrecache(Pos, MAX_int32))
			{
				SetError();
				UE_CLOGF(!IsSilent(), LogAssetRegistry, Warning, "ReadFile failed during precaching for file %ls", *Filename);
				return;
			}
			bReadUncached = false;
			Copy = FMath::Min(Length, BufferBase + BufferSize - Pos);
			if (Copy <= 0)
			{
				SetError();
				UE_CLOGF(!IsSilent(), LogAssetRegistry, Error, "ReadFile beyond EOF %lld+%lld/%lld for file %ls",
					Pos, Length, Size, *Filename);
			}
			if (IsError())
			{
				return;
			}
		}
		FMemory::Memcpy(V, Buffer + Pos - BufferBase, Copy);
		Pos += Copy;
		Length -= Copy;
		V = (uint8*)V + Copy;
	}
}
