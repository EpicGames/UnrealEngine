// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "HAL/FileManager.h"
#include "Serialization/ArchiveUObject.h"

/*
* This archive uses non-buffered IO to avoid overhead caused by reading into the OS cache manager 
* rather than not reading into an archive owned buffer (it does that). It is intended to be 
* used when reads will occur in parallel, contention would be caused at an OS level to allocate cache pages, 
* and typically the whole file is not read sequentially but instead only a smaller portion of the file 
* is needed (otherwise OS caching likely performs better since it can predict more accurately without waste).
* This archive is intended to be re-usable by opening new files on top so the internal buffer can be re-used.
*/
class FNonBufferingReadOnlyArchive : public FArchive
{
public:
	FNonBufferingReadOnlyArchive(int64 InBufferSize = 4096, uint32 InFlags = 0);
	~FNonBufferingReadOnlyArchive();

	// FArchive API
	virtual void Seek(int64 InPos) override final;
	virtual int64 Tell() override final
	{
		return Pos;
	}
	virtual int64 TotalSize() override final
	{
		return Size;
	}
	virtual bool Close() override final;
	virtual void Serialize(void* V, int64 Length) override final;
	virtual FString GetArchiveName() const override
	{
		return Filename;
	}
	virtual bool Precache(int64 PrecacheOffset, int64 PrecacheSize) override;
	//End FArchive API

	/** Open a new file to read with this archive. Any existing open file is closed and the archive is reset. */
	bool OpenFile(const TCHAR* InFilename);

protected:
	/** Populates the internal buffer to read from during Serialize. */
	bool InternalPrecache(int64 PrecacheOffset, int64 PrecacheSize);

	/**
	 * Platform specific seek
	 * @param InPos - Offset from beginning of file to seek to
	 * @return false on failure
	**/
	bool SeekLowLevel(int64 InPos);

	/** Open the file handle */
	void OpenFileLowLevel(const TCHAR* InFilename);

	/** Close the file handle **/
	void CloseLowLevel();

	/** Platform specific read */
	void ReadLowLevel(uint8* Dest, int64 CountToRead, int64& OutBytesRead);

	/** Returns true if the archive should suppress logging in case of error */
	bool IsSilent() const
	{
		return !!(Flags & EFileRead::FILEREAD_Silent);
	}

	FString Filename;
	int64 Size;
	int64 Pos;
	int64 BufferBase;
	TUniquePtr<IFileHandle> Handle;

	uint8* Buffer;
	int64 BufferReadAlignment;
	int64 BufferSize;
	uint32 Flags;
};