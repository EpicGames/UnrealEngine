// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageDependencyData.h"

#include "Algo/Unique.h"
#include "Misc/StringBuilder.h"
#include "String/Find.h"
#include "UObject/NameTypes.h"

FName FPackageDependencyData::GetImportPackageName(TConstArrayView<FObjectImport> ImportMap, int32 ImportIndex)
{
	for (int32 NumCycles = 0; NumCycles < ImportMap.Num(); ++NumCycles)
	{
		if (!ImportMap.IsValidIndex(ImportIndex))
		{
			return NAME_None;
		}
		const FObjectImport& Resource = ImportMap[ImportIndex];
		// If the import has a package name set, then that's the import package name,
		if (Resource.HasPackageName())
		{
			return Resource.GetPackageName();
		}
		// If our outer is null, then we have a package
		else if (Resource.OuterIndex.IsNull())
		{
			return Resource.ObjectName;
		}
		if (!Resource.OuterIndex.IsImport())
		{
				return NAME_None;
		}
		ImportIndex = Resource.OuterIndex.ToImport();
	}
	return NAME_None;
}

struct FSortPackageDependency
{
	FORCEINLINE bool operator()(const FPackageDependencyData::FPackageDependency& A, const FPackageDependencyData::FPackageDependency& B) const
	{
		if (int32 Comparison = A.PackageName.CompareIndexes(B.PackageName))
		{
			return Comparison < 0;
		}
		return static_cast<uint8>(A.Property) < static_cast<uint8>(B.Property);
	}
};

void FPackageDependencyData::LoadDependenciesFromPackageHeader(FName SourcePackageName, TConstArrayView<FObjectImport> ImportMap,
	TArray<FName>& SoftPackageReferenceList, TMap<FPackageIndex, TArray<FName>>& SearchableNames,
	TBitArray<>& ImportUsedInGame, TBitArray<>& SoftPackageUsedInGame,
	TArray<TPair<FName, UE::AssetRegistry::EExtraDependencyFlags>>& ExtraPackageDependencies)
{
	using namespace UE::AssetRegistry;

	// ExternalActor imports of their outer can be treated as editor-only imports because ExternalActors are EditorOnly
	// packages. References to ExternalActors can occur from collector assets that find all ExternalActor packages in
	// a plugin. When propagating should-be-cooked rules or chunk assignment rules in the AssetManager, we don't want
	// to pull in/set the chunk of the world containing the plugin's actor. SavePackage records the import of the
	// externalactor's world package so we would ordinarily mark it as UsedInGame in the loop below. But we override
	// that for AssetRegistry dependencies so that the AssetManager propagation does not pull in the map package when
	// an actor is referenced from a collector. We implement that here by naming convention.
	// See also FAssetRegistryGenerator::ComputePackageDifferences.
	FStringView ExternalActorFolder(FPackagePath::GetExternalActorsFolderName());
	TStringBuilder<FName::StringBufferSize> SourceStr(InPlace, SourcePackageName);
	FStringView ExternalActorWorldRelPath;
	if (int32 FolderIndex = UE::String::FindFirst(SourceStr, ExternalActorFolder, ESearchCase::IgnoreCase);
		FolderIndex != INDEX_NONE)
	{
		ExternalActorWorldRelPath = SourceStr.ToView().RightChop(FolderIndex + ExternalActorFolder.Len());
	}
	auto IsWorldOfExternalActor = [ExternalActorWorldRelPath](FName DependencyPackageName)
		{
			TStringBuilder<256> TargetStr(InPlace, DependencyPackageName);
			FStringView TargetMountPoint = FPathViews::GetMountPointNameFromPath(TargetStr);
			FStringView TargetRelativePath = FStringView(TargetStr).RightChop(TargetMountPoint.Len() + 1);
			return (INDEX_NONE !=
				UE::String::FindFirst(ExternalActorWorldRelPath, TargetRelativePath, ESearchCase::IgnoreCase));
		};

	PackageDependencies.Reset(ImportMap.Num() + SoftPackageReferenceList.Num());
	check(ImportMap.Num() == ImportUsedInGame.Num());
	for (int32 ImportIdx = 0; ImportIdx < ImportMap.Num(); ++ImportIdx)
	{
		FName DependencyPackageName = GetImportPackageName(ImportMap, ImportIdx);
		EDependencyProperty DependencyProperty = EDependencyProperty::Hard;
		bool bUsedInGame = ImportUsedInGame[ImportIdx];

		if (!ExternalActorWorldRelPath.IsEmpty())
		{
			bUsedInGame &= !IsWorldOfExternalActor(DependencyPackageName);
		}

		DependencyProperty |= bUsedInGame ? EDependencyProperty::Game : EDependencyProperty::None;
		PackageDependencies.Add({ DependencyPackageName, DependencyProperty });
	}

	// Sort and make unique to reduce data saved and processed
	PackageDependencies.Sort(FSortPackageDependency());

	int UniqueNum = Algo::Unique(PackageDependencies);
	PackageDependencies.SetNum(UniqueNum, EAllowShrinking::No);

	check(SoftPackageReferenceList.Num() == SoftPackageUsedInGame.Num());
	for (int32 SoftPackageIdx = 0; SoftPackageIdx < SoftPackageReferenceList.Num(); ++SoftPackageIdx)
	{
		FName DependencyPackageName = SoftPackageReferenceList[SoftPackageIdx];
		FAssetIdentifier AssetId(DependencyPackageName);
		EDependencyProperty DependencyProperty = EDependencyProperty::None;

		bool bUsedInGame = SoftPackageUsedInGame[SoftPackageIdx];
		if (!ExternalActorWorldRelPath.IsEmpty())
		{
			bUsedInGame &= !IsWorldOfExternalActor(DependencyPackageName);
		}

		DependencyProperty |= (bUsedInGame ? EDependencyProperty::Game : EDependencyProperty::None);

		// Don't need to remove duplicates here because SavePackage only writes unique elements into SoftPackageReferenceList
		PackageDependencies.Add({ DependencyPackageName, DependencyProperty });
	}

	for (const TPair<FName, EExtraDependencyFlags>& Pair : ExtraPackageDependencies)
	{
		FName DependencyPackageName = Pair.Key;
		EDependencyProperty DependencyProperty = EDependencyProperty::None;
		DependencyProperty |= EnumHasAnyFlags(Pair.Value, EExtraDependencyFlags::Build)
			? EDependencyProperty::Build : EDependencyProperty::None;

		if (!ExternalActorWorldRelPath.IsEmpty())
		{
			if (IsWorldOfExternalActor(DependencyPackageName))
			{
				continue;
			}
		}

		// Don't need to remove duplicates here because SavePackage only writes unique elements into PackageBuildDependencies
		PackageDependencies.Add({ DependencyPackageName, DependencyProperty });
	}

	SearchableNameDependencies.Reset(SearchableNames.Num());
	for (const TPair<FPackageIndex, TArray<FName>>& SearchableNameList : SearchableNames)
	{
		FName ObjectName;
		FName DependencyPackageName;

		// Find object and package name from linker
		FPackageIndex LinkerIndex = SearchableNameList.Key;
		if (LinkerIndex.IsExport())
		{
			// Package name has to be this package, take a guess at object name
			DependencyPackageName = SourcePackageName;
			ObjectName = FName(*FPackageName::GetLongPackageAssetName(DependencyPackageName.ToString()));
		}
		else if (LinkerIndex.IsImport())
		{
			int32 ImportIndex = LinkerIndex.ToImport();
			if (!ImportMap.IsValidIndex(ImportIndex))
			{
				continue;
			}
			const FObjectImport& Resource = ImportMap[ImportIndex];
			FPackageIndex OuterLinkerIndex = Resource.OuterIndex;
			if (!OuterLinkerIndex.IsNull())
			{
				ObjectName = Resource.ObjectName;
			}
			DependencyPackageName = GetImportPackageName(ImportMap, ImportIndex);
			if (DependencyPackageName.IsNone())
			{
				continue;
			}
		}
		else
		{
			continue;
		}

		FSearchableNamesDependency& Dependency = SearchableNameDependencies.Emplace_GetRef();
		Dependency.PackageName = DependencyPackageName;
		Dependency.ObjectName = ObjectName;
		Dependency.ValueNames = SearchableNameList.Value;
	}
}
