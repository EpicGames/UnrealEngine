// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/ArrayView.h"
#include "Containers/BitArray.h"
#include "DependsNode.h"
#include "UObject/Linker.h"
class FPackageDependencyData
{
public:
	struct FPackageDependency
	{
		FName PackageName;
		UE::AssetRegistry::EDependencyProperty Property;
		friend FArchive& operator<<(FArchive& Ar, FPackageDependency& Dependency)
		{
			Ar << Dependency.PackageName;
			uint8 PropertyAsInteger = static_cast<uint8>(Dependency.Property);
			Ar << PropertyAsInteger;
			Dependency.Property = static_cast<UE::AssetRegistry::EDependencyProperty>(PropertyAsInteger);
			return Ar;
		}

		bool operator==(const FPackageDependency& Other) const
		{
			return PackageName == Other.PackageName && Property == Other.Property;
		}
	};
	struct FSearchableNamesDependency
	{
		FName PackageName;
		FName ObjectName;
		TArray<FName> ValueNames;
		friend FArchive& operator<<(FArchive& Ar, FSearchableNamesDependency& Dependency)
		{
			Ar << Dependency.PackageName << Dependency.ObjectName << Dependency.ValueNames;
			return Ar;
		}
	};

	// Dependency Data
	TArray<FPackageDependency> PackageDependencies;
	TArray<FSearchableNamesDependency> SearchableNameDependencies;

	FDependsNodeReservations DependsNodeReservations;

	/** Asset Package data, gathered at the same time as dependency data */
	FAssetPackageData PackageData;

	/** The name of the package that dependency data is gathered from */
	FName PackageName;

	// Transient Flags indicating which types of data have been gathered
	bool bHasPackageData = false;
	bool bHasDependencyData = false;

	/**
	 * Serialize as part of the registry cache. This is not meant to be serialized as part of a package so it does not handle versions normally
	 * To version this data change FAssetRegistryVersion or AssetDataGathererConstants::CacheSerializationMagic
	 */
	void SerializeForCache(FArchive& Ar)
	{
		Ar << PackageName;
		PackageData.SerializeForCache(Ar);
		Ar << PackageDependencies;
		Ar << SearchableNameDependencies;
		Ar << DependsNodeReservations;
	}

	void LoadDependenciesFromPackageHeader(FName PackageName, TConstArrayView<FObjectImport> ImportMap,
		TArray<FName>& SoftPackageReferenceList, TMap<FPackageIndex, TArray<FName>>& SearchableNames,
		TBitArray<>& ImportUsedInGame, TBitArray<>& SoftPackageUsedInGame,
		TArray<TPair<FName, UE::AssetRegistry::EExtraDependencyFlags>>& ExtraPackageDependencies);

	/** Returns the amount of memory allocated by this container, not including sizeof(*this). */
	SIZE_T GetAllocatedSize() const
	{
		SIZE_T Result = PackageDependencies.GetAllocatedSize();
		Result += SearchableNameDependencies.GetAllocatedSize();
		Result += PackageData.GetAllocatedSize();
		return Result;
	}

private:
	FName GetImportPackageName(TConstArrayView<FObjectImport> ImportMap, int32 ImportIndex);


};
