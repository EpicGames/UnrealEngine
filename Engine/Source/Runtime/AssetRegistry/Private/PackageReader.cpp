// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/PackageReader.h"

#include "Algo/Reverse.h"
#include "AssetRegistry.h"
#include "AssetRegistryPrivate.h"
#include "AssetRegistry/AssetData.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/GatherableTextData.h"
#include "Logging/MessageLog.h"
#include "Logging/StructuredLog.h"
#include "Misc/Guid.h"
#include "Misc/PackageName.h"
#include "Misc/PackagePath.h"
#include "Misc/ScopeExit.h"
#include "Serialization/ImportTypeHierarchy.h"
#include "UObject/Class.h"
#include "UObject/Linker.h"
#include "UObject/LinkerLoad.h"
#include "UObject/MetaData.h"
#include "UObject/PackageRelocation.h"
#include "UObject/PackageTrailer.h"

#define LOCTEXT_NAMESPACE "AssetRegistry"

const TCHAR* LexToString(FPackageReader::EOpenPackageResult Result)
{
	switch(Result)
	{
		case FPackageReader::EOpenPackageResult::Success: return TEXT("Success");
		case FPackageReader::EOpenPackageResult::NoLoader: return TEXT("NoLoader");
		case FPackageReader::EOpenPackageResult::MalformedTag: return TEXT("MalformedTag");
		case FPackageReader::EOpenPackageResult::VersionTooOld: return TEXT("VersionTooOld");
		case FPackageReader::EOpenPackageResult::VersionTooNew: return TEXT("VersionTooNew");
		case FPackageReader::EOpenPackageResult::CustomVersionMissing: return TEXT("CustomVersionMissing");
		case FPackageReader::EOpenPackageResult::CustomVersionInvalid: return TEXT("CustomVersionInvalid");
		case FPackageReader::EOpenPackageResult::Unversioned: return TEXT("Unversioned");
	}
	checkf(false, TEXT("LexToString unimplemented for EOpenPackageResult %d"), (int32)Result);
	return TEXT("UNKNOWN");
}

namespace UE::Private
{
static void ApplyRelocationToTagsAndValues(FAssetDataTagMap& TagsAndValues, UE::Package::Relocation::Private::FPackageRelocationContext RelocationArgs)
{
	using namespace UE::Package::Relocation::Private;
	for (TPair<FName, FString>& Iter : TagsAndValues)
	{
		FString& Value = Iter.Value;
		if (Value.IsEmpty())
		{
			continue;
		}

		if (FPackageName::IsValidObjectPath(Value))
		{
			FNameBuilder RelocatedPackageName;
			if (TryRelocateReference(RelocationArgs, Value, RelocatedPackageName)
				&& RelocatedPackageName.Len() != 0)
			{
				Value = *RelocatedPackageName;
			}
		}
		else if (
			FStringView ClassName, ObjectPath;
			FPackageName::ParseExportTextPath(Value, &ClassName, &ObjectPath))
		{
			FNameBuilder RelocatedClassName;
			const bool bHasNewClassName = TryRelocateReference(RelocationArgs, ClassName, RelocatedClassName)
				&& RelocatedClassName.Len() != 0;
			FNameBuilder RelocatedObjectName;
			const bool bHasNewObjectName = TryRelocateReference(RelocationArgs, ObjectPath, RelocatedObjectName)
				&& RelocatedObjectName.Len() != 0;

			// want to be careful with all these string views around:
			FString NewValue = FString::Format(TEXT("{0}'{1}'"),
				{ bHasNewClassName ? *RelocatedClassName : ClassName,
				bHasNewObjectName ? *RelocatedObjectName : ObjectPath });
			Value = MoveTemp(NewValue);
			// validate value:
			ensure(FPackageName::ParseExportTextPath(Value, &ClassName, &ObjectPath));
		}
	}
}
}

FPackageReader::FPackageReader()
{
	this->SetIsLoading(true);
	this->SetIsPersistent(true);
}

FPackageReader::~FPackageReader()
{
	if (Loader && bLoaderOwner)
	{
		delete Loader;
	}
}

bool FPackageReader::OpenPackageFile(FStringView InPackageFilename, EOpenPackageResult* OutErrorCode)
{
	return OpenPackageFile(FStringView(), InPackageFilename, OutErrorCode);
}

bool FPackageReader::OpenPackageFile(FStringView InLongPackageName, FStringView InPackageFilename, EOpenPackageResult* OutErrorCode)
{
	check(!Loader);
	LongPackageName = InLongPackageName;
	PackageFilename = InPackageFilename;
	Loader = IFileManager::Get().CreateFileReader(*PackageFilename);
	bLoaderOwner = true;
	EOpenPackageResult Tmp = EOpenPackageResult::Success;
	return OpenPackageFile(OutErrorCode ? *OutErrorCode : Tmp); 
}

bool FPackageReader::OpenPackageFile(FArchive* InLoader, FStringView InLongPackageName, EOpenPackageResult* OutErrorCode)
{
	check(!Loader);
	Loader = InLoader;
	bLoaderOwner = false;
	LongPackageName = InLongPackageName;
	PackageFilename = Loader->GetArchiveName();
	EOpenPackageResult Tmp = EOpenPackageResult::Success;
	return OpenPackageFile(OutErrorCode ? *OutErrorCode : Tmp); 
}

bool FPackageReader::OpenPackageFile(TUniquePtr<FArchive> InLoader, EOpenPackageResult* OutErrorCode)
{
	check(!Loader);
	Loader = InLoader.Release();
	bLoaderOwner = true;
	LongPackageName.Empty();
	PackageFilename = Loader->GetArchiveName();
	EOpenPackageResult Tmp = EOpenPackageResult::Success;
	return OpenPackageFile(OutErrorCode ? *OutErrorCode : Tmp); 
}

bool FPackageReader::OpenPackageFile(EOpenPackageResult& OutErrorCode)
{
	OutErrorCode = EOpenPackageResult::Success;
	if( Loader == nullptr )
	{
		// Couldn't open the file
		OutErrorCode = EOpenPackageResult::NoLoader;
		return false;
	}

	// Read package file summary from the file
	*this << PackageFileSummary;

	// Validate the summary.

	// Make sure this is indeed a package
	if (PackageFileSummary.Tag != PACKAGE_FILE_TAG || IsError())
	{
		// Unrecognized or malformed package file
		UE_LOGF(LogAssetRegistry, Error,
			"Package is unloadable: %ls. Reason: Invalid value for PACKAGE_FILE_TAG at start of file.",
			*PackageFilename);
		OutErrorCode = EOpenPackageResult::MalformedTag;
		return false;
	}

	// IsEnforcePackageCompatibleVersionCheck(): If LinkerLoad is not validating, PackageReader should not either.
	// Optimize the IsEnforcePackageCompatibleVersionCheck==true but no errors case; only test
	// IsEnforcePackageCompatibleVersionCheck after finding a version mismatch.
	if (!PackageFileSummary.IsFileVersionValid() &&
		IsEnforcePackageCompatibleVersionCheck())
	{
		// Log a warning rather than an error. Linkerload gracefully handles this case.
		UE_LOGF(LogAssetRegistry, Warning,
			"Package is unloadable: %ls. Reason: Package was saved unversioned and the current process does not support loading unversioned packages.",
			*PackageFilename);
		OutErrorCode = EOpenPackageResult::Unversioned;
		return false;
	}

	// Don't read packages that are too old
	if (PackageFileSummary.IsFileVersionTooOld() &&
		IsEnforcePackageCompatibleVersionCheck())
	{
		// Log a warning rather than an error. Linkerload gracefully handles this case.
		UE_LOGF(LogAssetRegistry, Warning,
			"Package is unloadable: %ls. Reason: Version is too old. Min Version: %i, Package Version: %i.",
			*PackageFilename, (int32)VER_UE4_OLDEST_LOADABLE_PACKAGE,
			PackageFileSummary.GetFileVersionUE().FileVersionUE4);

		OutErrorCode = EOpenPackageResult::VersionTooOld;
		return false;
	}

	// Don't read packages that were saved with a package version newer than the current one.
	if (PackageFileSummary.IsFileVersionTooNew() &&
		IsEnforcePackageCompatibleVersionCheck())
	{
		// Log a warning rather than an error. Linkerload gracefully handles this case.
		UE_LOGF(LogAssetRegistry, Warning,
			"Package is unloadable: %ls. Reason: Version is too new. Engine Version: %i, Package Version: %i.",
			*PackageFilename, GPackageFileUEVersion.ToValue(), PackageFileSummary.GetFileVersionUE().ToValue());

		OutErrorCode = EOpenPackageResult::VersionTooNew;
		return false;
	}

	if (PackageFileSummary.GetFileVersionLicenseeUE() > GPackageFileLicenseeUEVersion &&
		IsEnforcePackageCompatibleVersionCheck())
	{
		// Log a warning rather than an error. Linkerload gracefully handles this case.
		UE_LOGF(LogAssetRegistry, Warning,
			"Package is unloadable: %ls. Reason: LicenseeVersion is too new. Licensee Version: %i, Package Licensee Version: %i.",
			*PackageFilename, GPackageFileLicenseeUEVersion, PackageFileSummary.GetFileVersionLicenseeUE());

		OutErrorCode = EOpenPackageResult::VersionTooNew;
		return false;
	}

	// Check serialized custom versions against latest custom versions.
	TArray<FCustomVersionDifference> Diffs = FCurrentCustomVersions::Compare(
		PackageFileSummary.GetCustomVersionContainer().GetAllVersions(), *PackageFilename);
	for (FCustomVersionDifference Diff : Diffs)
	{
		if (Diff.Type == ECustomVersionDifference::Missing)
		{
			if (IsEnforcePackageCompatibleVersionCheck())
			{
				OutErrorCode = EOpenPackageResult::CustomVersionMissing;
			}
		}
		else if (Diff.Type == ECustomVersionDifference::Invalid)
		{
			if (IsEnforcePackageCompatibleVersionCheck())
			{
				OutErrorCode = EOpenPackageResult::CustomVersionInvalid;
			}
		}
		else if (Diff.Type == ECustomVersionDifference::Newer)
		{
			if (IsEnforcePackageCompatibleVersionCheck())
			{
				int32 PackageVersion = -1;
				int32 HeadCodeVersion = -1;
				if (const FCustomVersion* PackagePtr
					= PackageFileSummary.GetCustomVersionContainer().GetVersion(Diff.Version->Key))
				{
					PackageVersion = PackagePtr->Version;
				}
				if (TOptional<FCustomVersion> CurrentPtr = FCurrentCustomVersions::Get(Diff.Version->Key))
				{
					HeadCodeVersion = CurrentPtr->Version;
				}
				UE_LOGF(LogAssetRegistry, Error,
					"Package is unloadable: %ls. Reason: Custom version is too new; the package has newer custom version of %ls: Package: %d, HeadCode: %d.",
					*PackageFilename, *Diff.Version->GetFriendlyName().ToString(), PackageVersion, HeadCodeVersion);
				OutErrorCode = EOpenPackageResult::VersionTooNew;
			}
		}
		// else ECustomVersionDifference::Older, which is not a problem
	}

	//make sure the filereader gets the correct version number (it defaults to latest version)
	SetUEVer(PackageFileSummary.GetFileVersionUE());
	SetLicenseeUEVer(PackageFileSummary.GetFileVersionLicenseeUE());
	SetEngineVer(PackageFileSummary.SavedByEngineVersion);

	const FCustomVersionContainer& PackageFileSummaryVersions = PackageFileSummary.GetCustomVersionContainer();
	SetCustomVersions(PackageFileSummaryVersions);
	
	SetUseUnversionedPropertySerialization((PackageFileSummary.GetPackageFlags() & EPackageFlags::PKG_UnversionedProperties) != 0);

	PackageFileSize = Loader->TotalSize();
	
	if (LongPackageName.IsEmpty())
	{
		LongPackageName = PackageFileSummary.PackageName;
	}

	return OutErrorCode == EOpenPackageResult::Success;
}

bool FPackageReader::TryGetLongPackageName(FString& OutLongPackageName) const
{
	if (!LongPackageName.IsEmpty())
	{
		OutLongPackageName = LongPackageName;
		return true;
	}
	else
	{
		return FPackageName::TryConvertFilenameToLongPackageName(PackageFilename, OutLongPackageName);
	}
}

FString FPackageReader::GetLongPackageName() const
{
	FString Result;
	verify(TryGetLongPackageName(Result));
	return Result;
}

bool FPackageReader::StartSerializeSection(int64 Offset)
{
	check(Loader);
	if (Offset <= 0 || Offset > PackageFileSize)
	{
		return false;
	}
	ClearError();
	Loader->ClearError();
	Seek(Offset);
	return !IsError();
}

#define UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING(MessageKey, PackageFileName) \
	do \
	{\
		FFormatNamedArguments CorruptPackageWarningArguments; \
		CorruptPackageWarningArguments.Add(TEXT("FileName"), FText::FromString(PackageFileName)); \
		UE_LOGF(LogAssetRegistry, Warning, "%ls", \
			*FText::Format(NSLOCTEXT("AssetRegistry", MessageKey, \
			"Package is unloadable: {FileName}. Reason: " MessageKey "."), \
			CorruptPackageWarningArguments).ToString()); \
	} while (false)


const FPackageFileSummary& FPackageReader::GetPackageFileSummary() const
{
	return PackageFileSummary;
}

bool FPackageReader::GetNames(TArray<FName>& OutNames)
{
	if (!SerializeNameMap())
	{
		return false;
	}	
	OutNames = NameMap;
	return true;
}

bool FPackageReader::GetImports(TArray<FObjectImport>& OutImportMap)
{
	if (!SerializeNameMap() || !SerializeImportMap())
	{
		return false;
	}
	OutImportMap = ImportMap;
	return true;
}

bool FPackageReader::GetExports(TArray<FObjectExport>& OutExportMap)
{
	if (!SerializeNameMap() || !SerializeExportMap())
	{
		return false;
	}
	OutExportMap = ExportMap;
	return true;
}

bool FPackageReader::GetDependsMap(TArray<TArray<FPackageIndex>>& OutDependsMap)
{
	if (!SerializeDependsMap())
	{
		return false;
	}
	OutDependsMap = DependsMap;
	return true;
}

bool FPackageReader::GetSoftPackageReferenceList(TArray<FName>& OutSoftPackageReferenceList)
{
	if (!SerializeNameMap() || !SerializeSoftPackageReferenceList())
	{
		return false;
	}
	OutSoftPackageReferenceList = SoftPackageReferenceList;
	return true;
}

bool FPackageReader::GetSoftObjectPaths(TArray<FSoftObjectPath>& OutSoftObjectPaths)
{
	if (!SerializeNameMap() || !SerializeSoftObjectPathMap())
	{
		return false;
	}
	OutSoftObjectPaths = SoftObjectPathMap;
	return true;
}

bool FPackageReader::GetGatherableTextData(TArray<FGatherableTextData>& OutText)
{
	if (!SerializeGatherableTextDataMap())
	{
		return false;
	}
	OutText = GatherableTextDataMap;
	return true;
}

bool FPackageReader::GetThumbnails(TArray<FObjectFullNameAndThumbnail>& OutThumbnails)
{
	if (!SerializeThumbnailMap())
	{
		return false;
	}	
	
	OutThumbnails = ThumbnailMap;
	return true;
}

bool FPackageReader::ReadEditorOnlyFlags(TBitArray<>& OutImportUsedInGame, TBitArray<>& OutSoftPackageUsedInGame)
{
	if (!SerializeNameMap() || !SerializeImportMap() || !SerializeSoftPackageReferenceList())
	{
		return false;
	}
	if (!SerializeEditorOnlyFlags(OutImportUsedInGame, OutSoftPackageUsedInGame))
	{
		return false;
	}
	return true;
}

bool FPackageReader::ReadImportedClasses(TArray<FName>& OutClassNames)
{
	if (!SerializeNameMap() || !SerializeImportMap())
	{
		return false;
	}
	if (!SerializeImportedClasses(ImportMap, OutClassNames))
	{
		return false;
	}
	return true;
}

bool FPackageReader::ReadImportTypeHierarchies(TMap<FSoftObjectPath, UE::Serialization::Private::FImportTypeHierarchy>& OutTypeHierarchies)
{
	// Skip export map as there should be no paths including exports in import type hierarchies. 
	// If there are and the export map has not been serialized, warnings will be logged.
	if (!SerializeNameMap() || !SerializeImportMap())
	{
		return false;
	}

	if (!StartSerializeSection(PackageFileSummary.ImportTypeHierarchiesOffset))
	{
		UE_LOGFMT_LOC(LogAssetRegistry, Warning,"ImportTypeHierarchies_BadOffset",
			"Package is unloadable: {FileName}. Reason: ImportTypeHierarchiesOffset {Offset} was invalid. ",
			("FileName", PackageFilename), ("Offset", PackageFileSummary.ImportTypeHierarchiesOffset));
		return false;
	}

	int32 Count = 0;
	*this << Count;
	if (IsError())
	{
		UE_LOGFMT_LOC(LogAssetRegistry, Warning,"ImportTypeHierarchies_CountSerializeFail",
			"Package is unloadable: {FileName}. Reason: Failed to serialize import type hierarchy count. ",
			("FileName", PackageFilename));
		return false;
	}
	if (Count != PackageFileSummary.ImportTypeHierarchiesCount)
	{
		UE_LOGFMT_LOC(LogAssetRegistry, Warning,"ImportTypeHierarchies_CountMismatch",
			"Package is unloadable: {FileName}. Reason: ImportTypeHierarchiesCount {HeaderCount} did not match embedded count {EmbeddedCount}. ",
			("FileName", PackageFilename), ("HeaderCount", PackageFileSummary.ImportTypeHierarchiesCount), ("EmbeddedCount", Count));
		return false;
	}

	bool bSuccess = true;
	for (int32 i=0; i < PackageFileSummary.ImportTypeHierarchiesCount; ++i)
	{
		TPair<FPackageIndex, UE::Serialization::Private::FImportTypeHierarchy> Pair;
		*this << Pair;
		if (IsError())
		{
			UE_LOGFMT_LOC(LogAssetRegistry, Warning,"ImportTypeHierarchies_SerializeFail",
				"Package is unloadable: {FileName}. Reason: Failed to serialize import type hierarchy at index {Index}. ",
				("FileName", PackageFilename), ("Index", i));
			return false;
		}

		TOptional<FSoftObjectPath> Path = LinkerIndexToSoftObjectPath(Pair.Key);
		if (Path.IsSet())
		{
			OutTypeHierarchies.Add(Path.GetValue(), MoveTemp(Pair.Value));
		}
		else
		{
			UE_LOGFMT_LOC(LogAssetRegistry, Warning,"ImportTypeHierarchies_BadPath",
				"Package is unloadable: {FileName}. Reason: Package index {Index} could not be resolved to a path. ",
				("FileName", PackageFilename), ("Index", Pair.Key.ForDebugging()));
			bSuccess = false;
		}
	}

	return bSuccess; 
}

bool FPackageReader::ReadCellExports(TArray<FCellExport>& OutExports)
{
	if (PackageFileSummary.CellExportCount == 0)
	{
		return true;
	}

	if (!StartSerializeSection(PackageFileSummary.CellExportOffset))
	{
		UE_LOGFMT_LOC(LogAssetRegistry, Warning,"CellExports_BadOffset",
			"Package is unloadable: {FileName}. Reason: CellExportOffset {Offset} was invalid. ",
			("FileName", PackageFilename), ("Offset", PackageFileSummary.CellExportOffset));
		return false;
	}

	OutExports.Reserve(PackageFileSummary.CellExportCount);
	for (int32 i=0; i < PackageFileSummary.CellExportCount; ++i)
	{
		FCellExport& Export = OutExports.Emplace_GetRef();
		*this << Export;

		if (IsError())
		{
			UE_LOGFMT_LOC(LogAssetRegistry, Warning,"CellExport_SerializeFailed",
				"Package is unloadable: {FileName}. Reason: Failed to serialize cell export at index {Index}. ",
				("FileName", PackageFilename), ("Index", i));
			return false;
		}
	}

	return true;
}

bool FPackageReader::ReadCellImports(TArray<FCellImport>& OutImports)
{
	if (PackageFileSummary.CellImportCount == 0)
	{
		return true;
	}

	if (!StartSerializeSection(PackageFileSummary.CellImportOffset))
	{
		UE_LOGFMT_LOC(LogAssetRegistry, Warning,"CellImports_BadOffset",
			"Package is unloadable: {FileName}. Reason: CellImportOffset {Offset} was invalid. ",
			("FileName", PackageFilename), ("Offset", PackageFileSummary.CellImportOffset));
		return false;
	}

	OutImports.Reserve(PackageFileSummary.CellImportCount);
	for (int32 i=0; i < PackageFileSummary.CellImportCount; ++i)
	{
		FCellImport& Import = OutImports.Emplace_GetRef();
		*this << Import;

		if (IsError())
		{
			UE_LOGFMT_LOC(LogAssetRegistry, Warning,"CellImport_SerializeFailed",
				"Package is unloadable: {FileName}. Reason: Failed to serialize cell import at index {Index}. ",
				("FileName", PackageFilename), ("Index", i));
			return false;
		}
	}

	return true;
}

bool FPackageReader::ReadSearchableNames(TMap<FTopLevelAssetPath, TArray<FName>>& OutSearchableNames)
{
	if (!SerializeNameMap() || !SerializeImportMap() || !SerializeExportMap())
	{
		return false;
	}

	FLinkerTables SearchableNames;
	if (!SerializeSearchableNamesMap(SearchableNames))
	{
		return false;
	}

	bool bSuccess = true;
	for (const TPair<FPackageIndex, TArray<FName>>& Pair : SearchableNames.SearchableNamesMap)
	{
		FPackageIndex Index = Pair.Key;
		TArray<FName> PathElements;
		while (!Index.IsNull())
		{
			FObjectResource* Obj = nullptr;
			if (Index.IsImport() && ImportMap.IsValidIndex(Index.ToImport()))
			{
				Obj = &ImportMap[Index.ToImport()];
			}
			else if (Index.IsExport() && ExportMap.IsValidIndex(Index.ToExport()))
			{
				Obj = &ExportMap[Index.ToExport()];
			}  
			if (Obj)
			{
				PathElements.Add(Obj->ObjectName);
				Index = Obj->OuterIndex;
			}
			else 
			{
				UE_LOGFMT_LOC(LogAssetRegistry, Warning,"SearchableNamePathInvalid",
					"Package is unloadable: {FileName}. Reason: SearchableName path was not valid. Index: {Index} ",
					("FileName", PackageFilename), ("Index", Pair.Key.ForDebugging()));
				bSuccess = false;
				break;
			}
		}

		if (!Index.IsNull())
		{
			// Exited loop early above
		}
		else if (PathElements.Num() == 2)
		{
			OutSearchableNames.Add(FTopLevelAssetPath(PathElements[1], PathElements[0]), Pair.Value);
		}
		else if (PathElements.Num() == 1)
		{
			OutSearchableNames.Add(FTopLevelAssetPath(PathElements[0], FName()), Pair.Value);
		}
		else 
		{
			UE_LOGFMT_LOC(LogAssetRegistry, Warning,"SearchableNamePathTooLong",
				"Package is unloadable: {FileName}. Reason: SearchableName object was not a top level asset. Path: {Path} ",
				("FileName", PackageFilename), ("Path", PathElements));
			bSuccess = false;
		}
	}

	return bSuccess;
}

bool FPackageReader::ReadWorldTileInfo(FWorldTileInfo& OutInfo)
{
	if (PackageFileSummary.WorldTileInfoDataOffset == 0)
	{
		return true;
	}

	if (!StartSerializeSection(PackageFileSummary.WorldTileInfoDataOffset))
	{
		UE_LOGFMT_LOC(LogAssetRegistry, Warning,"WorldTileInfo_BadOffset",
			"Package is unloadable: {FileName}. Reason: WorldTileInfoDataOffset {Offset} was invalid. ",
			("FileName", PackageFilename), ("Offset", PackageFileSummary.WorldTileInfoDataOffset));
		return false;
	}

	*this << OutInfo;
	if (IsError())
	{
		UE_LOGFMT_LOC(LogAssetRegistry, Warning,"WorldTileInfo_SerializeFail",
			"Package is unloadable: {FileName}. Reason: Failed to serialize WorldTileInfo. ",
			("FileName", PackageFilename));
		return false;
	}

	return true;
}

bool FPackageReader::ReadDataResourceMap(TArray<TTuple<FSoftObjectPath, FObjectDataResource>>& OutDataResources)
{
	// Data resources should never be associated with imports, but there may be a case in which an export is
	// an inner of an import, e.g. external packages
	if (!SerializeNameMap() || !SerializeExportMap() || !SerializeImportMap())
	{
		return false;
	}

	if (!StartSerializeSection(PackageFileSummary.DataResourceOffset))
	{
		UE_LOGFMT_LOC(LogAssetRegistry, Warning,"DataResourceMap_BadOffset",
			"Package is unloadable: {FileName}. Reason: DataResourceOffset {Offset} was invalid. ",
			("FileName", PackageFilename), ("Offset", PackageFileSummary.DataResourceOffset));
		return false;
	}

	TArray<FObjectDataResource> DataResources;
	FObjectDataResource::Serialize(*this, DataResources);
	if (IsError())
	{
		UE_LOGFMT_LOC(LogAssetRegistry, Warning,"DataResourceMap_SerializeFail",
			"Package is unloadable: {FileName}. Reason: Failed to serialize DataResourceMap. ",
			("FileName", PackageFilename));
		return false;
	}

	bool bSuccess = true;
	for (int32 i=0; i < DataResources.Num(); ++i)
	{
		const FObjectDataResource& DataResource = DataResources[i];
		TOptional<FSoftObjectPath> Path = LinkerIndexToSoftObjectPath(DataResource.OuterIndex);
		if (Path.IsSet())
		{
			OutDataResources.Emplace(*Path, DataResource);
		}
		else
		{
			UE_LOGFMT_LOC(LogAssetRegistry, Warning,"DataResourceMap_BadObjectPath",
				"Package is unloadable: {FileName}. Reason: Failed to get object path for data resource {Index}. ",
				("FileName", PackageFilename), ("Index", i));
			bSuccess = false;
		}
	}

	return bSuccess;
}

bool FPackageReader::ReadAssetRegistryData(TArray<FAssetData*>& AssetDataList, bool& bOutIsCookedWithoutAssetData)
{
	bOutIsCookedWithoutAssetData = false;
	if (!!(GetPackageFlags() & PKG_FilterEditorOnly))
	{
		return ReadAssetRegistryDataFromCookedPackage(AssetDataList, bOutIsCookedWithoutAssetData);
	}

	if (!SerializeNameMap() || !SerializeImportMap() || !SerializeExportMap())
	{
		return false;
	}

	if (!StartSerializeSection(PackageFileSummary.AssetRegistryDataOffset))
	{
		if (!ReadAssetDataFromThumbnailCache(AssetDataList))
		{
			// Legacy files without AR data and without a thumbnail cache are treated as having no assets
			AssetDataList.Reset();
		}
		return true;
	}

	// Determine the package name and path
	FString PackageName;
	if (!TryGetLongPackageName(PackageName))
	{
		// Path was possibly unmounted
		return false;
	}

	using namespace UE::AssetRegistry;

	EReadPackageDataMainErrorCode ErrorCode;
	if (!ReadPackageDataMain(*this, PackageName, PackageFileSummary, AssetRegistryDependencyDataOffset, AssetDataList,
		ErrorCode, &ImportMap, &ExportMap))
	{
		switch (ErrorCode)
		{
		case EReadPackageDataMainErrorCode::InvalidObjectCount:
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("EReadPackageDataMainErrorCode::InvalidObjectCount", PackageFilename);
			break;
		case EReadPackageDataMainErrorCode::InvalidTagCount:
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("EReadPackageDataMainErrorCode::InvalidTagCount", PackageFilename);
			break;
		case EReadPackageDataMainErrorCode::InvalidTag:
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("EReadPackageDataMainErrorCode::InvalidTag", PackageFilename);
			break;
		default:
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("EReadPackageDataMainErrorCode::Unknown", PackageFilename);
			break;
		}
		return false;
	}

	return true;
}

#if WITH_METADATA
bool FPackageReader::ReadMetaData(FMetaData& OutMetaData)
{
	if (PackageFileSummary.MetaDataOffset == 0)
	{
		// Empty
		return true;
	}

	if (!SerializeNameMap() || !SerializeSoftObjectPathMap())
	{
		return false;
	}

	if (!StartSerializeSection(PackageFileSummary.MetaDataOffset))
	{
		UE_LOGFMT_LOC(LogAssetRegistry, Warning,"MetaDataInvalidOffset",
			"Package is unloadable: {FileName}. Reason: Invalid metadata offset {Offset}",
			("FileName", PackageFilename), ("Offset", PackageFileSummary.MetaDataOffset));
		return false;
	}

	int32 NumObjectMetaDataMap = 0;
	int32 NumRootMetaDataMap = 0;

	*this << NumObjectMetaDataMap;
	*this << NumRootMetaDataMap;
	if (IsError())
	{
		UE_LOGFMT_LOC(LogAssetRegistry, Warning,"MetaDataFailedToReadCounts",
			"Package is unloadable: {FileName}. Reason: Failed to read metadata counts",
			("FileName", PackageFilename));
		return false;
	}

	for (int32 i=0; i < NumObjectMetaDataMap; ++i)
	{
		TPair<FSoftObjectPath, TMap<FName, FString>> ObjectMetaData;
		*this << ObjectMetaData;
		if (IsError())
		{
			UE_LOGFMT_LOC(LogAssetRegistry, Warning,"MetaDataFailedToReadObjectMetaData",
				"Package is unloadable: {FileName}. Reason: Failed to read object metadata. Index: {Index}",
				("FileName", PackageFilename), ("Index", i));
			return false;
		}
		OutMetaData.ObjectMetaDataMap.Add(ObjectMetaData.Key, ObjectMetaData.Value);
	}

	for (int32 i=0; i < NumRootMetaDataMap; ++i)
	{
		TPair<FName, FString> RootMetaData;
		*this << RootMetaData;
		if (IsError())
		{
			UE_LOGFMT_LOC(LogAssetRegistry, Warning,"MetaDataFailedToReadRootMetaData",
				"Package is unloadable: {FileName}. Reason: Failed to read root metadata. Index: {Index}",
				("FileName", PackageFilename), ("Index", i));
			return false;
		}
		OutMetaData.RootMetaDataMap.Add(RootMetaData.Key, RootMetaData.Value);
	}

	return true;
}
#endif // WITH_METADATA

bool FPackageReader::ReadLinkerObjects(TMap<FSoftObjectPath, FObjectData>& OutExports,
	TMap<FSoftObjectPath, FObjectData>& OutImports, TMap<FName, bool>& OutSoftPackageReferences)
{
	if (!SerializeNameMap() || !SerializeImportMap() || !SerializeExportMap() || !SerializeSoftPackageReferenceList())
	{
		return false;
	}
	TBitArray<> ImportUsedInGame;
	TBitArray<> SoftPackageUsedInGame;
	if (!SerializeEditorOnlyFlags(ImportUsedInGame, SoftPackageUsedInGame))
	{
		return false;
	}

	TArray<FSoftObjectPath> ExportsPaths;
	TArray<FSoftObjectPath> ImportsPaths;
	int32 NumExports = ExportMap.Num();
	int32 NumImports = ImportMap.Num();
	int32 NumSoftPackageReferences = SoftPackageReferenceList.Num();
	ConvertLinkerTableToPaths(FName(*GetLongPackageName()), ExportMap, ImportMap, ExportsPaths, ImportsPaths);
	auto GetClassPath = [&ExportsPaths, &ImportsPaths](FPackageIndex ClassIndex)
	{
		if (ClassIndex.IsNull())
		{
			return FSoftObjectPath(TEXT("/Script/CoreUObject.Class"));
		}
		else if (ClassIndex.IsExport())
		{
			int32 ExportIndex = ClassIndex.ToExport();
			return ExportIndex < ExportsPaths.Num() ? ExportsPaths[ExportIndex] :
				FSoftObjectPath(TEXT("/Script/Unknown.Unknown"));
		}
		else
		{
			int32 ImportIndex = ClassIndex.ToImport();
			return ImportIndex < ImportsPaths.Num() ? ImportsPaths[ImportIndex] :
				FSoftObjectPath(TEXT("/Script/Unknown.Unknown"));
		}
	};
	OutExports.Reserve(NumExports);
	for (int32 Index = 0; Index < NumExports; ++Index)
	{
		FObjectData Data;
		Data.ClassPath = GetClassPath(ExportMap[Index].ClassIndex);
		Data.bUsedInGame = true;
		OutExports.Add(ExportsPaths[Index], MoveTemp(Data));
	}
	OutImports.Reserve(NumImports);
	for (int32 Index = 0; Index < NumImports; ++Index)
	{
		FObjectData Data;
		const FObjectImport& Import = ImportMap[Index];
		Data.ClassPath = FSoftObjectPath::ConstructFromAssetPath(FTopLevelAssetPath(Import.ClassPackage, Import.ClassName));
		Data.bUsedInGame = ImportUsedInGame[Index];
		OutImports.Add(ImportsPaths[Index], MoveTemp(Data));
	}
	OutSoftPackageReferences.Reserve(NumSoftPackageReferences);
	for (int32 Index = 0; Index < NumSoftPackageReferences; ++Index)
	{
		OutSoftPackageReferences.Add(SoftPackageReferenceList[Index], SoftPackageUsedInGame[Index]);
	}

	auto SoftObjectPathLexicalLess = [](const FSoftObjectPath& A, const FSoftObjectPath& B) { return A.LexicalLess(B); };
	OutExports.KeySort(SoftObjectPathLexicalLess);
	OutImports.KeySort(SoftObjectPathLexicalLess);
	OutSoftPackageReferences.KeySort(FNameLexicalLess());
	return true;
}

bool FPackageReader::SerializeAssetRegistryDependencyData(TBitArray<>& OutImportUsedInGame,
	TBitArray<>& OutSoftPackageUsedInGame,
	TArray<TPair<FName, UE::AssetRegistry::EExtraDependencyFlags>>& OutExtraPackageDependencies)
{
	UE::AssetRegistry::FReadPackageDataDependenciesArgs Args;
	Args.BinaryNameAwareArchive = this;
	Args.AssetRegistryDependencyDataOffset = AssetRegistryDependencyDataOffset;
	Args.NumImports = ImportMap.Num();
	Args.NumSoftPackageReferences = SoftPackageReferenceList.Num();
	Args.PackageVersion = PackageFileSummary.GetFileVersionUE();

	ClearError();
	Loader->ClearError();

	if (!UE::AssetRegistry::ReadPackageDataDependencies(Args))
	{
		UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeAssetRegistryDependencyData", PackageFilename);
		return false;
	}
	OutImportUsedInGame = MoveTemp(Args.ImportUsedInGame);
	OutSoftPackageUsedInGame = MoveTemp(Args.SoftPackageUsedInGame);
	OutExtraPackageDependencies = MoveTemp(Args.ExtraPackageDependencies);
	return true;
}

bool FPackageReader::ReadPackageTrailer(UE::FPackageTrailer& OutTrailer)
{
	if (PackageFileSummary.PayloadTocOffset == INDEX_NONE)
	{
		return true;
	}

	if (!StartSerializeSection(PackageFileSummary.PayloadTocOffset))
	{
		UE_LOGFMT_LOC(LogAssetRegistry, Warning,"PackageTrailer_BadOffset",
			"Package is unloadable: {FileName}. Reason: Bad offset for PayloadTocOffset {Offset}. ",
			("FileName", PackageFilename), ("Offset", PackageFileSummary.PayloadTocOffset));
		return false;
	}

	if (!OutTrailer.TryLoad(*this))
	{
		UE_LOGFMT_LOC(LogAssetRegistry, Warning,"PackageTrailer_SerializeFail",
			"Package is unloadable: {FileName}. Reason: Failed to deserialize trailer. ",
			("FileName", PackageFilename));
		return false;
	}

	return true;
}

bool FPackageReader::SerializePackageTrailer(FAssetPackageData& PackageData)
{
	if (!StartSerializeSection(PackageFileSummary.PayloadTocOffset))
	{
		PackageData.SetHasVirtualizedPayloads(false);
		return true;
	}

	UE::FPackageTrailer Trailer;
	if (!Trailer.TryLoad(*this))
	{
		// This is not necessarily corrupt; TryLoad will return false if the trailer is empty
		PackageData.SetHasVirtualizedPayloads(false);
		return true;
	}

	PackageData.SetHasVirtualizedPayloads(Trailer.GetNumPayloads(UE::EPayloadStorageType::Virtualized) > 0);
	return true;
}

void FPackageReader::ApplyRelocationToImportMapAndSoftPackageReferenceList(FStringView LoadedPackageName, 
	TArray<FName>& InOutSoftPackageReferenceList,
	TArray<TPair<FName, UE::AssetRegistry::EExtraDependencyFlags>>& InOutExtraPackageDependencies)
{
#if WITH_EDITOR
	UE::Package::Relocation::Private::FPackageRelocationContext RelocationArgs;
	if (UE::Package::Relocation::Private::ShouldApplyRelocation(PackageFileSummary, LoadedPackageName, RelocationArgs))
	{
		UE_LOGF(LogPackageRelocation, Verbose, "Detected relocated package (%.*ls). The package was saved as (%ls).",
			LoadedPackageName.Len(), LoadedPackageName.GetData(), *PackageFileSummary.PackageName);
		UE::Package::Relocation::Private::ApplyRelocationToObjectImportMap(RelocationArgs, ImportMap);
		UE::Package::Relocation::Private::ApplyRelocationToNameArray(RelocationArgs, InOutSoftPackageReferenceList);
		for (TPair<FName, UE::AssetRegistry::EExtraDependencyFlags>& Pair : InOutExtraPackageDependencies)
		{
			FNameBuilder Package(Pair.Key);
			FNameBuilder RelocatedPackageName;
			if (UE::Package::Relocation::Private::TryRelocateReference(
				RelocationArgs, Package.ToView(), RelocatedPackageName))
			{
				Pair.Key = *RelocatedPackageName;
			}
		}
	}
#endif
}

bool FPackageReader::ReadAssetDataFromThumbnailCache(TArray<FAssetData*>& AssetDataList)
{
	if (!SerializeThumbnailMap())
	{
		return false;
	}

	// Iterate over every thumbnail entry and harvest the objects classnames
	for(const FObjectFullNameAndThumbnail& Thumbnail : ThumbnailMap)
	{
		FString ClassName;
		FString PackageName;
		FString ObjectName;
		FString SubobjectName;
		FPackageName::SplitFullObjectPath(Thumbnail.ObjectFullName.ToString(), ClassName, PackageName, ObjectName, SubobjectName);
		FString PackagePath = FPackageName::GetLongPackagePath(PackageName);

		// Create a new FAssetData for this asset and update it with the gathered data
		AssetDataList.Add(new FAssetData(
			FName(PackageName),
			FName(PackagePath), 
			FName(ObjectName), // AssetName
			FTopLevelAssetPath(ClassName), 
			FAssetDataTagMap(), 
			PackageFileSummary.ChunkIDs, 
			PackageFileSummary.GetPackageFlags())
		);
	}

	return true;
}

bool FPackageReader::ReadAssetRegistryDataFromCookedPackage(TArray<FAssetData*>& AssetDataList, bool& bOutIsCookedWithoutAssetData)
{
	FString PackageName;
	if (!TryGetLongPackageName(PackageName))
	{
		return false;
	}

	bool bFoundAtLeastOneAsset = false;

	// If the packaged is saved with the right version we have the information
	// which of the objects in the export map as the asset.
	// Otherwise we need to store a temp minimal data and then force load the asset
	// to re-generate its registry data
	if (UEVer() >= VER_UE4_COOKED_ASSETS_IN_EDITOR_SUPPORT)
	{
		const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);

		if (!SerializeNameMap() || !SerializeImportMap() || !SerializeExportMap())
		{
			return false;
		}
		for (const FObjectExport& Export : ExportMap)
		{
			if (Export.bIsAsset)
			{
				// We need to get the class name from the import/export maps
				FString ObjectClassName;
				if (Export.ClassIndex.IsNull())
				{
					ObjectClassName = UClass::StaticClass()->GetPathName();
				}
				else if (Export.ClassIndex.IsExport())
				{
					const FObjectExport& ClassExport = ExportMap[Export.ClassIndex.ToExport()];
					ObjectClassName = PackageName;
					ObjectClassName += '.'; 
					ClassExport.ObjectName.AppendString(ObjectClassName);
				}
				else if (Export.ClassIndex.IsImport())
				{
					const FObjectImport& ClassImport = ImportMap[Export.ClassIndex.ToImport()];
					const FObjectImport& ClassPackageImport = ImportMap[ClassImport.OuterIndex.ToImport()];
					ClassPackageImport.ObjectName.AppendString(ObjectClassName);
					ObjectClassName += '.';
					ClassImport.ObjectName.AppendString(ObjectClassName);
				}

				AssetDataList.Add(new FAssetData(FName(*PackageName), FName(*PackagePath), Export.ObjectName, FTopLevelAssetPath(ObjectClassName), FAssetDataTagMap(), TArray<int32>(), GetPackageFlags()));
				bFoundAtLeastOneAsset = true;
			}
		}
	}
	bOutIsCookedWithoutAssetData = !bFoundAtLeastOneAsset;
	return true;
}

bool FPackageReader::ReadDependencyData(FPackageDependencyData& OutDependencyData, EReadOptions Options)
{
	FString PackageNameString;
	if (!TryGetLongPackageName(PackageNameString))
	{
		// Path was possibly unmounted
		return false;
	}

	OutDependencyData.PackageName = FName(*PackageNameString);
	if (!EnumHasAnyFlags(Options, EReadOptions::PackageData | EReadOptions::Dependencies))
	{
		return true;
	}

	if (!SerializeNameMap() || !SerializeImportMap())
	{
		return false;
	}

	if (EnumHasAnyFlags(Options, EReadOptions::PackageData))
	{
		OutDependencyData.bHasPackageData = true;
		FAssetPackageData& PackageData = OutDependencyData.PackageData;
		PackageData.DiskSize = PackageFileSize;
#if WITH_EDITORONLY_DATA
		PackageData.SetPackageSavedHash(PackageFileSummary.GetSavedHash());
#endif
		PackageData.SetCustomVersions(PackageFileSummary.GetCustomVersionContainer().GetAllVersions());
		PackageData.FileVersionUE = PackageFileSummary.GetFileVersionUE();
		PackageData.FileVersionLicenseeUE = PackageFileSummary.GetFileVersionLicenseeUE();
		PackageData.SetIsLicenseeVersion(PackageFileSummary.SavedByEngineVersion.IsLicenseeVersion());
		PackageData.Extension = FPackagePath::ParseExtension(PackageFilename);

		// Add the filesystem location to any existing location as it's 
        // possible we have the same content available from more than one location.
		PackageData.SetPackageLocation(FPackageName::EPackageLocationFilter(
			uint8(FPackageName::EPackageLocationFilter::FileSystem) | uint8(PackageData.GetPackageLocation())));

		if (!SerializeImportedClasses(ImportMap, PackageData.ImportedClasses))
		{
			return false;
		}
		if (!SerializePackageTrailer(PackageData))
		{
			return false;
		}
	}

	if (EnumHasAnyFlags(Options, EReadOptions::Dependencies))
	{
		OutDependencyData.bHasDependencyData = true;
		if (!SerializeSoftPackageReferenceList())
		{
			return false;
		}
		FLinkerTables SearchableNames;
		if (!SerializeSearchableNamesMap(SearchableNames))
		{
			return false;
		}

		TBitArray<> ImportUsedInGame;
		TBitArray<> SoftPackageUsedInGame;
		TArray<TPair<FName, UE::AssetRegistry::EExtraDependencyFlags>> ExtraPackageDependencies;
		if (!SerializeAssetRegistryDependencyData(ImportUsedInGame, SoftPackageUsedInGame, ExtraPackageDependencies))
		{
			return false;
		}

		ApplyRelocationToImportMapAndSoftPackageReferenceList(PackageNameString, SoftPackageReferenceList,
			ExtraPackageDependencies);

		OutDependencyData.LoadDependenciesFromPackageHeader(OutDependencyData.PackageName, ImportMap,
			SoftPackageReferenceList, SearchableNames.SearchableNamesMap, ImportUsedInGame, SoftPackageUsedInGame,
			ExtraPackageDependencies);
	}

	return true;
}

bool FPackageReader::SerializeNameMap()
{
	if (NameMap.Num() > 0)
	{
		return true;
	}
	if( PackageFileSummary.NameCount > 0 )
	{
		if (!StartSerializeSection(PackageFileSummary.NameOffset))
		{
			UE_LOGFMT_LOC(LogAssetRegistry, Warning, "SerializeNameMapInvalidNameOffset",
				"Package is unloadable: {FileName}. Reason: Failed to seek to name table offset {NameOffset} in package of size {PackageSize}",
				("FileName", PackageFilename), ("NameOffset", PackageFileSummary.NameOffset), ("PackageSize", PackageFileSize));
			return false;
		}

		const int MinSizePerNameEntry = 1;
		if (PackageFileSize < Tell() + PackageFileSummary.NameCount * MinSizePerNameEntry)
		{
			UE_LOGFMT_LOC(LogAssetRegistry, Warning, "SerializeNameMapInvalidNameCount",
				"Package is unloadable: {FileName}. Reason: Name table count {NameCount} in package of size {PackageSize} at name offset {NameOffset}",
				("FileName", PackageFilename), ("NameCount", PackageFileSummary.NameCount), ("NameOffset", PackageFileSummary.NameOffset), ("PackageSize", PackageFileSize));
			return false;
		}

		for ( int32 NameMapIdx = 0; NameMapIdx < PackageFileSummary.NameCount; ++NameMapIdx )
		{
			// Read the name entry from the file.
			FNameEntrySerialized NameEntry(ENAME_LinkerConstructor);
			*this << NameEntry;
			if (IsError())
			{
				UE_LOGFMT_LOC(LogAssetRegistry, Warning, "SerializeNameMapInvalidName",
					"Package is unloadable: {FileName}. Reason: Invalid name at index {NameIndex}",
					("FileName", PackageFilename), ("NameIndex", NameMapIdx));
				NameMap.Reset();
				return false;
			}
			NameMap.Add(FName(NameEntry));
		}
	}

	return true;
}

bool FPackageReader::SerializeImportMap()
{
	if (ImportMap.Num() > 0)
	{
		return true;
	}

	if( PackageFileSummary.ImportCount > 0 )
	{
		if (!StartSerializeSection(PackageFileSummary.ImportOffset))
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeImportMapInvalidImportOffset", PackageFilename);
			return false;
		}

		const int MinSizePerImport = 1;
		if (PackageFileSize < Tell() + PackageFileSummary.ImportCount * MinSizePerImport)
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeImportMapInvalidImportCount", PackageFilename);
			return false;
		}
		ImportMap.Reserve(PackageFileSummary.ImportCount);
		for ( int32 ImportMapIdx = 0; ImportMapIdx < PackageFileSummary.ImportCount; ++ImportMapIdx )
		{
			*this << ImportMap.Emplace_GetRef();
			if (IsError())
			{
				UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeImportMapInvalidImport", PackageFilename);
				ImportMap.Reset();
				return false;
			}
		}
	}

	return true;
}

static FName CoreUObjectPackageName("/Script/CoreUObject");
static FName ScriptStructName("ScriptStruct");

bool FPackageReader::SerializeImportedClasses(const TArray<FObjectImport>& InImportMap, TArray<FName>& OutClassNames)
{
	OutClassNames.Reset();

	TSet<int32> ClassImportIndices;
	// Any import that is specified as the class of an export is an imported class
	if (PackageFileSummary.ExportCount > 0)
	{
		if (!StartSerializeSection(PackageFileSummary.ExportOffset))
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeExportMapInvalidExportOffset", PackageFilename);
			return false;
		}

		const int MinSizePerExport = 1;
		if (PackageFileSize < Tell() + PackageFileSummary.ExportCount * MinSizePerExport)
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeExportMapInvalidExportCount", PackageFilename);
			return false;
		}
		FObjectExport ExportBuffer;
		for (int32 ExportMapIdx = 0; ExportMapIdx < PackageFileSummary.ExportCount; ++ExportMapIdx)
		{
			*this << ExportBuffer;
			if (IsError())
			{
				UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeExportMapInvalidExport", PackageFilename);
				return false;
			}
			if (ExportBuffer.ClassIndex.IsImport())
			{
				ClassImportIndices.Add(ExportBuffer.ClassIndex.ToImport());
			}
		}
	}
	// Any imports of types UScriptStruct are an imported struct and need to be added to ImportedClasses
	// This covers e.g. DataTable, which has a RowStruct pointer that it uses in its native serialization to
	// serialize data into its rows
	// TODO: Projects may create their own ScriptStruct subclass, and if they use one of these subclasses
	// as a serialized-external-struct-pointer then we will miss it. In a future implementation we will 
	// change the PackageReader to report all imports, and allow the AssetRegistry to decide which ones
	// are classes based on its class database.
	for (int32 ImportIndex = 0; ImportIndex < InImportMap.Num(); ++ImportIndex)
	{
		const FObjectImport& ObjectImport = InImportMap[ImportIndex];
		if (ObjectImport.ClassPackage == CoreUObjectPackageName && ObjectImport.ClassName == ScriptStructName)
		{
			ClassImportIndices.Add(ImportIndex);
		}
	}

	TArray<FName, TInlineAllocator<5>>  ParentChain;
	FNameBuilder ClassObjectPath;
	for (int32 ClassImportIndex : ClassImportIndices)
	{
		ParentChain.Reset();
		ClassObjectPath.Reset();
		if (!InImportMap.IsValidIndex(ClassImportIndex))
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeImportedClassesInvalidClassIndex", PackageFilename);
			return false;
		}
		bool bParentChainComplete = false;
		int32 CurrentParentIndex = ClassImportIndex;
		for (;;)
		{
			const FObjectImport& ObjectImport = InImportMap[CurrentParentIndex];
			ParentChain.Add(ObjectImport.ObjectName);
			if (ObjectImport.OuterIndex.IsImport())
			{
				CurrentParentIndex = ObjectImport.OuterIndex.ToImport();
				if (!InImportMap.IsValidIndex(CurrentParentIndex))
				{
					UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeImportedClassesInvalidImportInParentChain",
						PackageFilename);
					return false;
				}
			}
			else if (ObjectImport.OuterIndex.IsNull())
			{
				bParentChainComplete = true;
				break;
			}
			else
			{
				check(ObjectImport.OuterIndex.IsExport());
				// Ignore classes in an external package but with an object in this package as one of their outers;
				// We do not need to handle that case yet for Import Classes, and we would have to make this
				// loop more complex (searching in both ExportMap and ImportMap) to do so
				break;
			}
		}

		if (bParentChainComplete)
		{
			int32 NumTokens = ParentChain.Num();
			check(NumTokens >= 1);
			const TCHAR Delimiters[] = { TEXT('.'), SUBOBJECT_DELIMITER_CHAR, TEXT('.') };
			int32 DelimiterIndex = 0;
			ParentChain[NumTokens - 1].AppendString(ClassObjectPath);
			for (int32 TokenIndex = NumTokens - 2; TokenIndex >= 0; --TokenIndex)
			{
				ClassObjectPath << Delimiters[DelimiterIndex];
				DelimiterIndex = FMath::Min(DelimiterIndex+1, static_cast<int32>(UE_ARRAY_COUNT(Delimiters))-1);
				ParentChain[TokenIndex].AppendString(ClassObjectPath);
			}
			OutClassNames.Emplace(ClassObjectPath);
		}
	}

	OutClassNames.Sort(FNameLexicalLess());
	return true;
}

bool FPackageReader::SerializeExportMap()
{
	if (ExportMap.Num() > 0)
	{
		return true;
	}

	if (PackageFileSummary.ExportCount > 0)
	{
		if (!StartSerializeSection(PackageFileSummary.ExportOffset))
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeExportMapInvalidExportOffset", PackageFilename);
			return false;
		}

		const int MinSizePerExport = 1;
		if (PackageFileSize < Tell() + PackageFileSummary.ExportCount * MinSizePerExport)
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeExportMapInvalidExportCount", PackageFilename);
			return false;
		}
		ExportMap.Reserve(PackageFileSummary.ExportCount);
		for (int32 ExportMapIdx = 0; ExportMapIdx < PackageFileSummary.ExportCount; ++ExportMapIdx)
		{
			*this << ExportMap.Emplace_GetRef();
			if (IsError())
			{
				UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeExportMapInvalidExport", PackageFilename);
				ExportMap.Reset();
				return false;
			}
		}
	}

	return true;
}

bool FPackageReader::SerializeDependsMap()
{
	if (DependsMap.Num() > 0)
	{
		return true;
	}

	if (PackageFileSummary.DependsOffset > 0 && PackageFileSummary.ExportCount > 0)
	{
		if (!StartSerializeSection(PackageFileSummary.DependsOffset))
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeDependsMapInvalidOffset", PackageFilename);
			return false;
		}

		const int MinSizePerExport = 1;
		if (PackageFileSize < Tell() + PackageFileSummary.ExportCount * MinSizePerExport)
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeDependsMapInvalidExportCount", PackageFilename);
			return false;
		}
		DependsMap.Reserve(PackageFileSummary.ExportCount);
		for (int32 DependsMapIdx = 0; DependsMapIdx < PackageFileSummary.ExportCount; ++DependsMapIdx)
		{
			*this << DependsMap.Emplace_GetRef();
			if (IsError())
			{
				UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeDependsMapInvalidEntry", PackageFilename);
				DependsMap.Reset();
				return false;
			}
		}
	}

	return true;
}

bool FPackageReader::SerializeSoftPackageReferenceList()
{
	if (SoftPackageReferenceList.Num() > 0)
	{
		return true;
	}

	if (UEVer() >= VER_UE4_ADD_STRING_ASSET_REFERENCES_MAP && PackageFileSummary.SoftPackageReferencesOffset > 0 && PackageFileSummary.SoftPackageReferencesCount > 0)
	{
		if (!StartSerializeSection(PackageFileSummary.SoftPackageReferencesOffset))
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeSoftPackageReferenceListInvalidReferencesOffset", PackageFilename);
			return false;
		}
		
		const int MinSizePerSoftPackageReference = 1;
		if (PackageFileSize < Tell() + PackageFileSummary.SoftPackageReferencesCount * MinSizePerSoftPackageReference)
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeSoftPackageReferenceListInvalidReferencesCount", PackageFilename);
			return false;
		}

		SoftPackageReferenceList.Reserve(PackageFileSummary.SoftPackageReferencesCount);
		if (UEVer() < VER_UE4_ADDED_SOFT_OBJECT_PATH)
		{
			for (int32 ReferenceIdx = 0; ReferenceIdx < PackageFileSummary.SoftPackageReferencesCount; ++ReferenceIdx)
			{
				FString PackageName;
				*this << PackageName;
				if (IsError())
				{
					UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeSoftPackageReferenceListInvalidReferencePreSoftObjectPath", PackageFilename);
					SoftPackageReferenceList.Reset();
					return false;
				}

				if (UEVer() < VER_UE4_KEEP_ONLY_PACKAGE_NAMES_IN_STRING_ASSET_REFERENCES_MAP)
				{
					PackageName = FPackageName::GetNormalizedObjectPath(PackageName);
					if (!PackageName.IsEmpty())
					{
						PackageName = FPackageName::ObjectPathToPackageName(PackageName);
					}
				}

				SoftPackageReferenceList.Add(FName(*PackageName));
			}
		}
		else
		{
			for (int32 ReferenceIdx = 0; ReferenceIdx < PackageFileSummary.SoftPackageReferencesCount; ++ReferenceIdx)
			{
				FName PackageName;
				*this << PackageName;
				if (IsError())
				{
					UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeSoftPackageReferenceListInvalidReference", PackageFilename);
					SoftPackageReferenceList.Reset();
					return false;
				}

				SoftPackageReferenceList.Add(PackageName);
			}
		}
	}

	return true;
}

bool FPackageReader::SerializeSoftObjectPathMap()
{
	if (SoftObjectPathMap.Num() > 0)
	{
		return true;
	}

	if (PackageFileSummary.SoftObjectPathsOffset > 0 && PackageFileSummary.SoftObjectPathsCount > 0)
	{
		if (!StartSerializeSection(PackageFileSummary.SoftObjectPathsOffset))
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeSoftObjectPathMapListInvalidOffset", PackageFilename);
			return false;
		}
		
		int32 MinSizePerSoftObjectPath = 0; 
		if (UEVer() < VER_UE4_ADDED_SOFT_OBJECT_PATH)
		{
			MinSizePerSoftObjectPath = 8; // FString
		}
		else if (UEVer() < EUnrealEngineObjectUE5Version::FSOFTOBJECTPATH_REMOVE_ASSET_PATH_FNAMES)
		{
			MinSizePerSoftObjectPath = 8 + 8; // FName + FString 
		}
		else
		{
			MinSizePerSoftObjectPath = 8 + 8 + 8; // 2xFName + FString
		}

		if (PackageFileSize < Tell() + PackageFileSummary.SoftObjectPathsCount * MinSizePerSoftObjectPath)
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeSoftObjectPathMapInvalidCount", PackageFilename);
			return false;
		}

		SoftObjectPathMap.Reserve(PackageFileSummary.SoftObjectPathsCount);
		for (int32 PathIdx = 0; PathIdx < PackageFileSummary.SoftObjectPathsCount; ++PathIdx)
		{
			FSoftObjectPath Path;
			Path.SerializePath(*this);
			if (IsError())
			{
				UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeSoftObjectPathMapInvalidPath", PackageFilename);
				SoftObjectPathMap.Reset();
				return false;
			}

			SoftObjectPathMap.Add(Path);
		}
	}

	return true;
}

bool FPackageReader::SerializeGatherableTextDataMap()
{
	if (GatherableTextDataMap.Num() > 0)
	{
		return true;
	}
	
	if (PackageFileSummary.GatherableTextDataCount > 0 && PackageFileSummary.GatherableTextDataOffset > 0)
	{
		if (!StartSerializeSection(PackageFileSummary.GatherableTextDataOffset))
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeGatherableTextDataMapInvalidOffset", PackageFilename);
			return false;
		}

		int32 MinSizePerText = 8 + 8 + 4;  // Two FStrings and as an empty array as a lower bound
		if (PackageFileSize < Tell() + PackageFileSummary.GatherableTextDataCount * MinSizePerText)
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeGatherableTextDataMapInvalidCount", PackageFilename);
			return false;
		}

		GatherableTextDataMap.Reset(PackageFileSummary.GatherableTextDataCount);	
		for (int32 Index = 0; Index < PackageFileSummary.GatherableTextDataCount; ++Index)
		{
			FGatherableTextData Data;
			*this << Data;	
			if (IsError())
			{
				UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeGatherableTextDataMapInvalidEntry", PackageFilename);
				GatherableTextDataMap.Reset();
				return false;
			}
			GatherableTextDataMap.Emplace(MoveTemp(Data));	
		}
	}

	return true;
}

bool FPackageReader::SerializeThumbnailMap()
{
	if (ThumbnailMap.Num() > 0)
	{
		return true;
	}
	
	if (PackageFileSummary.ThumbnailTableOffset > 0)
	{
		if (!StartSerializeSection(PackageFileSummary.ThumbnailTableOffset))
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeThumbnailMapInvalidOffset", PackageFilename);
			return false;
		}

		FString PackageName;
		if (!TryGetLongPackageName(PackageName))
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeThumbnailMapNoPackageName", PackageFilename);
			return false;
		}

		int32 NumThumbnails = 0;
		*this << NumThumbnails;
		if (IsError() || NumThumbnails < 0)
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeThumbnailMapInvalidCount", PackageFilename);
			return false;
		}	

		int32 MinSizePerThumbnail = 8 + 8 + 4;  // Two FStrings and an offset 
		if (PackageFileSize < Tell() + PackageFileSummary.GatherableTextDataCount * MinSizePerThumbnail)
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeThumbnailMapInvalidCount", PackageFilename);
			return false;
		}
		ThumbnailMap.Reset(NumThumbnails);
		
		for (int32 Index=0; Index < NumThumbnails; ++Index)
		{
			FString ObjectClassName;
			*this << ObjectClassName;
			FString ObjectPathWithoutPackageName;
			*this << ObjectPathWithoutPackageName;
			int32 Offset = 0;
			*this << Offset;

			if (IsError())
			{
				UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeThumbnailMapInvalidEntry", PackageFilename);
				return false;
			}
			
			FObjectFullNameAndThumbnail Thumbnail;
			Thumbnail.ObjectFullName = FName(WriteToString<FName::StringBufferSize>(ObjectClassName, TEXT(" "), PackageName, TEXT("."), ObjectPathWithoutPackageName));
			Thumbnail.FileOffset = Offset;
			ThumbnailMap.Emplace(MoveTemp(Thumbnail));
		}
	}

	return true;
}

bool FPackageReader::SerializeEditorOnlyFlags(TBitArray<>& OutImportUsedInGame, TBitArray<>& OutSoftPackageUsedInGame)
{
	if (AssetRegistryDependencyDataOffset == INDEX_NONE &&
		!(PackageFileSummary.GetPackageFlags() & PKG_FilterEditorOnly) &&
		StartSerializeSection(PackageFileSummary.AssetRegistryDataOffset))
	{
		UE::AssetRegistry::FDeserializePackageData DeserializePackageData;
		UE::AssetRegistry::EReadPackageDataMainErrorCode Error;
		if (!DeserializePackageData.DoSerialize(*this, PackageFileSummary, Error))
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("EReadPackageDataMainErrorCode::InvalidTagCount", PackageFilename);
			return false;
		}
		AssetRegistryDependencyDataOffset = DeserializePackageData.DependencyDataOffset;
	}

	if (AssetRegistryDependencyDataOffset == INDEX_NONE)
	{
		// For cooked packages or old package versions that did not write out the dependency flags,
		// set default values of the flags
		OutImportUsedInGame.Init(true, ImportMap.Num());
		OutSoftPackageUsedInGame.Init(true, SoftPackageReferenceList.Num());
		return true;
	}

	ClearError();
	Loader->ClearError();

	UE::AssetRegistry::FReadPackageDataDependenciesArgs Args;
	Args.BinaryNameAwareArchive = this;
	Args.AssetRegistryDependencyDataOffset = AssetRegistryDependencyDataOffset;
	Args.NumImports = ImportMap.Num();
	Args.NumSoftPackageReferences = SoftPackageReferenceList.Num();
	Args.PackageVersion = PackageFileSummary.GetFileVersionUE();

	if (!UE::AssetRegistry::ReadPackageDataDependencies(Args))
	{
		UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeAssetRegistryDependencyData", PackageFilename);
		return false;
	}
	OutImportUsedInGame = MoveTemp(Args.ImportUsedInGame);
	OutSoftPackageUsedInGame = MoveTemp(Args.SoftPackageUsedInGame);

	return true;
}

bool FPackageReader::SerializeSearchableNamesMap(FLinkerTables& OutSearchableNames)
{
	if (UEVer() >= VER_UE4_ADDED_SEARCHABLE_NAMES && PackageFileSummary.SearchableNamesOffset > 0)
	{
		if (!StartSerializeSection(PackageFileSummary.SearchableNamesOffset))
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeSearchableNamesMapInvalidOffset", PackageFilename);
			return false;
		}

		OutSearchableNames.SerializeSearchableNamesMap(*this);
		if (IsError())
		{
			UE_PACKAGEREADER_CORRUPTPACKAGE_WARNING("SerializeSearchableNamesMapInvalidSearchableNamesMap", PackageFilename);
			return false;
		}
	}

	return true;
}

void FPackageReader::Serialize( void* V, int64 Length )
{
	check(Loader);
	Loader->Serialize( V, Length );
	if (Loader->IsError())
	{
		SetError();
	}
}

bool FPackageReader::Precache( int64 PrecacheOffset, int64 PrecacheSize )
{
	check(Loader);
	return Loader->Precache( PrecacheOffset, PrecacheSize );
}

void FPackageReader::Seek( int64 InPos )
{
	check(Loader);
	Loader->Seek( InPos );
	if (Loader->IsError())
	{
		SetError();
	}
}

int64 FPackageReader::Tell()
{
	check(Loader);
	return Loader->Tell();
}

int64 FPackageReader::TotalSize()
{
	check(Loader);
	return Loader->TotalSize();
}

uint32 FPackageReader::GetPackageFlags() const
{
	return PackageFileSummary.GetPackageFlags();
}

FArchive& FPackageReader::operator<<( FName& Name )
{
	int32 NameIndex;
	FArchive& Ar = *this;
	Ar << NameIndex;

	if( !NameMap.IsValidIndex(NameIndex) )
	{
		UE_LOGF(LogAssetRegistry, Warning,
			"Package is unloadable: %ls. Reason: Bad name index %i/%i when reading package.",
			*PackageFilename, NameIndex, NameMap.Num() );
		SetError();
		return *this;
	}

	// if the name wasn't loaded (because it wasn't valid in this context)
	if (NameMap[NameIndex] == NAME_None)
	{
		int32 TempNumber;
		Ar << TempNumber;
		Name = NAME_None;
	}
	else
	{
		int32 Number;
		Ar << Number;
		// simply create the name from the NameMap's name and the serialized instance number
		Name = FName(NameMap[NameIndex], Number);
	}

	return *this;
}

// Potentially look up by index into the soft object path table as with LinkerLoad
FArchive& FPackageReader::operator<<(FSoftObjectPath& Value)
{
	FArchive& Ar = *this;
	// Comment taken from FLinkerLoad:
	// if we have items in the soft object path list consider soft object path saved as index into it.
	// Otherwise deserialize them as usual, cooking for example might not want soft object path serialized as index
	if (SoftObjectPathMap.Num() > 0)
	{
		int32 SoftObjectPathIndex = INDEX_NONE;
		Ar << SoftObjectPathIndex;
		if (SoftObjectPathMap.IsValidIndex(SoftObjectPathIndex))
		{
			Value = SoftObjectPathMap[SoftObjectPathIndex];

#if WITH_EDITOR
			Value.PostLoadPath(this);
#endif // WITH_EDITOR
		}
		else
		{
			UE_LOGFMT_LOC(LogAssetRegistry, Warning,"SoftObjectPathInvalidIndex",
				"Package is unloadable: {FileName}. Reason: Bad soft object path index. Index: {Index}/{Count}",
				("FileName", PackageFilename), ("Index", SoftObjectPathIndex), ("Count", SoftObjectPathMap.Num()));
			Value = FSoftObjectPath();
			SetError();
		}
	}
	else
	{
		FArchiveUObject::operator<<(Value);
	}
	return Ar;
}

void FPackageReader::ConvertLinkerTableToPaths(FName InPackageName, TArray<FObjectExport>& InExportMap, TArray<FObjectImport>& InImportMap,
	TArray<FSoftObjectPath>& OutExports, TArray<FSoftObjectPath>& OutImports)
{
	TMap<FPackageIndex, FSoftObjectPath> ObjectForIndex;
	TFunction<FSoftObjectPath& (FPackageIndex Index)> GetSoftObjectPath;
	FSoftObjectPath EmptySoftObjectPath;
	int32 Counter = 0;
	int32 NumExports = InExportMap.Num();
	int32 NumImports = InImportMap.Num();
	int32 NumObjects = NumExports + NumImports;

	ObjectForIndex.Reserve(NumObjects);
	GetSoftObjectPath = [&InImportMap, &InExportMap, &ObjectForIndex, &GetSoftObjectPath, &EmptySoftObjectPath,
		&Counter, NumObjects, InPackageName]
	(FPackageIndex Index) -> FSoftObjectPath&
	{
		if (++Counter > (NumObjects + 1) * 2)
		{
			// Recursive overflow should be impossible because every call fills in a new element of the table
			check(false);
		}
		ON_SCOPE_EXIT
		{
			--Counter;
		};
		if (Index.IsNull())
		{
			return EmptySoftObjectPath;
		}
		FSoftObjectPath* Existing = ObjectForIndex.Find(Index);
		if (Existing)
		{
			return *Existing;
		}

		FPackageIndex ParentIndex;
		FName ObjectName;
		if (Index.IsExport())
		{
			int32 ExportIndex = Index.ToExport();
			if (ExportIndex < InExportMap.Num())
			{
				FObjectExport& Export = InExportMap[ExportIndex];
				ParentIndex = Export.OuterIndex;
				ObjectName = Export.ObjectName;
			}
		}
		else
		{
			int32 ImportIndex = Index.ToImport();
			if (ImportIndex < InImportMap.Num())
			{
				FObjectImport& Import = InImportMap[ImportIndex];
				ParentIndex = Import.OuterIndex;
				ObjectName = Import.ObjectName;
			}
		}
		FSoftObjectPath Result;
		if (!ObjectName.IsNone())
		{
			FSoftObjectPath& ParentPath = GetSoftObjectPath(ParentIndex);
			if (ParentPath.IsNull())
			{
				if (Index.IsExport())
				{
					Result = FSoftObjectPath::ConstructFromAssetPath(FTopLevelAssetPath( InPackageName, ObjectName));
				}
				else
				{
					Result = FSoftObjectPath::ConstructFromAssetPath(FTopLevelAssetPath(ObjectName, NAME_None));
				}
			}
			else if (ParentPath.GetAssetFName().IsNone())
			{
				Result = FSoftObjectPath::ConstructFromAssetPath(FTopLevelAssetPath(ParentPath.GetLongPackageFName(), ObjectName));
			}
			else if (ParentPath.GetSubPathString().IsEmpty())
			{
				Result = FSoftObjectPath::ConstructFromAssetPathAndSubpath(
					FTopLevelAssetPath(ParentPath.GetLongPackageFName(), ParentPath.GetAssetFName()),
					ObjectName.ToString());
			}
			else
			{
				Result = FSoftObjectPath::ConstructFromAssetPathAndSubpath(
					FTopLevelAssetPath(ParentPath.GetLongPackageFName(), ParentPath.GetAssetFName()),
					ParentPath.GetSubPathString() + TEXT(".") + ObjectName.ToString());
			}
		}

		// Note we have to look up the Element again in ObjectForIndex. We can not cache a FindOrAdd Result for Index
		// because we have potentially modified ObjectForIndex by calling GetSoftObjectPath(ParentIndex).
		return ObjectForIndex.Add(Index, Result);
	};
	OutExports.Reset(NumExports);
	for (int32 ExportIndex = 0; ExportIndex < NumExports; ++ExportIndex)
	{
		OutExports.Add(GetSoftObjectPath(FPackageIndex::FromExport(ExportIndex)));
	}
	OutImports.Reset(NumImports);
	for (int32 ImportIndex = 0; ImportIndex < NumImports; ++ImportIndex)
	{
		OutImports.Add(GetSoftObjectPath(FPackageIndex::FromImport(ImportIndex)));
	}
}


namespace UE::AssetRegistry
{
	class FNameMapAwareArchive : public FArchiveProxy
	{
		TArray<FNameEntryId>	NameMap;

	public:

		FNameMapAwareArchive(FArchive& Inner)
			: FArchiveProxy(Inner)
		{}

		FORCEINLINE virtual FArchive& operator<<(FName& Name) override
		{
			FArchive& Ar = *this;
			int32 NameIndex;
			Ar << NameIndex;
			int32 Number = 0;
			Ar << Number;

			if (NameMap.IsValidIndex(NameIndex))
			{
				// if the name wasn't loaded (because it wasn't valid in this context)
				FNameEntryId MappedName = NameMap[NameIndex];

				// simply create the name from the NameMap's name and the serialized instance number
				Name = FName::CreateFromDisplayId(MappedName, Number);
			}
			else
			{
				Name = FName();
				SetCriticalError();
			}

			return *this;
		}

		void SerializeNameMap(const FPackageFileSummary& PackageFileSummary)
		{
			Seek(PackageFileSummary.NameOffset);
			NameMap.Reserve(PackageFileSummary.NameCount);
			FNameEntrySerialized NameEntry(ENAME_LinkerConstructor);
			for (int32 Idx = NameMap.Num(); Idx < PackageFileSummary.NameCount; ++Idx)
			{
				*this << NameEntry;
				NameMap.Emplace(FName(NameEntry).GetDisplayIndex());
			}
		}
	};
	FString ReconstructFullClassPath(FArchive& BinaryArchive, const FString& PackageName, const FPackageFileSummary& PackageFileSummary, const FString& AssetClassName,
		const TArray<FObjectImport>* InImports = nullptr, const TArray<FObjectExport>* InExports = nullptr)
	{
		FName ClassFName(*AssetClassName);
		FLinkerTables LinkerTables;
		if (!InImports || !InExports)
		{
			FNameMapAwareArchive NameMapArchive(BinaryArchive);
			NameMapArchive.SerializeNameMap(PackageFileSummary);

			// Load the linker tables
			if (!InImports)
			{
				BinaryArchive.Seek(PackageFileSummary.ImportOffset);
				for (int32 ImportMapIndex = 0; ImportMapIndex < PackageFileSummary.ImportCount; ++ImportMapIndex)
				{
					NameMapArchive << LinkerTables.ImportMap.Emplace_GetRef();
				}
			}
			if (!InExports)
			{
				BinaryArchive.Seek(PackageFileSummary.ExportOffset);
				for (int32 ExportMapIndex = 0; ExportMapIndex < PackageFileSummary.ExportCount; ++ExportMapIndex)
				{
					NameMapArchive << LinkerTables.ExportMap.Emplace_GetRef();
				}
			}
		}
		if (InImports)
		{
			LinkerTables.ImportMap = *InImports;
		}
		if (InExports)
		{
			LinkerTables.ExportMap = *InExports;
		}

		FString ClassPathName;

		// Now look through the exports' classes and find the one matching the asset class
		for (const FObjectExport& Export : LinkerTables.ExportMap)
		{
			if (Export.ClassIndex.IsImport())
			{
				if (LinkerTables.ImportMap[Export.ClassIndex.ToImport()].ObjectName == ClassFName)
				{
					ClassPathName = LinkerTables.GetImportPathName(Export.ClassIndex.ToImport());
					break;
				}					
			}
			else if (Export.ClassIndex.IsExport())
			{
				if (LinkerTables.ExportMap[Export.ClassIndex.ToExport()].ObjectName == ClassFName)
				{
					ClassPathName = LinkerTables.GetExportPathName(PackageName, Export.ClassIndex.ToExport());
					break;
				}
			}
		}
		if (ClassPathName.IsEmpty())
		{
			UE_LOGF(LogAssetRegistry, Error, "Failed to find an import or export matching asset class short name \"%ls\".", *AssetClassName);
			// Just pass through the short class name
			ClassPathName = AssetClassName;
		}


		return ClassPathName;
	}

	bool FDeserializePackageData::DoSerialize(FArchive& BinaryArchive, const FPackageFileSummary& PackageFileSummary, EReadPackageDataMainErrorCode& OutError)
	{
		// To avoid large patch sizes, we have frozen cooked package format at the format before VER_UE4_ASSETREGISTRY_DEPENDENCYFLAGS
		const bool bPreDependencyFormat = PackageFileSummary.GetFileVersionUE() < VER_UE4_ASSETREGISTRY_DEPENDENCYFLAGS || !!(PackageFileSummary.GetPackageFlags() & PKG_FilterEditorOnly);

		// Load offsets to optionally-read data
		if (bPreDependencyFormat)
		{
			DependencyDataOffset = INDEX_NONE;
		}
		else
		{
			BinaryArchive << DependencyDataOffset;
		}

		// Load the object count
		ObjectCount = 0;
		BinaryArchive << ObjectCount;
		const int64 PackageFileSize = BinaryArchive.TotalSize();
		const int32 MinBytesPerObject = 1;
		if (BinaryArchive.IsError() || ObjectCount < 0 || PackageFileSize < BinaryArchive.Tell() + ObjectCount * MinBytesPerObject)
		{
			OutError = EReadPackageDataMainErrorCode::InvalidObjectCount;
			return false;
		}

		return true;
	}

	bool FDeserializeObjectPackageData::DoSerialize(FArchive& BinaryArchive, EReadPackageDataMainErrorCode& OutError)
	{
		const int32 MinBytesPerTag = 1;
		const int64 PackageFileSize = BinaryArchive.TotalSize();

		BinaryArchive << ObjectPath;
		BinaryArchive << ObjectClassName;
		// @todo make sure this is a full path name
		BinaryArchive << TagCount;
		if (BinaryArchive.IsError() || TagCount < 0 || PackageFileSize < BinaryArchive.Tell() + TagCount * MinBytesPerTag)
		{
			OutError = EReadPackageDataMainErrorCode::InvalidTagCount;
			return false;
		}		

		return true;
	}

	bool FDeserializeTagData::DoSerialize(FArchive& BinaryArchive, EReadPackageDataMainErrorCode& OutError)
	{
		BinaryArchive << Key;
		BinaryArchive << Value;
		if (BinaryArchive.IsError())
		{
			OutError = EReadPackageDataMainErrorCode::InvalidTag;
			return false;
		}

		return true;
	}

	// See the corresponding WritePackageData defined in SavePackageUtilities.cpp in CoreUObject module
	bool ReadPackageDataMain(FArchive& BinaryArchive, const FString& PackageName, const FPackageFileSummary& PackageFileSummary, int64& OutDependencyDataOffset,
		TArray<FAssetData*>& OutAssetDataList, EReadPackageDataMainErrorCode& OutError, const TArray<FObjectImport>* InImports, const TArray<FObjectExport>* InExports)
	{
		OutError = EReadPackageDataMainErrorCode::Unknown;

		const FString PackagePath = FPackageName::GetLongPackagePath(PackageName);
		const int64 PackageFileSize = BinaryArchive.TotalSize();
		const bool bIsMapPackage = (PackageFileSummary.GetPackageFlags() & PKG_ContainsMap) != 0;

		FDeserializePackageData DeserializePackageData;
		if (!DeserializePackageData.DoSerialize(BinaryArchive, PackageFileSummary, OutError))
		{
			return false;
		}

		OutDependencyDataOffset = DeserializePackageData.DependencyDataOffset;
		
		// support package relocation:
		UE::Package::Relocation::Private::FPackageRelocationContext RelocationArgs;
		const bool bIsRelocated = UE::Package::Relocation::Private::ShouldApplyRelocation(PackageFileSummary, PackageName, RelocationArgs);

		// Worlds that were saved before they were marked public do not have asset data so we will synthesize it here to make sure we see all legacy umaps
		// We will also do this for maps saved after they were marked public but no asset data was saved for some reason. A bug caused this to happen for some maps.
		if (bIsMapPackage)
		{
			const bool bLegacyPackage = PackageFileSummary.GetFileVersionUE() < VER_UE4_PUBLIC_WORLDS;
			const bool bNoMapAsset = (DeserializePackageData.ObjectCount == 0);
			if (bLegacyPackage || bNoMapAsset)
			{
				FString AssetName = FPackageName::GetLongPackageAssetName(PackageName);
				OutAssetDataList.Add(new FAssetData(FName(*PackageName), FName(*PackagePath), FName(*AssetName), FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("World")), FAssetDataTagMap(), PackageFileSummary.ChunkIDs, PackageFileSummary.GetPackageFlags()));
			}
		}

		// UAsset files usually only have one asset, maps and redirectors have multiple
		for (int32 ObjectIdx = 0; ObjectIdx < DeserializePackageData.ObjectCount; ++ObjectIdx)
		{
			FDeserializeObjectPackageData ObjectPackageData;
			if (!ObjectPackageData.DoSerialize(BinaryArchive, OutError))
			{
				return false;
			}

			FAssetDataTagMap TagsAndValues;
			TagsAndValues.Reserve(ObjectPackageData.TagCount);

			for (int32 TagIdx = 0; TagIdx < ObjectPackageData.TagCount; ++TagIdx)
			{
				FDeserializeTagData TagData;
				if (!TagData.DoSerialize(BinaryArchive, OutError))
				{
					return false;
				}

				if (!TagData.Key.IsEmpty() && !TagData.Value.IsEmpty())
				{
					TagsAndValues.Add(FName(*TagData.Key), TagData.Value);
				}
			}

			if(bIsRelocated)
			{
				UE::Private::ApplyRelocationToTagsAndValues(TagsAndValues, RelocationArgs);
			}

			// Before worlds were RF_Public, other non-public assets were added to the asset data table in map packages.
			// Here we simply skip over them
			if (bIsMapPackage && PackageFileSummary.GetFileVersionUE() < VER_UE4_PUBLIC_WORLDS)
			{
				if (ObjectPackageData.ObjectPath != FPackageName::GetLongPackageAssetName(PackageName))
				{
					continue;
				}
			}

			// if we have an object path that starts with the package then this asset is outer-ed to another package
			const bool bFullObjectPath = ObjectPackageData.ObjectPath.StartsWith(TEXT("/"), ESearchCase::CaseSensitive);

			// if we do not have a full object path already, build it
			if (!bFullObjectPath)
			{
				// if we do not have a full object path, ensure that we have a top level object for the package and not
				// a subobject. This warning can also fire if a top level object was created with the invalid character
				// '.' in its objectname. Savepackage is supposed to prevent that, but we do not enforce it yet.
				if (ObjectPackageData.ObjectPath.Contains(TEXT("."), ESearchCase::CaseSensitive))
				{
					UE_LOGF(LogAssetRegistry, Warning,
						"Package is loadable but its AssetRegistry data is corrupt: %ls. Reason: Cannot make FAssetData for sub object %ls.",
						*PackageName, *ObjectPackageData.ObjectPath);
					continue;
				}
				ObjectPackageData.ObjectPath = PackageName + TEXT(".") + ObjectPackageData.ObjectPath;
			}
			// Previously export couldn't have its outer as an import
			else if (PackageFileSummary.GetFileVersionUE() < VER_UE4_NON_OUTER_PACKAGE_IMPORT)
			{
				UE_LOGF(LogAssetRegistry, Warning,
					"Package is loadable but has invalid data; resave the package! Package: %ls. Reason: Export %ls is invalid.",
					*PackageName, *ObjectPackageData.ObjectPath);
				continue;
			}

			// Create a new FAssetData for this asset and update it with the gathered data
			if (!ObjectPackageData.ObjectClassName.IsEmpty() && FPackageName::IsShortPackageName(ObjectPackageData.ObjectClassName))
			{
				int64 CurrentPos = BinaryArchive.Tell();
				ObjectPackageData.ObjectClassName = ReconstructFullClassPath(BinaryArchive, PackageName, PackageFileSummary,
					ObjectPackageData.ObjectClassName, InImports, InExports);
				BinaryArchive.Seek(CurrentPos);
			}
			OutAssetDataList.Add(new FAssetData(PackageName, ObjectPackageData.ObjectPath, FTopLevelAssetPath(ObjectPackageData.ObjectClassName), MoveTemp(TagsAndValues), PackageFileSummary.ChunkIDs, PackageFileSummary.GetPackageFlags()));
		}

		return true;
	}

	// See the corresponding WriteAssetRegistryPackageData defined in SavePackageUtilities.cpp in CoreUObject module
	bool ReadPackageDataDependencies(FReadPackageDataDependenciesArgs& Args)
	{
		// Always set the output AssetRegistryVersion; in an error case it indicates to the caller whether the error
		// was caused by a too-high version.
		if (Args.AssetRegistryDependencyDataOffset == INDEX_NONE)
		{
			// For old package versions that did not write out the dependency flags, set default values of the flags
			Args.ImportUsedInGame.Init(true, Args.NumImports);
			Args.SoftPackageUsedInGame.Init(true, Args.NumSoftPackageReferences);
			Args.ExtraPackageDependencies.Empty();
			Args.AssetRegistryDependencyDataSize = 0;
			return true;
		}

		FArchive& Ar = *Args.BinaryNameAwareArchive;
		Ar.Seek(Args.AssetRegistryDependencyDataOffset);
		if (Ar.IsError())
		{
			return false;
		}

		Ar << Args.ImportUsedInGame;
		Ar << Args.SoftPackageUsedInGame;
		if (Args.PackageVersion >= EUnrealEngineObjectUE5Version::ASSETREGISTRY_PACKAGEBUILDDEPENDENCIES)
		{
			// Reinterpret ExtraPackageDependencies as an Array with integer values so we can serialize the 
			// EExtraDependencyFlags as integers.
			TArray<TPair<FName, uint32>>& ExtraPackageDependenciesAsIntegers = 
				reinterpret_cast<TArray<TPair<FName, uint32>>&>(Args.ExtraPackageDependencies);
			Ar << ExtraPackageDependenciesAsIntegers;
		}
		else
		{
			Args.ExtraPackageDependencies.Empty();
		}
		if (Args.ImportUsedInGame.Num() != Args.NumImports ||
			Args.SoftPackageUsedInGame.Num() != Args.NumSoftPackageReferences)
		{
			return false;
		}

		Args.AssetRegistryDependencyDataSize = Ar.Tell() - Args.AssetRegistryDependencyDataOffset;
		return !Ar.IsError();
	}
}

TOptional<FSoftObjectPath> FPackageReader::LinkerIndexToSoftObjectPath(FPackageIndex Index)
{
	if (Index.IsNull())
	{
		return FSoftObjectPath();
	}

	// Most paths will be 2-3 elements but there may be some deeply nested subobjects in certain asset types.
	TArray<FName, TInlineAllocator<8>> PathElements; 

	auto GetObjectResource = [this](FPackageIndex I) -> FObjectResource*
	{
		if (I.IsImport())
		{
			if (ImportMap.IsValidIndex(I.ToImport()))
			{
				return &ImportMap[I.ToImport()];
			}
		}
		else 
		{
			if (ExportMap.IsValidIndex(I.ToExport()))
			{
				return &ExportMap[I.ToExport()];
			}
		}
		UE_LOGFMT_LOC(LogAssetRegistry, Warning,"ConstructObjectPath_PackageIndexOutOfRange",
			"Package is unloadable: {FileName}. Reason: Bad object index {Index}. ",
			("FileName", PackageFilename), ("Index", I.ForDebugging()));
		return nullptr;
	};

	// Simultaneously check for loops with another pointer moving at twice the rate
	FPackageIndex P1 = Index;
	FPackageIndex P2 = Index;
	while (!P1.IsNull())
	{
		FObjectResource* Obj = GetObjectResource(P1);
		if (!Obj)
		{
			return {};
		}
		PathElements.Add(Obj->ObjectName);
		P1 = Obj->OuterIndex;
		if (P1.IsNull())
		{
			// If P1 terminates we know there is no loop and we have all path elements that we want.
			break;
		}

		// Advance P2 by two steps, if it terminates we know there is no loop so we can stop advancing it
		for (int32 i=0; i <2; ++i)
		{
			if (!P2.IsNull())
			{
				FObjectResource* Obj2 = GetObjectResource(P2);
				if (!ensureMsgf(Obj2, TEXT("Unexpected null object resource traversing object path")))
				{
					return {};
				}
				P2 = Obj2->OuterIndex;
			}
		}

		// If P1 did not terminate and P2 is at the same place as P1, we have a loop
		if (P1 == P2)
		{
			UE_LOGFMT_LOC(LogAssetRegistry, Warning,"PackageIndexOuterLoop",
				"Package is unloadable: {FileName}. Reason: Loop detected in outer chain of object index {Index}. ",
				("FileName", PackageFilename), ("Index", Index.ForDebugging()));
			return {};
		}
	}

	// PathElements was populated from leaf to root, reverse it before handling any of these cases for clarity
	Algo::Reverse(PathElements);
	if (PathElements.Num() == 2)
	{
		return FSoftObjectPath(FTopLevelAssetPath(PathElements[0], PathElements[1]));
	}
	else if (PathElements.Num() == 1)
	{
		return FSoftObjectPath(FTopLevelAssetPath(PathElements[0], FName()));
	}
	else 
	{
		// Technically some delimiters ought to be : but we can't do that without using type information
		TStringBuilder<512> Builder;
		Builder.Join(PathElements, '.'); 
		return FSoftObjectPath(Builder.ToView());
	}
}

#undef LOCTEXT_NAMESPACE
