// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/PathTree.h"

#include "Misc/PathViews.h"
#include "Misc/ReverseIterate.h"
#include "UObject/NameTypes.h"

FPathTree::FPathTree()
{
	// Ensure an entry for the root of the path
	static const FName PathRoot = "/";

	ParentPathToChildPaths.FindOrAdd(PathRoot, {});
}

void FPathTree::EnsureAdditionalCapacity(int32 NumNewPaths)
{
	ParentPathToChildPaths.Reserve(ParentPathToChildPaths.Num() + NumNewPaths); 
	ChildPathToParentPath.Reserve(ChildPathToParentPath.Num() + NumNewPaths);
}

bool FPathTree::CachePath(FName InPath, TFunctionRef<void(FName)> OnPathAdded)
{
	if (InPath.IsNone())
	{
		return false;
	}

	if (ParentPathToChildPaths.Contains(InPath))
	{
		// Already cached - nothing more to do
		return false;
	}

	TStringBuilder<FName::StringBufferSize> PathBuffer(InPlace, InPath);
	FStringView PathView = PathBuffer.ToView();
	check(PathView.Len() >= 2);	// Must be at least "/A"
	check(PathView [0] == '/');	// Must start with a "/"
	check(PathView[PathView.Len() - 1] != '/'); // Must not contain trailing slash 

	static const FName Root("/");

	TArray<FName, TInlineAllocator<16>> NewPaths;
	NewPaths.Add(InPath);
	ParentPathToChildPaths.FindOrAdd(InPath, {}); // Add this new path with no children
	FName LastPath = InPath;

	// Walk backwards through the string until we encounter a path we've already created
	FPathViews::IterateAncestors(PathView, [this, &NewPaths, &LastPath, PathView](FStringView InAncestor) {
		if (InAncestor == PathView)
		{
			// Initial input path was already handled
			return true;
		}

		FName ParentPath = InAncestor.IsEmpty() ? Root : FName(InAncestor);
		ChildPathToParentPath.FindOrAdd(LastPath, ParentPath);

		uint32 Hash = GetTypeHash(ParentPath);
		TSet<FName>* Children = ParentPathToChildPaths.FindByHash(Hash, ParentPath);
		if (Children)
		{
			// Parent path already existed in tree, no need to continue looking at parents
			Children->Add(LastPath);
			return false;
		}
		else
		{
			ParentPathToChildPaths.AddByHash(Hash, ParentPath).Add(LastPath);
			NewPaths.Add(ParentPath);
			LastPath = ParentPath;
			return true;
		}
	});

	// Notify caller of each path created in order from root to leaf
	for (FName NewPath : ReverseIterate(NewPaths))
	{
		OnPathAdded(NewPath);
	}

	return true;
}

bool FPathTree::RemovePath(FName Path, TFunctionRef<void(FName)> OnPathRemoved)
{
	if (Path.IsNone())
	{
		return false;
	}

	TSet<FName>* Children = ParentPathToChildPaths.Find(Path);
	if (!Children)
	{
		TStringBuilder<FName::StringBufferSize> PathString(InPlace, Path);
		checkf(!PathString.ToView().EndsWith(TEXT("/")), TEXT("Path tree arguments should not have trailing slashes: %s"), *PathString);
		return false;
	}

	// We also need to gather up and remove any children of this path
	TSet<FName> SubPathsToRemove;
	GetSubPaths(Path, SubPathsToRemove, /*bRecurse=*/true);

	// Sort the sub-paths by length, longest -> shortest, so that children are notified before their parents
	SubPathsToRemove.Sort([](FName SubPathOne, FName SubPathTwo)
	{
		return SubPathOne.Compare(SubPathTwo) > 0;
	});

	// Simply remove sub-paths from both maps
	for (const FName& SubPathToRemove : SubPathsToRemove)
	{
		ParentPathToChildPaths.Remove(SubPathToRemove);
		ChildPathToParentPath.Remove(SubPathToRemove);
		OnPathRemoved(SubPathToRemove);
	}

	// We also need to remove ourself from our parent list before removing ourself from the maps
	{
		const FName* ParentPathPtr = ChildPathToParentPath.Find(Path);
		if (ParentPathPtr)
		{
			TSet<FName>* ChildPaths = ParentPathToChildPaths.Find(*ParentPathPtr);
			ChildPaths->Remove(Path);
		}
	}

	ParentPathToChildPaths.Remove(Path);
	ChildPathToParentPath.Remove(Path);
	OnPathRemoved(Path);

	return true;
}

bool FPathTree::PathExists(FName Path) const
{
	if (Path.IsNone())
	{
		return false;
	}

	const TSet<FName>* ChildPathsPtr = ParentPathToChildPaths.Find(Path);
	if (!ChildPathsPtr)
	{
		// Paths are cached without their trailing slash, so if the given path has a trailing slash, test it again now as it may already be cached
		// We do this after the initial map test as: a) Most paths are well formed, b) This avoids string allocations until we know we need them
		FString PathStr = Path.ToString();
		if (PathStr[PathStr.Len() - 1] == '/')
		{
			PathStr.RemoveAt(PathStr.Len() - 1, EAllowShrinking::No);
			Path = *PathStr;

			ChildPathsPtr = ParentPathToChildPaths.Find(Path);
		}
	}

	return ChildPathsPtr != nullptr;
}

bool FPathTree::GetAllPaths(TSet<FName>& OutPaths) const
{
	OutPaths.Reset();
	EnumerateAllPaths([&OutPaths](FName Path)
	{
		OutPaths.Emplace(Path);
		return true;
	});
	return OutPaths.Num() > 0;
}

void FPathTree::EnumerateAllPaths(TFunctionRef<bool(FName)> Callback) const
{
	for (const auto& PathPair : ParentPathToChildPaths)
	{
		if (!Callback(PathPair.Key))
		{
			return;
		}
	}
}

bool FPathTree::GetSubPaths(FName BasePath, TSet<FName>& OutPaths, bool bRecurse) const
{
	const int32 OutPathsOriginalNum = OutPaths.Num();
	return EnumerateSubPaths(BasePath, [&OutPaths](FName Path)
	{
		OutPaths.Emplace(Path);
		return true;
	}, bRecurse) && OutPaths.Num() > OutPathsOriginalNum;
}

FName FPathTree::NormalizePackagePath(FName In)
{
	FNameBuilder InStr(In);
	if (InStr.Len() == 0 || InStr.LastChar() != TEXT('/'))
	{
		return In;
	}
	InStr.RemoveSuffix(1);
	return FName(InStr);
}

bool FPathTree::EnumerateSubPaths(FName BasePath, TFunctionRef<bool(FName)> Callback, bool bRecurse) const
{
	if (BasePath.IsNone())
	{
		return false;
	}

	const TSet<FName>* ChildPathsPtr = ParentPathToChildPaths.Find(BasePath);
	if (!ChildPathsPtr)
	{
		// Paths are cached without their trailing slash, so if the given path has a trailing slash, test it again now as it may already be cached
		// We do this after the initial map test as: a) Most paths are well formed, b) This avoids string tests until we know it's possibly needed
		FName NormalizedBasePath = NormalizePackagePath(BasePath);
		if (NormalizedBasePath != BasePath)
		{
			ChildPathsPtr = ParentPathToChildPaths.Find(NormalizedBasePath);
			if (!ChildPathsPtr)
			{
				return false;
			}
		}
		else
		{
			return false;
		}
	}

	for (const FName& ChildPath : *ChildPathsPtr)
	{
		checkf(ParentPathToChildPaths.Find(ChildPath) != nullptr, TEXT("PathTree integrity failure, expected to contain %s"), *WriteToString<FName::StringBufferSize>(ChildPath)); // This failing is an integrity violation as this entry lists a child that we don't know about

		if (!Callback(ChildPath))
		{
			return true;
		}

		if (bRecurse)
		{
			EnumerateSubPaths(ChildPath, Callback, /*bRecurse=*/true);
		}
	}

	return true;
}

void FPathTree::Shrink()
{
	ParentPathToChildPaths.Shrink();
	ChildPathToParentPath.Shrink();
}
