// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/IsSorted.h"
#include "Containers/ArrayView.h"
#include "Containers/BitArray.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"

/**
 * TPropertyCombinationSet is a set of integers where the integers are bit fields for whether each of n independent properties are present.
 * We refer to each element in the set as a corner; this term is derived from the model of a BitWidth-dimensional hypercube; each corner of the hypercube maps to one of the possible combinations of the bit field.
 * TPropertyCombinationSet has another trait: redundant child corners are culled from the set. 
 * Corner B is a redundant child of Corner A if every property present in B is also present in A.
 * 
 * The motivations for having a set of property bit fields, and culling the redundant child corners:
 *   1) We need a set of property bit fields when a combination of properties in a single element is important to distinguish from multiple elements with those properties in isolation
 *      e.g. it is important to know whether we have (A) [(a not-used-in-game hard dependency) AND (a used-in-game soft dependency)] vs (B) [(a used-in-game hard dependency)]
 *   2) We need to cull redundant child corners for performance, if we are allowed to.
 *      Minimizing the number of property combinations necessary to iterate over is important for performance of any calling iteration code
 *      We are allowed to eliminate redundant child corners when each property is an increasing superset - the behavior when the property is present is a superset of the behavior when the property is not present
 *      e.g. When we have a hard dependency, that means everything that having a soft dependency means, PLUS some other behavior that is specific to the hard-ness of the dependency
 *
 * Note that adding elements (aka bitfields aka corners) to a TPropertyCombinationSet is not reversible; we lose information about what the original corners were when we remove redundant corners after the add
 * 
 * For the generic template implementation of TPropertyCombinationSet, it remains an unsolved problem for how to enumerate all possible sets non-redundant corners in this general case
 * We therefore fall back to a less-optimal case where each corner gets assigned to a bit, and dynamically check for redundant corners whenever adding to the set
 * More optimal (fewer bits required) instantiations for specific bitwidths with manual solutions are defined below the generic implementation.
 */
template<int BitWidth>
class TPropertyCombinationSet
{
public:	
	static constexpr uint32 StorageBitCount = 1 << BitWidth;
	static constexpr uint32 StorageWordCount = (StorageBitCount + NumBitsPerDWORD - 1) / NumBitsPerDWORD;
	static constexpr uint32 MaxValue = StorageBitCount - 1;

public:
	TPropertyCombinationSet()
	{
		Construct();
		FMemory::Memset(Storage, 0);
		AddNoCheck(0); // Empty PropertyCombinationSets include 0
	}

	TPropertyCombinationSet(const TPropertyCombinationSet<BitWidth>& Other)
	{
		Construct();
		FMemory::Memcpy(Storage, Other.Storage, sizeof(Storage));
	}

	TPropertyCombinationSet(const TBitArray<>& ArchivedBits, uint32 BitOffset = 0)
	{
		Construct();
		Load(ArchivedBits, BitOffset);
	}

	void Load(const TBitArray<>& ArchiveBits, uint32 BitOffset)
	{
		ArchiveBits.GetRange(BitOffset, StorageBitCount, Storage);
	}

	void Save(TBitArray<>& ArchiveBits, uint32 BitOffset) const
	{
		ArchiveBits.SetRangeFromRange(BitOffset, StorageBitCount, Storage);
	}

	void Load(const uint32* ArchiveBits)
	{
		FMemory::Memcpy(Storage, ArchiveBits, sizeof(Storage));
	}

	void Save(uint32* ArchiveBits) const
	{
		FMemory::Memcpy(ArchiveBits, Storage, sizeof(Storage));
	}

	/**
	 * Given a prospective PropertyCombination, add it to the PropertyCombinationSet if it is not a Redundant Combination of a PropertyCombination already in the Set. Once added, remove any now-redundant PropertyCombinations.
	 */
	void Add(uint32 PropertyCombination)
	{
		check(PropertyCombination <= MaxValue);
		if (IsRedundantPropertyCombinationNoCheck(PropertyCombination))
		{
			return;
		}
		AddNoCheck(PropertyCombination);
		RemoveRedundantPropertyCombinationsNoCheck(PropertyCombination);
	}

	void AddRange(TPropertyCombinationSet<BitWidth>& Other)
	{
		for (uint32 Value : Other)
		{
			Add(Value);
		}
	}

	bool Contains(uint32 Value) const
	{
		check(Value <= MaxValue);
		return ContainsNoCheck(Value);
	}

	bool operator==(const TPropertyCombinationSet<BitWidth>& Other) const
	{
		// Note that all bits after our ending bit are memset to 0, so it's okay to compare words
		for (int n = 0; n < StorageWordCount; ++n)
		{
			if (Storage[n] != Other.Storage[n])
			{
				return false;
			}
		}
		return true;
	}

	struct FIterator
	{
	public:
		FIterator& operator++()
		{
			++Value;
			uint32 Max = StorageBitCount;
			while (Value < Max && !Array.ContainsNoCheck(Value))
			{
				++Value;
			}
			return *this;
		}
		uint32 operator*() { return Value; }
		bool operator!=(const FIterator& Other) const { return Value != Other.Value; }
	private:
		friend class TPropertyCombinationSet;

		FIterator(TPropertyCombinationSet& InArray, uint32 OneBeforeStart = (uint32)-1)
			: Array(InArray)
			, Value(OneBeforeStart)
		{
			++(*this);
		}

		TPropertyCombinationSet& Array;
		uint32 Value;
	};

	FIterator begin()
	{
		return FIterator(*this);
	}
	FIterator end()
	{
		return FIterator(*this, MaxValue); // Note that MaxValue is a valid value, and the FIterator constructor adds one to it to get the first invalid value
	}

private:
	void Construct()
	{
		// TPropertyCombinationSet has exponential storage requirements with the number of bits, and having n >= 32 is not feasible.
		// We take advantage of this to assume that a bit combination can fit in uint32 and that 1 << BitWidth fits in a uint32
		static_assert(BitWidth < sizeof(uint32) * 8, "TPropertyCombinationSet cannot be used with BitWidths >= 32.");
	}

	void AddNoCheck(uint32 Value)
	{
		Storage[Value / NumBitsPerDWORD] |= (1 << (Value & (NumBitsPerDWORD - 1)));
	}

	void RemoveNoCheck(uint32 Value)
	{
		Storage[Value / NumBitsPerDWORD] &= ~((1 << (Value & (NumBitsPerDWORD - 1))));
	}

	bool ContainsNoCheck(uint32 Value) const
	{
		return (Storage[Value / NumBitsPerDWORD] & (1 << (Value & (NumBitsPerDWORD - 1)))) != 0;
	}

	bool IsRedundantPropertyCombinationNoCheck(uint32 PropertyCombination) const
	{
		return IsRedundantPropertyCombinationRecursive(PropertyCombination, 0x1);
	}

	bool IsRedundantPropertyCombinationRecursive(uint32 PropertyCombination, uint32 StartBit) const
	{
		if (ContainsNoCheck(PropertyCombination))
		{
			return true;
		}

		for (uint32 Bit = StartBit; Bit < StorageBitCount; Bit <<= 1)
		{
			if (!(PropertyCombination & Bit))
			{
				if (IsRedundantPropertyCombinationRecursive(PropertyCombination | Bit, StartBit << 1))
				{
					return true;
				}
			}
		}
		return false;
	}

	void RemoveRedundantPropertyCombinationsNoCheck(uint32 PropertyCombination)
	{
		RemoveRedundantPropertyCombinationsRecursive(PropertyCombination, 0x1);
	}

	void RemoveRedundantPropertyCombinationsRecursive(uint32 PropertyCombination, uint32 StartBit)
	{
		for (uint32 Bit = StartBit; Bit < StorageBitCount; Bit <<= 1)
		{
			if (PropertyCombination & Bit)
			{
				uint32 CombinationToRemove = PropertyCombination & ~Bit;
				RemoveNoCheck(CombinationToRemove);
				RemoveRedundantPropertyCombinationsRecursive(CombinationToRemove, Bit << 1);
			}
		}
	}

private:
	uint32 Storage[StorageWordCount];
};

/**
 * For a TPropertyCombinationSet over bitfields with only a single bit, the possible sets of non-redundant corners are:
 * [0] - the bit is not set
 * [1] - the bit is set
 */
template<>
class TPropertyCombinationSet<1>
{
public:
	static constexpr uint32 BitWidth = 1;
	static constexpr uint32 StorageBitCount = 1;
	static constexpr uint32 StorageWordCount = 1;
	static constexpr uint32 MaxValue = 1;
	static constexpr uint32 NumPackedValues = 2;

public:
	TPropertyCombinationSet()
	{
		Storage = 0;
	}

	TPropertyCombinationSet(const TPropertyCombinationSet<1>& Other)
	{
		Storage = Other.Storage;
	}

	TPropertyCombinationSet(const TBitArray<>& ArchivedBits, uint32 BitOffset = 0)
	{
		Load(ArchivedBits, BitOffset);
	}

	void Load(const TBitArray<>& ArchiveBits, uint32 BitOffset)
	{
		ArchiveBits.GetRange(BitOffset, StorageBitCount, &Storage);
	}

	void Save(TBitArray<>& ArchiveBits, uint32 BitOffset) const
	{
		ArchiveBits.SetRangeFromRange(BitOffset, StorageBitCount, &Storage);
	}

	void Load(const uint32* ArchiveBits)
	{
		Storage = ArchiveBits[0];
	}

	void Save(uint32* ArchiveBits) const
	{
		ArchiveBits[0] = Storage;
	}

	void Add(uint32 PropertyCombination)
	{
		check(PropertyCombination <= MaxValue);
		Storage = Storage | (PropertyCombination != 0);
	}

	void AddRange(TPropertyCombinationSet<1>& Other)
	{
		Storage = Storage | Other.Storage;
	}

	bool Contains(uint32 Value) const
	{
		check(Value <= MaxValue);
		return Value == Storage;
	}

	bool operator==(const TPropertyCombinationSet<1>& Other) const
	{
		return Storage == Other.Storage;
	}

	struct FIterator
	{
	public:
		FIterator& operator++()
		{
			++Index;
			return *this;
		}
		uint32 operator*() { return Array.Storage; }
		bool operator!=(const FIterator& Other) const { return Index != Other.Index; }
	private:
		friend class TPropertyCombinationSet;

		FIterator(TPropertyCombinationSet& InArray, int32 InIndex)
			: Array(InArray)
			, Index(InIndex)
		{
		}

		TPropertyCombinationSet& Array;
		int32 Index;
	};

	FIterator begin()
	{
		return FIterator(*this, 0);
	}
	FIterator end()
	{
		return FIterator(*this, 1);
	}

private:
	friend class FPropertyCombinationSetTest;

	uint32 Storage;
};


template<typename PackerClass>
class TPropertyCombinationSetHardcoded
{
public:
	static constexpr uint32 BitWidth = PackerClass::BitWidth;
	static constexpr uint32 StorageBitCount = PackerClass::StorageBitCount;
	static constexpr uint32 StorageWordCount = 1;
	static constexpr uint32 MaxValue = (1 << BitWidth) - 1;
	static constexpr uint32 ArrayMax = PackerClass::ArrayMax; // This value is analytically computable - n choose floor(n/2) - but we manually hardcode it rather than calculating it in the compiler
	static constexpr uint32 NumPackedValues = PackerClass::NumPackedValues;

	TPropertyCombinationSetHardcoded()
	{
		Storage = 0;
	}

	TPropertyCombinationSetHardcoded(const TPropertyCombinationSetHardcoded<PackerClass>& Other)
	{
		Storage = Other.Storage;
	}

	TPropertyCombinationSetHardcoded(const TBitArray<>& ArchivedBits, uint32 BitOffset = 0)
	{
		Storage = 0;
		Load(ArchivedBits, BitOffset);
	}

	void Load(const TBitArray<>& ArchiveBits, uint32 BitOffset)
	{
		ArchiveBits.GetRange(BitOffset, StorageBitCount, &Storage);
	}

	void Save(TBitArray<>& ArchiveBits, uint32 BitOffset) const
	{
		ArchiveBits.SetRangeFromRange(BitOffset, StorageBitCount, &Storage);
	}

	void Load(const uint32* ArchiveBits)
	{
		Storage = ArchiveBits[0];
	}

	void Save(uint32* ArchiveBits) const
	{
		ArchiveBits[0] = Storage;
	}

	void Add(uint32 PropertyCombination)
	{
		check(PropertyCombination <= MaxValue);
		const uint32* OldValues;
		int OldNum;
		PackerClass::Unpack(Storage, OldValues, OldNum);

		uint32 NewValues[ArrayMax];
		int NewNum;
		if (AddNonRedundant(OldValues, OldNum, PropertyCombination, NewValues, NewNum))
		{
			Storage = PackerClass::Pack(NewValues, NewNum);
		}
	}

	void AddRange(TPropertyCombinationSetHardcoded<PackerClass>& Other)
	{
		const uint32* ExistingValues;
		int ExistingNum;
		PackerClass::Unpack(Storage, ExistingValues, ExistingNum);

		const uint32* AddingValues;
		int AddingNum;
		PackerClass::Unpack(Other.Storage, AddingValues, AddingNum);

		uint32 BufferValues1[ArrayMax];
		uint32 BufferValues2[ArrayMax];
		const uint32* OldValues = ExistingValues;
		int OldNum = ExistingNum;
		uint32* NewValues = BufferValues1;
		int NewNum;
		for (uint32 AddingValue : TArrayView<const uint32>(AddingValues, AddingNum))
		{
			if (AddNonRedundant(OldValues, OldNum, AddingValue, NewValues, NewNum))
			{
				OldValues = NewValues;
				OldNum = NewNum;
				NewValues = OldValues == BufferValues1 ? BufferValues2 : BufferValues1;
			}
		}
		Storage = PackerClass::Pack(OldValues, OldNum);
	}

	bool Contains(uint32 Value) const
	{
		check(Value <= MaxValue);
		const uint32* Values;
		int Num;
		PackerClass::Unpack(Storage, Values, Num);
		for (uint32 Existing : TArrayView<const uint32>(Values, Num))
		{
			if (Existing == Value)
			{
				return true;
			}
		}
		return false;
	}

	bool operator==(const TPropertyCombinationSetHardcoded<PackerClass>& Other) const
	{
		return Storage == Other.Storage;
	}

	struct FIterator
	{
	public:
		FIterator& operator++()
		{
			++Index;
			return *this;
		}
		uint32 operator*()
		{
			return Values[Index];
		}
		bool operator!=(const FIterator& Other) const { return Index != Other.Index; }
	private:
		friend class TPropertyCombinationSetHardcoded;

		FIterator() = default;

		const uint32* Values;
		int Num;
		int32 Index;
	};

	FIterator begin()
	{
		FIterator It;
		PackerClass::Unpack(Storage, It.Values, It.Num);
		It.Index = 0;
		return It;
	}
	FIterator end()
	{
		FIterator It;
		PackerClass::Unpack(Storage, It.Values, It.Num);
		It.Index = It.Num;
		return It;
	}

private:
	friend class FPropertyCombinationSetTest;

	static bool AddNonRedundant(const uint32* OldValues, const int OldNum, const uint32 AddingValue, uint32* NewValues, int& NewNum)
	{
		bool bAdded = false;
		NewNum = 0;
		for (const uint32 OldValue : TArrayView<const uint32>(OldValues, OldNum))
		{
			const uint32 Overlap = OldValue & AddingValue;
			if (Overlap == AddingValue)
			{
				// Adding value is redundant
				return false;
			}
			if (OldValue > AddingValue && !bAdded)
			{
				check(NewNum < ArrayMax);
				NewValues[NewNum++] = AddingValue;
				bAdded = true;
			}
			if (Overlap != OldValue)
			{
				check(NewNum < ArrayMax);
				NewValues[NewNum++] = OldValue;
			}
		}
		if (!bAdded)
		{
			check(NewNum < ArrayMax);
			NewValues[NewNum++] = AddingValue;
		}
		check(1 <= NewNum && NewNum <= ArrayMax);
		check(Algo::IsSorted(TArrayView<uint32>(NewValues, NewNum)));
		return true;
	}

	uint32 Storage;
};


/**
 * For a TPropertyCombinationSet over bitfields with two bits, there are 5 possible sets of non-redundant corners:
 * { [00], [01], [10], [11], [01, 10] }
 */
class FPropertyCombinationPack2
{
public:
	static constexpr uint32 BitWidth = 2;
	static constexpr uint32 StorageBitCount = 3;
	static constexpr uint32 ArrayMax = 2;
	static constexpr uint32 MaxValue = (1 << BitWidth) - 1;
	static constexpr uint32 NumPackedValues = 5;

	static void Unpack(const uint32 Compressed, const uint32*& OutValues, int& OutNum)
	{
		if (Compressed <= MaxValue)
		{
			static uint32 Values[] = { 0,1,2,3 };
			OutValues = &Values[Compressed];
			OutNum = 1;
		}
		else
		{
			check(Compressed == 4);
			static uint32 Values[] = { 1,2 };
			OutValues = Values;
			OutNum = 2;
		}
	}

	static uint32 Pack(const uint32* Values, const int Num)
	{
		if (Num == 1)
		{
			check(Values[0] <= MaxValue);
			return Values[0];
		}
		else
		{
			check(Num == 2);
			check(Values[0] == 1 && Values[1] == 2);
			return 4;
		}
	}
};
template<>
class TPropertyCombinationSet<2> : public TPropertyCombinationSetHardcoded<FPropertyCombinationPack2>
{
public:
	using TPropertyCombinationSetHardcoded<FPropertyCombinationPack2>::TPropertyCombinationSetHardcoded;
};


/**
 * For a TPropertyCombinationSet over bitfields with three bits, there are 19 possible sets of non-redundant corners:
 * 8 One Corner Lists:   { [000],         [001],         [010],     [011],     [100],     [101],     [110],     [111],
 *   Proof there are no more One Corner Lists: this set already contains every possible one-element list, for all legal values from 0 to 2 ^ BitWidth - 1
 * 9 Two Corner Lists:     [001,010],     [001,100],     [001,110], [010,100], [010,101], [011,100], [011,101], [011,110], [101,110],
 *   Proof there are no more Two Corner Lists:
 *		   Each 0-property-set corner (000) is a redundant child of all other corners, 0 two corner lists contain it.
 *         Each 1-property-set corner is a redundant child of all other corners on its high-property face. On the low-property face, 000 is a redundant child, and the other 3 corners on the face are nonreundant. This adds 3x3 == 9 corners
 *         Each 2-property-set corner is a redundant child of 111. It is further a parent of the two redundant children formed by setting one of its lowbits to 0, plus 000. There are 3 remaining corners it is non-redundant with. This adds 3x3 == 9 corners.
 *         Each 3-property-set corner (111) is a redundant parent of all other corners, 0 two corner lists contain it.
 *         Divide by 2 since each of our elements above is double-counted for x,y and y,x
 * 2 Three Corner Lists:   [001,010,100], [011,101,110] }
 *   Proof there are no more Three Corner Lists:
 *         A 1-property-set corner can not be non-redundant with two 2-property-set corners; one of the two must necessarily include the 1-property-set as a high bit.
 *         2 1-property-set corners can not be non-redundant with a 2-property-set corner; one of the two must necessarily be one of the 2-property-set's high bits.
 *         0-property-set corner 000 and 3-property-set corner 111 can not be non-redundant in any lists > 1 element.
 *         There are (3 choose 1) == 3 1-property-set corners and (3 choose 2) == 3 2-property-set corners, so only one list of 3 corners for each of those.
 * Proof there are no Four+ Corner Lists:
 *         We would have to add another corner to one of the Three Corner Lists, and by the proof for three corner lists, we're out of corners that could be added non-redundantly.
 */
class FPropertyCombinationPack3
{
public:
	static constexpr uint32 BitWidth = 3;
	static constexpr uint32 StorageBitCount = 5;
	static constexpr uint32 ArrayMax = 3;
	static constexpr uint32 MaxValue = (1 << BitWidth) - 1;
	static constexpr uint32 NumPackedValues = 19;

	static void Unpack(const uint32 Compressed, const uint32*& OutValues, int& OutNum)
	{
		/*0-7: [000] ->[111]
		* 8 : [001, 010]
		* 9 : [001, 100]
		* 10 : [001, 110]
		* 11 : [010, 100]
		* 12 : [010, 101]
		* 13 : [011, 100]
		* 14 : [011, 101]
		* 15 : [011, 110]
		* 16 : [101, 110]
		* 17 : [001, 010, 100]
		* 18 : [011, 101, 110] */
		if (Compressed < 8)
		{
			static uint32 Values[] = { 0,1,2,3,4,5,6,7 };
			OutValues = &Values[Compressed];
			OutNum = 1;
		}
		else
		{
			switch (Compressed)
			{
			case 8:
			{
				static uint32 Values[] = { 1,2 };
				OutValues = Values;
				OutNum = 2;
				break;
			}
			case 9:
			{
				static uint32 Values[] = { 1,4 };
				OutValues = Values;
				OutNum = 2;
				break;
			}
			case 10:
			{
				static uint32 Values[] = { 1,6 };
				OutValues = Values;
				OutNum = 2;
				break;
			}
			case 11:
			{
				static uint32 Values[] = { 2,4 };
				OutValues = Values;
				OutNum = 2;
				break;
			}
			case 12:
			{
				static uint32 Values[] = { 2,5 };
				OutValues = Values;
				OutNum = 2;
				break;
			}
			case 13:
			{
				static uint32 Values[] = { 3,4 };
				OutValues = Values;
				OutNum = 2;
				break;
			}
			case 14:
			{
				static uint32 Values[] = { 3,5 };
				OutValues = Values;
				OutNum = 2;
				break;
			}
			case 15:
			{
				static uint32 Values[] = { 3,6 };
				OutValues = Values;
				OutNum = 2;
				break;
			}
			case 16:
			{
				static uint32 Values[] = { 5,6 };
				OutValues = Values;
				OutNum = 2;
				break;
			}
			case 17:
			{
				static uint32 Values[] = { 1,2,4 };
				OutValues = Values;
				OutNum = 3;
				break;
			}
			case 18:
			{
				static uint32 Values[] = { 3,5,6 };
				OutValues = Values;
				OutNum = 3;
				break;
			}
			default:
				check(false);
				break;
			}
		}
	}

	static uint32 Pack(const uint32* Values, const int Num)
	{
		if (Num == 1)
		{
			check(Values[0] <= MaxValue);
			return Values[0];
		}
		else if (Num == 2)
		{
			switch (Values[0])
			{
			case 1:
				check(Values[1] == 2 || Values[1] == 4 || Values[1] == 6);
				return 8 + (Values[1] - 2) / 2;
			case 2:
				check(Values[1] == 4 || Values[1] == 5);
				return 11 + Values[1] - 4;
			case 3:
				check(Values[1] == 4 || Values[1] == 5 || Values[1] == 6);
				return 13 + Values[1] - 4;
			default:
				check(Values[0] == 5 && Values[1] == 6);
				return 16;
			}
		}
		else
		{
			check(Num == 3);
			if (Values[0] == 1)
			{
				check(Values[1] == 2 && Values[2] == 4);
				return 17;
			}
			else
			{
				check(Values[0] == 3 && Values[1] == 5 && Values[2] == 6);
				return 18;
			}
		}
	}
};

template<>
class TPropertyCombinationSet<3> : public TPropertyCombinationSetHardcoded<FPropertyCombinationPack3>
{
public:
	using TPropertyCombinationSetHardcoded<FPropertyCombinationPack3>::TPropertyCombinationSetHardcoded;
};
