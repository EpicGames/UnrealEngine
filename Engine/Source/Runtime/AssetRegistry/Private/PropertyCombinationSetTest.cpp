// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "PropertyCombinationSet.h"

class FPropertyCombinationSetTest : public FAutomationTestBase
{
public:
	FPropertyCombinationSetTest(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask)
	{
	}

protected:

	template <uint32 BitWidth>
	void TestHardcodedSet()
	{
		// Constructor
		{
			TPropertyCombinationSet<BitWidth> Set;
			TestTrue(TEXT("Hardcoded Constructor stores value 0"), Set.Contains(0));
			for (int n = 1; n <= TPropertyCombinationSet<BitWidth>::MaxValue; ++n)
			{
				TestFalse(TEXT("Hardcoded Constructor is empty other than 0"), Set.Contains(n));
			}

			for (uint32 PackedValue = 0; PackedValue < TPropertyCombinationSet<BitWidth>::NumPackedValues; ++PackedValue)
			{
				TPropertyCombinationSet<BitWidth> HackConstructedSet;
				HackConstructedSet.Storage = PackedValue;
				TPropertyCombinationSet<BitWidth> CopyConstructed(HackConstructedSet);
				for (int n = 0; n <= TPropertyCombinationSet<BitWidth>::MaxValue; ++n)
				{
					TestEqual(TEXT("Hardcoded CopyConstructor matches"), CopyConstructed.Contains(n), HackConstructedSet.Contains(n));
				}
			}
		}

		// Save/Load BitArray
		{
			for (uint32 StartOffset = 0; StartOffset < NumBitsPerDWORD; ++StartOffset)
			{
				for (uint32 PackedValue = 0; PackedValue < TPropertyCombinationSet<BitWidth>::NumPackedValues; ++PackedValue)
				{
					TPropertyCombinationSet<BitWidth> HackConstructedSet;
					HackConstructedSet.Storage = PackedValue;
					TBitArray<> Archive;
					Archive.Add(true, StartOffset + TPropertyCombinationSet<BitWidth>::StorageBitCount);
					HackConstructedSet.Save(Archive, StartOffset);
					TPropertyCombinationSet<BitWidth> SerializedSet;
					SerializedSet.Load(Archive, StartOffset);
					for (int n = 0; n <= TPropertyCombinationSet<BitWidth>::MaxValue; ++n)
					{
						TestEqual(TEXT("Hardcoded Load from archive bits"), SerializedSet.Contains(n), HackConstructedSet.Contains(n));
					}
				}
			}
		}


		// Save/Load uint32*
		{
			for (uint32 PackedValue = 0; PackedValue < TPropertyCombinationSet<BitWidth>::NumPackedValues; ++PackedValue)
			{
				TPropertyCombinationSet<BitWidth> HackConstructedSet;
				HackConstructedSet.Storage = PackedValue;
				uint32 ArchiveBits[1];
				HackConstructedSet.Save(ArchiveBits);
				TPropertyCombinationSet<BitWidth> SerializedSet;
				SerializedSet.Load(ArchiveBits);
				for (int n = 0; n <= TPropertyCombinationSet<BitWidth>::MaxValue; ++n)
				{
					TestEqual(TEXT("Hardcoded Load from archive bits"), SerializedSet.Contains(n), HackConstructedSet.Contains(n));
				}
			}
		}

		auto IsRedundantChild = [](uint32 Child, uint32 Parent)
		{
			return (bool)((Parent != Child) & ((Parent & Child) == Child));
		};

		auto Choose = [](int n, int k)
		{
			int Numerator = 1;
			int Denominator = 1;
			while (k >= 1)
			{
				int NextNumerator = Numerator * n;
				check(NextNumerator > Numerator); // watch out for overflow
				Numerator = NextNumerator;
				Denominator *= k;
				--n;
				--k;
			}
			return Numerator / Denominator;
		};

		// Add, Iteration, AddRange
		{
			int MaxNonRedundant = Choose(BitWidth, BitWidth / 2);
			int TupleWidth = MaxNonRedundant + 1;
			constexpr int MaxValue = TPropertyCombinationSet<BitWidth>::MaxValue;
			constexpr int MaxBufferLength = 10;
			check(MaxBufferLength >= MaxNonRedundant + 1);
			uint32 AddValues[MaxBufferLength];
			AddValues[0] = 0;
			int NumValues = 1;
			// Depth-first search of all the tuples from 0,0,...,0 to MaxValue,MaxValue,...,MaxValue, with TupleWidth elements in the tuple
			auto Pop = [&NumValues, &AddValues]()
			{
				--NumValues;
				if (NumValues > 0)
				{
					++AddValues[NumValues - 1];
				}
			};
			auto Push = [&NumValues, &AddValues, TupleWidth]()
			{
				if (NumValues < TupleWidth)
				{
					AddValues[NumValues] = 0;
				}
				++NumValues;
			};
			while (NumValues > 0)
			{
				if (NumValues == TupleWidth)
				{
					// This is the inner loop that occurs once for each tuple
					// Run the tests for this tuple,

					// Add and Contains:
					// Add each element of the tuple to a TPropertyCombinationSet one after another and verify that new redundant values are not added and that existing redundant values are removed
					uint32 ExistingValues[MaxBufferLength];
					TPropertyCombinationSet<BitWidth> Set;
					int NumExisting = 0; // 0 is present in an empty TPropertyCombinationSet
					ExistingValues[NumExisting++] = 0;
					for (int AddIndex = 0; AddIndex < NumValues; ++AddIndex)
					{
						uint32 AddValue = AddValues[AddIndex];
						Set.Add(AddValue);
						bool bAddedIsRedundant = false;
						bool bAddedAlreadyExists = false;
						for (int ExistingIndex = 0; ExistingIndex < NumExisting; )
						{
							uint32 ExistingValue = ExistingValues[ExistingIndex];
							bAddedAlreadyExists = bAddedAlreadyExists | (AddValue == ExistingValue);
							bAddedIsRedundant = bAddedIsRedundant | IsRedundantChild(AddValue, ExistingValue); //-V792
							bool bExistingIsRedundant = IsRedundantChild(ExistingValue, AddValue);
							if (bExistingIsRedundant)
							{
								for (int ShiftIndex = ExistingIndex; ShiftIndex < NumExisting - 1; ++ShiftIndex)
								{
									ExistingValues[ShiftIndex] = ExistingValues[ShiftIndex + 1];
								}
								--NumExisting;
								TestFalse(TEXT("Hardcoded Add correctly removed existing redundant"), Set.Contains(ExistingValue));
							}
							else
							{
								TestTrue(TEXT("Hardcoded Add correctly kept existing non-redundant"), Set.Contains(ExistingValue));
								++ExistingIndex;
							}
						}
						if (bAddedIsRedundant)
						{
							TestFalse(TEXT("Hardcoded Add correctly did not add redundant"), Set.Contains(AddValue));
						}
						else if (bAddedAlreadyExists)
						{
							// We already tested that the set contains its duplicate value, so no need to test again, and we should not add the duplicate to existing
						}
						else
						{
							TestTrue(TEXT("Hardcoded Add correctly added non-redundant"), Set.Contains(AddValue));
							check(NumExisting < MaxNonRedundant);
							ExistingValues[NumExisting++] = AddValue;
						}
					}
					TestTrue(TEXT("Hardcoded MaxNonRedundant is correct - trying to add more than MaxNonRedundant results in one of them being redundant"), NumExisting <= MaxNonRedundant);


					// Iteration
					Algo::Sort(TArrayView<uint32>(ExistingValues, NumExisting));
					int IterationIndex = 0;
					for (uint32 Value : Set)
					{
						TestTrue(TEXT("Hardcoded Iteration lists the existing non-redundant values in order"), IterationIndex < NumExisting&& Value == ExistingValues[IterationIndex]);
						++IterationIndex;
					}

					// AddRange
					for (int FirstNum = 0; FirstNum < TupleWidth; ++FirstNum)
					{
						TPropertyCombinationSet<BitWidth> Set1;
						TPropertyCombinationSet<BitWidth> Set2;
						for (int Index = 0; Index < FirstNum; ++Index)
						{
							Set1.Add(AddValues[Index]);
						}
						for (int Index = FirstNum; Index < TupleWidth; ++Index)
						{
							Set2.Add(AddValues[Index]);
						}
						Set1.AddRange(Set2);
						TestTrue(TEXT("Hardcoded AddRange results in same set as adding all values to a single set"), Set1 == Set);
					}

					// Done with tests, move on to the next tuple
					Pop();
				}
				else if (AddValues[NumValues - 1] <= MaxValue)
				{
					Push();
				}
				else
				{
					Pop();
				}
			}
		}
	}

	template <uint32 BitWidth>
	void TestGenericSet()
	{
		// Constructor
		{
			TPropertyCombinationSet<BitWidth> Set;
			TestTrue(TEXT("Constructor stores value 0"), Set.Contains(0));
			for (int n = 1; n <= TPropertyCombinationSet<BitWidth>::MaxValue; ++n)
			{
				TestFalse(TEXT("Generic Constructor is empty other than 0"), Set.Contains(n));
			}

			for (int AddValue = 0; AddValue <= TPropertyCombinationSet<BitWidth>::MaxValue; ++AddValue)
			{
				TPropertyCombinationSet<BitWidth> HackConstructedSet;
				HackConstructedSet.Add(AddValue);
				TPropertyCombinationSet<BitWidth> CopyConstructed(HackConstructedSet);
				for (int n = 0; n <= TPropertyCombinationSet<BitWidth>::MaxValue; ++n)
				{
					TestEqual(TEXT("Generic CopyConstructor matches"), CopyConstructed.Contains(n), HackConstructedSet.Contains(n));
				}
			}
		}

		// Load BitArray
		{
			for (uint32 StartOffset = 0; StartOffset < NumBitsPerDWORD; ++StartOffset)
			{
				uint32 EndBitValue = 1 << TPropertyCombinationSet<BitWidth>::StorageBitCount;
				for (uint32 SetInt = 1; SetInt < EndBitValue; ++SetInt)
				{
					TBitArray<> Archive;
					Archive.Add(true, StartOffset);
					Archive.AddRange(&SetInt, TPropertyCombinationSet<BitWidth>::StorageBitCount, 0);
					TPropertyCombinationSet<BitWidth> SerializedSet;
					SerializedSet.Load(Archive, StartOffset);
					for (int n = 0; n <= TPropertyCombinationSet<BitWidth>::MaxValue; ++n)
					{
						TestEqual(TEXT("Generic Load from archive bits"), SerializedSet.Contains(n), (SetInt & (1 << n)) != 0);
					}
				}
			}
		}

		// Save BitArray
		{
			for (uint32 StartOffset = 0; StartOffset < NumBitsPerDWORD; ++StartOffset)
			{
				uint32 EndBitValue = 1 << TPropertyCombinationSet<BitWidth>::StorageBitCount;
				for (uint32 SetInt = 1; SetInt < EndBitValue; ++SetInt)
				{
					TBitArray<> Archive;
					Archive.Add(true, StartOffset);
					Archive.AddRange(&SetInt, TPropertyCombinationSet<BitWidth>::StorageBitCount, 0);
					TPropertyCombinationSet<BitWidth> SerializedSet;
					SerializedSet.Load(Archive, StartOffset);
					TBitArray<> SavedArchiveZeroes;
					TBitArray<> SavedArchiveOnes;
					SavedArchiveZeroes.Add(true, StartOffset);
					SavedArchiveZeroes.Add(false, TPropertyCombinationSet<BitWidth>::StorageBitCount);
					SavedArchiveOnes.Add(true, StartOffset);
					SavedArchiveOnes.Add(true, TPropertyCombinationSet<BitWidth>::StorageBitCount);

					SerializedSet.Save(SavedArchiveZeroes, StartOffset);
					SerializedSet.Save(SavedArchiveOnes, StartOffset);
					TestTrue(TEXT("Generic Save to archive bits"), SavedArchiveZeroes == Archive && SavedArchiveOnes == Archive);
				}
			}
		}

		// Load uint32*
		{
			uint32 StartOffset = 0;
			uint32 EndBitValue = 1 << TPropertyCombinationSet<BitWidth>::StorageBitCount;
			for (uint32 SetInt = 1; SetInt < EndBitValue; ++SetInt)
			{
				TBitArray<> Archive;
				Archive.Add(true, StartOffset);
				Archive.AddRange(&SetInt, TPropertyCombinationSet<BitWidth>::StorageBitCount, 0);
				uint32 Bits[10];
				Archive.GetRange(0, StartOffset + TPropertyCombinationSet<BitWidth>::StorageBitCount, Bits, 0);
				TPropertyCombinationSet<BitWidth> SerializedSet;
				SerializedSet.Load(Bits);
				for (int n = 0; n <= TPropertyCombinationSet<BitWidth>::MaxValue; ++n)
				{
					TestEqual(TEXT("Generic Load from uint32* bits"), SerializedSet.Contains(n), (SetInt & (1 << n)) != 0);
				}
			}
		}

		// Save uint32*
		{
			uint32 StartOffset = 0;
			uint32 EndBitValue = 1 << TPropertyCombinationSet<BitWidth>::StorageBitCount;
			for (uint32 SetInt = 1; SetInt < EndBitValue; ++SetInt)
			{
				TBitArray<> Archive;
				Archive.Add(true, StartOffset);
				Archive.AddRange(&SetInt, TPropertyCombinationSet<BitWidth>::StorageBitCount, 0);
				TPropertyCombinationSet<BitWidth> SerializedSet;
				SerializedSet.Load(Archive, StartOffset);

				TBitArray<> SavedArchiveZeroes;
				TBitArray<> SavedArchiveOnes;
				SavedArchiveZeroes.Add(true, StartOffset);
				SavedArchiveZeroes.Add(false, TPropertyCombinationSet<BitWidth>::StorageBitCount);
				SavedArchiveOnes.Add(true, StartOffset);
				SavedArchiveOnes.Add(true, TPropertyCombinationSet<BitWidth>::StorageBitCount);
				uint32 BitsZeroes[10];
				uint32 BitsOnes[10];
				SavedArchiveZeroes.GetRange(0, StartOffset + TPropertyCombinationSet<BitWidth>::StorageBitCount, BitsZeroes, 0);
				SavedArchiveOnes.GetRange(0, StartOffset + TPropertyCombinationSet<BitWidth>::StorageBitCount, BitsOnes, 0);

				SerializedSet.Save(BitsZeroes);
				SerializedSet.Save(BitsOnes);
				SavedArchiveZeroes.SetRangeFromRange(0, StartOffset + TPropertyCombinationSet<BitWidth>::StorageBitCount, BitsZeroes, 0);
				SavedArchiveOnes.SetRangeFromRange(0, StartOffset + TPropertyCombinationSet<BitWidth>::StorageBitCount, BitsOnes, 0);
				TestTrue(TEXT("Generic Save to uint32* bits"), SavedArchiveZeroes == Archive && SavedArchiveOnes == Archive);
			}
		}

		auto IsRedundantChild = [](uint32 Child, uint32 Parent)
		{
			return (bool)((Parent != Child) & ((Parent & Child) == Child));
		};

		// Add, Iterate, AddRange - try every pair of values instead of every possible tuple; every possible tuple would take too long
		{
			for (int FirstIndex = 0; FirstIndex <= TPropertyCombinationSet<BitWidth>::MaxValue; ++FirstIndex)
			{
				for (int SecondIndex = 0; SecondIndex <= TPropertyCombinationSet<BitWidth>::MaxValue; ++SecondIndex)
				{
					TPropertyCombinationSet<BitWidth> Set;
					Set.Add(FirstIndex);
					Set.Add(SecondIndex);
					uint32 Existing[2];
					int NumExisting = 0;
					if (FirstIndex  == SecondIndex || !IsRedundantChild(FirstIndex, SecondIndex))
					{
						TestTrue(TEXT("Generic Add - Contains FirstIndex"), Set.Contains(FirstIndex));
						Existing[NumExisting++] = FirstIndex;
					}
					else
					{
						TestFalse(TEXT("Generic Add - Not Contains FirstIndex"), Set.Contains(FirstIndex));
					}
					if (FirstIndex == SecondIndex || !IsRedundantChild(SecondIndex, FirstIndex))
					{
						TestTrue(TEXT("Generic Add - Contains SecondIndex"), Set.Contains(SecondIndex));
						if (FirstIndex != SecondIndex)
						{
							Existing[NumExisting++] = SecondIndex;
						}
					}
					else
					{
						TestFalse(TEXT("Generic Add - Not Contains SecondIndex"), Set.Contains(SecondIndex));
					}
					for (int n = 0; n < TPropertyCombinationSet<BitWidth>::MaxValue; ++n)
					{
						if (n != FirstIndex && n != SecondIndex)
						{
							TestFalse(TEXT("Generic Add - doesn't contain any others"), Set.Contains(n));
						}
					}
					Algo::Sort(TArrayView<uint32>(Existing, NumExisting));
					int IterationIndex = 0;
					for (uint32 Value : Set)
					{
						TestEqual(TEXT("Generic Iterate"), Value, Existing[IterationIndex++]);
					}

					for (int SplitIndex = 0; SplitIndex < 2; ++SplitIndex)
					{
						TPropertyCombinationSet<BitWidth> Set1;
						TPropertyCombinationSet<BitWidth> Set2;
						bool bFirst = true;
						for (int n = 0; n < SplitIndex; ++n) //-V1008
						{
							Set1.Add(bFirst ? FirstIndex : SecondIndex);
							bFirst = false;
						}
						for (int n = SplitIndex; n < 2; ++n)
						{
							Set2.Add(bFirst ? FirstIndex : SecondIndex);
							bFirst = false;
						}
						Set1.AddRange(Set2);
						TestTrue(TEXT("Generic AddRange results in same set as adding all values to a single set"), Set1 == Set);
					}
				}
			}
		}
	}

	bool TestAll()
	{
		TestHardcodedSet<1>();
		TestHardcodedSet<2>();
		TestHardcodedSet<3>();
		TestGenericSet<4>();

		return true;
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPropertyCombinationSetTestSubClass, FPropertyCombinationSetTest, "System.AssetRegistry.PropertyCombinationSet", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);

bool FPropertyCombinationSetTestSubClass::RunTest(const FString& Parameters)
{
	return TestAll();
}

#endif