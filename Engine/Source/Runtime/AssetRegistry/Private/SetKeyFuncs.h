// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "HAL/UnrealMemory.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMath.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"

struct FSetKeyFuncsStats;

/**
 * Replacement for TSet that takes an instance of KeyFuncs rather than static functions on KeyFuncs.
 *
 * KeyFuncs must support:
 * struct KeyFuncs
 * {
 *     ~KeyFuncs();
 *     KeyFuncs(KeyFuncs&& Other); // Used in constructor and SetKeyFuncs
 *     KeyFuncs& operator=(const KeyFuncs& Other); // Used in TSetKeyFuncs operator=(const TSetKeyFuncs&)
 *     KeyFuncs& operator=(KeyFuncs&& Other); // Used in TSetKeyFuncs operator=(TSetKeyFuncs&&)
 *
 *     // Functions needed for ElementType
 *     ElementType GetInvalidElement();
 *     bool IsInvalid(const ElementType&);
 *     uint32 GetTypeHash(const ElementType&);
 *     bool Matches(const ElementType& Element, const ElementType& Comparison);
 * 
 *     // Functions needed for each ComparisonType passed into Find
 *     bool Matches(const ElementType& Element, const ComparisonType& Comparison);
 *     uint32 GetTypeHash(const ComparisonType&);
 * };
 * 
 * ElementType must support:
 * ElementType(const ElementType& Other)
 * ElementType(ElementType&& Other)
 *
 */
template <typename ElementType, typename KeyFuncsType>
class TSetKeyFuncs
{
public:
	struct FIterator;
	struct FIterationSentinel;

public:
	TSetKeyFuncs(KeyFuncsType KeyFuncs, int32 ExpectedNumElements = 0);
	TSetKeyFuncs(const TSetKeyFuncs<ElementType, KeyFuncsType>& Other);
	TSetKeyFuncs(TSetKeyFuncs<ElementType, KeyFuncsType>&& Other);
	TSetKeyFuncs& operator=(const TSetKeyFuncs<ElementType, KeyFuncsType>& Other);
	TSetKeyFuncs& operator=(TSetKeyFuncs<ElementType, KeyFuncsType>&& Other);
	~TSetKeyFuncs();

	void SetKeyFuncs(KeyFuncsType KeyFuncs);

	void Reset();
	void Empty(int32 ExpectedNumElements = 0);
	void Reserve(int32 ExpectedNumElements);
	/**
	 * Shrinks or grows the container to be equal in size to our target hardcoded loadfactor
	 * (HashSize == NumValues/TargetLoadFactor).
	 */
	void ResizeToTargetSize();

	int32 Num() const;
	SIZE_T GetAllocatedSize() const;
	FSetKeyFuncsStats GetStats() const;

	template <typename CompareType>
	const ElementType* Find(const CompareType& Key) const;
	template <typename CompareType>
	const ElementType* FindByHash(uint32 TypeHash, const CompareType& Key) const;
	
	void Add(ElementType Value, bool* bAlreadyExists = nullptr);
	void AddByHash(uint32 TypeHash, ElementType Value, bool* bAlreadyExists = nullptr);
	int32 Remove(const ElementType& Value);
	int32 RemoveByHash(uint32 TypeHash, const ElementType& Value);

public:
	/** Implementation detail for ranged for loops; do not use this type directly. */
	struct FIterator
	{
		FIterator(const TSetKeyFuncs<ElementType, KeyFuncsType>& InOwner);
		const ElementType& operator*() const;
		FIterator& operator++();
		bool operator!=(FIterationSentinel) const;

	private:
		const TSetKeyFuncs& Owner;
		uint32 Bucket;
	};

	/** Implementation detail for ranged for loops; do not use this type directly. */
	struct FIterationSentinel
	{
	};

private:
	// Hidden friends for ranged for-loops
	friend FIterator begin(const TSetKeyFuncs<ElementType, KeyFuncsType>& Set)
	{
		return FIterator(Set);
	}
	friend FIterationSentinel end(const TSetKeyFuncs<ElementType, KeyFuncsType>& Set)
	{
		return FIterationSentinel();
	}

	void AddByHashNoReallocate(uint32 TypeHash, ElementType&& Value, bool* bAlreadyExists);
	void Reallocate(uint32 NewHashSize);
	uint32 GetTargetHashSize() const;
	uint32 GetTargetHashSize(uint32 TargetNumValues) const;
	/**
	 * Convert an integer from GetTypeHash(), or from arithmetic on a Bucket index,
	 * by modulus into the [0, HashSize) range of bucket indices.
	 */
	uint32 HashSpaceToBucketSpace(uint32 HashKey) const;
	void DestructHash();

private:
	constexpr static float MaxLoadFactorDuringAdd = 0.75f;
	constexpr static float TargetLoadFactor = 0.5f;
	constexpr static uint32 InitialAllocationSize = 8;
	constexpr static uint32 MinimumNonZeroSize = 8;

	mutable KeyFuncsType KeyFuncs;
	ElementType* Hash = nullptr;
	uint32 HashSize = 0;
	uint32 NumValues = 0;
};

struct FSetKeyFuncsStats
{
	/** Average number of compares per Find across all keys in the container. */
	float AverageSearch;
	/** The longest number of compares in Find for a key in the container. */
	int32 LongestSearch;
};


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////


template <typename ElementType, typename KeyFuncsType>
inline TSetKeyFuncs<ElementType, KeyFuncsType>::TSetKeyFuncs(KeyFuncsType InKeyFuncs, int32 ExpectedNumElements)
	: KeyFuncs(MoveTemp(InKeyFuncs))
{
	Empty(ExpectedNumElements);
}

template <typename ElementType, typename KeyFuncsType>
inline TSetKeyFuncs<ElementType, KeyFuncsType>::TSetKeyFuncs(const TSetKeyFuncs& Other)
{
	*this = Other;
}

template <typename ElementType, typename KeyFuncsType>
inline TSetKeyFuncs<ElementType, KeyFuncsType>::TSetKeyFuncs(TSetKeyFuncs&& Other)
{
	*this = MoveTemp(Other);
}

template <typename ElementType, typename KeyFuncsType>
inline TSetKeyFuncs<ElementType, KeyFuncsType>&
TSetKeyFuncs<ElementType, KeyFuncsType>::operator=(const TSetKeyFuncs<ElementType, KeyFuncsType>& Other)
{
	if (this == &Other)
	{
		return *this;
	}
	DestructHash();

	KeyFuncs = Other.KeyFuncs;
	HashSize = Other.HashSize;
	NumValues = Other.NumValues;

	if (HashSize > 0)
	{
		Hash = reinterpret_cast<ElementType*>(FMemory::Malloc(sizeof(ElementType) * HashSize, alignof(ElementType)));
		for (uint32 Bucket = 0; Bucket < HashSize; ++Bucket)
		{
			new (&Hash[Bucket]) ElementType(Other.Hash[Bucket]);
		}
	}
	return *this;
}

template <typename ElementType, typename KeyFuncsType>
inline TSetKeyFuncs<ElementType, KeyFuncsType>&
TSetKeyFuncs<ElementType, KeyFuncsType>::operator=(TSetKeyFuncs&& Other)
{
	if (this == &Other)
	{
		return *this;
	}
	DestructHash();

	KeyFuncs = MoveTemp(Other.KeyFuncs);
	Hash = Other.Hash;
	HashSize = Other.HashSize;
	NumValues = Other.NumValues;

	Other.Hash = nullptr;
	Other.HashSize = 0;
	Other.NumValues = 0;
	return *this;
}

template <typename ElementType, typename KeyFuncsType>
inline TSetKeyFuncs<ElementType, KeyFuncsType>::~TSetKeyFuncs()
{
	DestructHash();
}

template <typename ElementType, typename KeyFuncsType>
inline void TSetKeyFuncs<ElementType, KeyFuncsType>::SetKeyFuncs(KeyFuncsType InKeyFuncs)
{
	// Use the destructor and move constructor rather than the move operator= so that the KeyFunc author
	// can use operator= for TSetKeyFuncs::operator=, and use constructor for SetKeyFuncs assignment.
	KeyFuncs.~KeyFuncsType();
	new (&KeyFuncs) KeyFuncsType(MoveTemp(InKeyFuncs));
}

template <typename ElementType, typename KeyFuncsType>
inline void TSetKeyFuncs<ElementType, KeyFuncsType>::Reset()
{
	NumValues = 0;
	if (HashSize > 0)
	{
		ElementType InvalidValue = KeyFuncs.GetInvalidElement();
		for (uint32 Bucket = 0; Bucket < HashSize; ++Bucket)
		{
			Hash[Bucket].~ElementType();
			new (&Hash[Bucket]) ElementType(InvalidValue);
		}
	}
}

template <typename ElementType, typename KeyFuncsType>
inline void TSetKeyFuncs<ElementType, KeyFuncsType>::Empty(int32 ExpectedNumElements)
{
	DestructHash();
	HashSize = GetTargetHashSize(static_cast<uint32>(FMath::Max(0,ExpectedNumElements)));
	NumValues = 0;
	
	if (HashSize > 0)
	{
		ElementType InvalidValue = KeyFuncs.GetInvalidElement();
		Hash = reinterpret_cast<ElementType*>(FMemory::Malloc(sizeof(ElementType) * HashSize, alignof(ElementType)));
		for (uint32 Bucket = 0; Bucket < HashSize; ++Bucket)
		{
			new (&Hash[Bucket]) ElementType(InvalidValue);
		}
	}
}

template <typename ElementType, typename KeyFuncsType>
inline void TSetKeyFuncs<ElementType, KeyFuncsType>::Reserve(int32 ExpectedNumElements)
{
	uint32 NewHashSize = GetTargetHashSize(static_cast<uint32>(FMath::Max(0, ExpectedNumElements)));
	if (NewHashSize > HashSize)
	{
		Reallocate(NewHashSize);
	}
}

template <typename ElementType, typename KeyFuncsType>
inline void TSetKeyFuncs<ElementType, KeyFuncsType>::ResizeToTargetSize()
{
	uint32 TargetHashSize = GetTargetHashSize();
	if (TargetHashSize != HashSize)
	{
		Reallocate(TargetHashSize);
	}
}

template <typename ElementType, typename KeyFuncsType>
inline int32 TSetKeyFuncs<ElementType, KeyFuncsType>::Num() const
{
	return IntCastChecked<int32>(NumValues);
}

template <typename ElementType, typename KeyFuncsType>
inline SIZE_T TSetKeyFuncs<ElementType, KeyFuncsType>::GetAllocatedSize() const
{
	return sizeof(ElementType) * HashSize;
}

template <typename ElementType, typename KeyFuncsType>
inline FSetKeyFuncsStats
TSetKeyFuncs<ElementType, KeyFuncsType>::GetStats() const
{
	FSetKeyFuncsStats Result;
	if (NumValues == 0)
	{
		Result.AverageSearch = 0.f;
		Result.LongestSearch = 0;
		return Result;
	}
	check(HashSize > 0 && Hash != nullptr);

	// Start measuring at the first collision chain on or after 0. That collision chain may
	// have started before 0 and wrapped around.
	uint32 EnumerateStart = 0;
	uint32 CollisionChainCount = 0;
	if (!KeyFuncs.IsInvalid(Hash[EnumerateStart]))
	{
		EnumerateStart = HashSize - 1; 
		++CollisionChainCount;
		for (; CollisionChainCount < HashSize; ++CollisionChainCount)
		{
			if (KeyFuncs.IsInvalid(Hash[EnumerateStart]))
			{
				EnumerateStart = HashSpaceToBucketSpace(EnumerateStart + 1);
				break;
			}
			EnumerateStart = HashSpaceToBucketSpace(EnumerateStart - 1);
		}
	}

	// We do not allow the container to become completely full, so we should always find an unused bucket
	check(CollisionChainCount < HashSize);

	Result.LongestSearch = 0;
	uint64 SumOfSearches = 0;
	bool bFirstLoop = true;
	uint32 CollisionChainStart;
	for (CollisionChainStart = EnumerateStart;
		CollisionChainStart != EnumerateStart || bFirstLoop;
		)
	{
		if (KeyFuncs.IsInvalid(Hash[CollisionChainStart]))
		{
			CollisionChainStart = HashSpaceToBucketSpace(CollisionChainStart + 1);
			bFirstLoop = false;
			continue;
		}

		uint32 Bucket;
		for (Bucket = CollisionChainStart;
			Bucket != EnumerateStart || bFirstLoop;
			Bucket = HashSpaceToBucketSpace(Bucket + 1), bFirstLoop = false)
		{
			const ElementType& Element = Hash[Bucket];
			if (KeyFuncs.IsInvalid(Element))
			{
				break;
			}
			uint32 RealBucket = HashSpaceToBucketSpace(KeyFuncs.GetTypeHash(Element));
			// RealBucket must be on the path from CollisionChainStart to Bucket.
			// Record the length from RealBucket to Bucket, including Bucket itself.
			int32 SearchLength = 0;
			if (CollisionChainStart <= Bucket)
			{
				check(CollisionChainStart <= RealBucket && RealBucket <= Bucket);
				SearchLength = static_cast<int32>(Bucket - RealBucket + 1);
			}
			else
			{
				check(CollisionChainStart <= RealBucket || RealBucket <= Bucket);
				if (RealBucket <= Bucket)
				{
					SearchLength = static_cast<int32>(Bucket - RealBucket + 1);
				}
				else
				{
					SearchLength = static_cast<int32>((HashSize - RealBucket) + Bucket + 1);
				}
			}
			Result.LongestSearch = FMath::Max(Result.LongestSearch, SearchLength);
			SumOfSearches += SearchLength;
		}
		CollisionChainStart = Bucket;
	}

	Result.AverageSearch = static_cast<float>(SumOfSearches) / static_cast<float>(NumValues);
	return Result;
}

template <typename ElementType, typename KeyFuncsType>
template <typename CompareType>
const ElementType* TSetKeyFuncs<ElementType, KeyFuncsType>::Find(const CompareType& Key) const
{
	return FindByHash(KeyFuncs.GetTypeHash(Key), Key);
}

template <typename ElementType, typename KeyFuncsType>
template <typename CompareType>
const ElementType* TSetKeyFuncs<ElementType, KeyFuncsType>::FindByHash(uint32 TypeHash, const CompareType& Key) const
{
	if (Hash == nullptr)
	{
		return nullptr;
	}
	uint32 Bucket = HashSpaceToBucketSpace(TypeHash);
	uint32 CollisionCount;
	for (CollisionCount = 0; CollisionCount < HashSize; ++CollisionCount)
	{
		const ElementType& Element = Hash[Bucket];
		if (KeyFuncs.IsInvalid(Element))
		{
			break;
		}
		if (KeyFuncs.Matches(Element, Key))
		{
			return &Element;
		}
		Bucket = HashSpaceToBucketSpace(Bucket + 1);
	}
	// We do not allow the container to become completely full, so we should always find an unused bucket
	check(CollisionCount < HashSize);
	return nullptr;
}

template <typename ElementType, typename KeyFuncsType>
inline void TSetKeyFuncs<ElementType, KeyFuncsType>::Add(ElementType Value, bool* bAlreadyExists)
{
	AddByHash(KeyFuncs.GetTypeHash(Value), MoveTemp(Value), bAlreadyExists);
}

template <typename ElementType, typename KeyFuncsType>
inline void TSetKeyFuncs<ElementType, KeyFuncsType>::AddByHash(uint32 TypeHash, ElementType Value, bool* bAlreadyExists)
{
	if (KeyFuncs.IsInvalid(Value))
	{
		checkf(false, TEXT("Add called with invalid element."));
		return;
	}
	if (HashSize == 0)
	{
		Empty(InitialAllocationSize);
	}
	check(Hash != nullptr && HashSize > 0);

	AddByHashNoReallocate(TypeHash, MoveTemp(Value), bAlreadyExists);

	float LoadFactor = static_cast<float>(NumValues) / static_cast<float>(HashSize);
	if (LoadFactor > MaxLoadFactorDuringAdd)
	{
		Reallocate(GetTargetHashSize());
	}
}

template <typename ElementType, typename KeyFuncsType>
inline void TSetKeyFuncs<ElementType, KeyFuncsType>::AddByHashNoReallocate(uint32 TypeHash, ElementType&& Value, bool* bAlreadyExists)
{
	uint32 CollisionChainCount = 0;
	uint32 Bucket = HashSpaceToBucketSpace(TypeHash);
	for (CollisionChainCount = 0; CollisionChainCount < HashSize; ++CollisionChainCount)
	{
		const ElementType& ExistingElement = Hash[Bucket];
		if (KeyFuncs.IsInvalid(ExistingElement))
		{
			break;
		}
		else if (KeyFuncs.Matches(ExistingElement, Value))
		{
			// Already exists
			if (bAlreadyExists)
			{
				*bAlreadyExists = true;
				return;
			}
		}
		Bucket = HashSpaceToBucketSpace(Bucket + 1);
	}

	// We do not allow the container to become completely full, so we should always find an unused bucket
	check(CollisionChainCount < HashSize);
	Hash[Bucket].~ElementType();
	new (&Hash[Bucket]) ElementType(MoveTemp(Value));
	++NumValues;
	if (bAlreadyExists)
	{
		*bAlreadyExists = false;
	}
}

template <typename ElementType, typename KeyFuncsType>
inline int32 TSetKeyFuncs<ElementType, KeyFuncsType>::Remove(const ElementType& Value)
{
	return RemoveByHash(KeyFuncs.GetTypeHash(Value), Value);
}

template <typename ElementType, typename KeyFuncsType>
inline int32 TSetKeyFuncs<ElementType, KeyFuncsType>::RemoveByHash(uint32 TypeHash, const ElementType& Value)
{
	if (KeyFuncs.IsInvalid(Value))
	{
		checkf(false, TEXT("Remove called with invalid element."));
		return 0;
	}
	if (NumValues == 0)
	{
		return 0;
	}
	check(HashSize != 0 && Hash != nullptr);

	uint32 CollisionChainCount;
	uint32 Bucket = HashSpaceToBucketSpace(TypeHash);
	for (CollisionChainCount = 0; CollisionChainCount < HashSize; ++CollisionChainCount)
	{
		const ElementType& ExistingElement = Hash[Bucket];
		if (KeyFuncs.IsInvalid(ExistingElement))
		{
			// Does not exist
			return 0;
		}
		else if (KeyFuncs.Matches(ExistingElement, Value))
		{
			break;
		}
		Bucket = HashSpaceToBucketSpace(Bucket + 1);
	}
	// We do not allow the container to become completely full, so we should always find the end of the collision chain.
	check(CollisionChainCount < HashSize);

	// If we remove a value from the middle of a collision chain, we have to shift other elements in the chain down to
	// plug the hole so that Find will be able to find them.
	uint32 HoleIndex = Bucket;
	uint32 CurrentBucket = HashSpaceToBucketSpace(HoleIndex + 1);
	for (CollisionChainCount = 0; CollisionChainCount < HashSize; ++CollisionChainCount)
	{
		ElementType& ExistingElement = Hash[CurrentBucket];
		if (KeyFuncs.IsInvalid(ExistingElement))
		{
			// None of the values in between HoleIndex and CurrentBucket needed to be patched into
			// the hole, and we've reached the end of the collision chain. Leave the hole empty,
			// which will split the collision chain in two (or will decrease the size of the collision
			// chain by one if the hole is at the start or end of the chain).
			break;
		}
		uint32 RealBucket = HashSpaceToBucketSpace(KeyFuncs.GetTypeHash(ExistingElement));

		// We are guaranteed that RealBucket comes earlier in the collision chain than CurrentBucket,
		// because when we resolve collisions during add we only move forward.
		// If the hole is in between RealBucket and CurrentBucket then we need to move the value back from
		// CurrentBucket into the Hole so that we find it when we start searching from RealBucket.
		// But the comparison is complicated because we're searching in a ring; the collision chain might
		// overlap the end of the bucket array and wrap around to the start, so RealBucket might be more than
		// Hole and CurrentBucket even though it is earlier in the collision chain.
		bool bPatchTheHole = false;
		if (RealBucket == CurrentBucket)
		{
			// No need to patch the hole if the value is already assigned to its RealBucket.
		}
		else if (RealBucket < CurrentBucket)
		{
			// Need to patch if the hole is on or after RealBucket on the path from RealBucket to CurrentBucket:
			// ################ RealBucket ### Hole #### CurrentBucket ########
			// No need to patch if the hole is after CurrentBucket on the path from CurrentBucket to RealBucket: 
			// ################ Hole ### RealBucket #### CurrentBucket ########
			// ################ RealBucket #### CurrentBucket ######## Hole ###
			bPatchTheHole = RealBucket <= HoleIndex && HoleIndex < CurrentBucket;
		}
		else
		{
			// Need to patch if the hole is on or after RealBucket on the path from RealBucket to CurrentBucket:
			// ################ Hole ### CurrentBucket #### RealBucket ########
			// ################ CurrentBucket ### RealBucket #### Hole ########
			// No need to patch if the hole is after CurrentBucket on the path from CurrentBucket to RealBucket: 
			// ################ CurrentBucket ### Hole #### RealBucket ########
			bPatchTheHole = HoleIndex < CurrentBucket || RealBucket <= HoleIndex;
		}

		if (bPatchTheHole)
		{
			// Move Value into the hole, which fills the hole, and creates a new hole at CurrentBucket. We now need
			// to patch the new hole, so continue iterating.
			Hash[HoleIndex].~ElementType();
			new (&Hash[HoleIndex]) ElementType(MoveTemp(ExistingElement));
			HoleIndex = CurrentBucket;
		}

		CurrentBucket = HashSpaceToBucketSpace(CurrentBucket + 1);
	}
	// We do not allow the container to become completely full, so we should always find the end of the collision chain.
	check(CollisionChainCount < HashSize);

	// We decided not to fill the last hole we created, so mark it as an unused bucket.
	Hash[HoleIndex].~ElementType();
	new (&Hash[HoleIndex]) ElementType(KeyFuncs.GetInvalidElement());
	--NumValues;

	return 1;
}

template <typename ElementType, typename KeyFuncsType>
inline void TSetKeyFuncs<ElementType, KeyFuncsType>::Reallocate(uint32 NewHashSize)
{
	check(NumValues == 0 || NewHashSize > NumValues);

	if (NewHashSize == 0)
	{
		Empty();
	}
	else
	{
		ElementType* OldHash = Hash;
		uint32 OldHashSize = HashSize;
		uint32 OldNumValues = NumValues;

		HashSize = NewHashSize;
		ElementType InvalidValue = KeyFuncs.GetInvalidElement();
		Hash = reinterpret_cast<ElementType*>(FMemory::Malloc(sizeof(ElementType) * HashSize, alignof(ElementType)));
		for (uint32 Bucket = 0; Bucket < HashSize; ++Bucket)
		{
			new (&Hash[Bucket]) ElementType(InvalidValue);
		}

		NumValues = 0;
		for (uint32 OldBucket = 0; OldBucket < OldHashSize; ++OldBucket)
		{
			ElementType& OldElement = OldHash[OldBucket];
			if (!KeyFuncs.IsInvalid(OldElement))
			{
				uint32 OldTypeHash = KeyFuncs.GetTypeHash(OldElement);
				AddByHashNoReallocate(OldTypeHash, MoveTemp(OldElement), nullptr);
			}
			OldElement.~ElementType();
		}
		FMemory::Free(OldHash);
		check(NumValues == OldNumValues);
	}
}

template <typename ElementType, typename KeyFuncsType>
inline uint32 TSetKeyFuncs<ElementType, KeyFuncsType>::GetTargetHashSize() const
{
	return GetTargetHashSize(NumValues);
}

template <typename ElementType, typename KeyFuncsType>
inline uint32 TSetKeyFuncs<ElementType, KeyFuncsType>::GetTargetHashSize(uint32 TargetNumValues) const
{
	if (TargetNumValues == 0)
	{
		return 0;
	}
	uint32 TargetHashSize = static_cast<uint32>(FMath::CeilToInt32(
		static_cast<float>(TargetNumValues) / static_cast<float>(TargetLoadFactor)));
	TargetHashSize = FMath::Max(TargetHashSize, MinimumNonZeroSize);
	return TargetHashSize;
}

template <typename ElementType, typename KeyFuncsType>
inline uint32 TSetKeyFuncs<ElementType, KeyFuncsType>::HashSpaceToBucketSpace(uint32 HashKey) const
{
	return HashSize != 0 ? (HashKey % HashSize) : 0;
}

template <typename ElementType, typename KeyFuncsType>
inline void TSetKeyFuncs<ElementType, KeyFuncsType>::DestructHash()
{
	if (Hash)
	{
		for (uint32 Bucket = 0; Bucket < HashSize; ++Bucket)
		{
			Hash[Bucket].~ElementType();
		}
		FMemory::Free(Hash);
		Hash = nullptr;
	}
}

template <typename ElementType, typename KeyFuncsType>
inline TSetKeyFuncs<ElementType, KeyFuncsType>::FIterator::FIterator(const TSetKeyFuncs& InOwner)
: Owner(InOwner)
, Bucket(0)
{
	if (Owner.HashSize > 0 && Owner.KeyFuncs.IsInvalid(Owner.Hash[Bucket]))
	{
		this->operator++();
	}
}

template <typename ElementType, typename KeyFuncsType>
inline const ElementType& TSetKeyFuncs<ElementType, KeyFuncsType>::FIterator::operator*() const
{
	return Owner.Hash[Bucket];
}

template <typename ElementType, typename KeyFuncsType>
inline typename TSetKeyFuncs<ElementType, KeyFuncsType>::FIterator&
TSetKeyFuncs<ElementType, KeyFuncsType>::FIterator::operator++()
{
	++Bucket;
	while (Bucket < Owner.HashSize && Owner.KeyFuncs.IsInvalid(Owner.Hash[Bucket]))
	{
		++Bucket;
	}
	return *this;
}

template <typename ElementType, typename KeyFuncsType>
inline bool TSetKeyFuncs<ElementType, KeyFuncsType>::FIterator::operator!=(FIterationSentinel) const
{
	return Bucket < Owner.HashSize;
}
