// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "SetKeyFuncs.h"

#include "Containers/Array.h"
#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(FSetKeyFuncsTest, "System::AssetRegistry::SetKeyFuncs", "[ApplicationContextMask][EngingeFilter]")
{
	struct FData
	{
		TArray<uint32> TypeHashes;
		TArray<uint32> ValuesA;
		TArray<uint32> ValuesB;
		TArray<bool> ValueInSet;
		TArray<bool> ValueInSetCopiedFromSet;
		TArray<uint32> RemainingValues;
		uint32 NumInSet = 0;
		uint32 EndValue = 500;
	};
	struct FKeyFuncs1
	{
		uint32 GetInvalidElement() 
		{
			return (uint32)-1;
		}
		bool IsInvalid(uint32 Value)
		{
			return Value == (uint32)-1;
		}
		uint32 GetTypeHash(uint32 Value)
		{
			if (Value < Data->EndValue)
			{
				uint32 TypeHash = Data->TypeHashes[Value];
				if (TypeHash != (uint32)-1)
				{
					return TypeHash;
				}
			}
			FAIL_CHECK(FString::Printf(TEXT("GetTypeHash was unexpectedly called on unknown value %u."), Value));
			return (uint32)-1;
		}
		bool Matches(uint32 A, uint32 B)
		{
			return A == B;
		}
		FData* Data;
	};

	FData Data;
	TSetKeyFuncs<uint32, FKeyFuncs1> Set(FKeyFuncs1{ &Data });

	auto ValidateExpectedSetContents = [&Data, &Set]()
		{
			for (uint32 VFind = 0; VFind < Data.EndValue; ++VFind)
			{
				if (Data.TypeHashes[VFind] == (uint32)-1)
				{
					continue;
				}
				const uint32* ExistingValue = Set.Find(VFind);
				if (ExistingValue != nullptr != Data.ValueInSet[VFind])
				{
					if (Data.ValueInSet[VFind])
					{
						FAIL_CHECK(FString::Printf(TEXT("Expected in-set value %u was unexpectedly not found."), VFind));
					}
					else
					{
						FAIL_CHECK(FString::Printf(TEXT("Expected not-in-set value %u was unexpectedly found."), VFind));
					}
				}
				else if (ExistingValue && *ExistingValue != VFind)
				{
					FAIL_CHECK(FString::Printf(TEXT("Expected value %u returned invalid result %u."),
						VFind, *ExistingValue));
				}
			}

			for (uint32 VIter = 0; VIter < Data.EndValue; ++VIter)
			{
				Data.ValueInSetCopiedFromSet[VIter] = false;
			}
			for (uint32 VIter : Set)
			{
				if (Data.ValueInSetCopiedFromSet[VIter])
				{
					FAIL_CHECK(FString::Printf(TEXT("Value %u unexpectedly encountered twice in Set iterator."), VIter));
				}
				Data.ValueInSetCopiedFromSet[VIter] = true;
			}
			for (uint32 VIter = 0; VIter < Data.EndValue; ++VIter)
			{
				if (Data.ValueInSet[VIter] != Data.ValueInSetCopiedFromSet[VIter])
				{
					if (Data.ValueInSet[VIter])
					{
						FAIL_CHECK(FString::Printf(TEXT("Expected in-set value %u was unexpectedly not found in the Set iterator."), VIter));
					}
					else
					{
						FAIL_CHECK(FString::Printf(TEXT("Expected not-in-set value %u was unexpectedly found in the Set iterator."), VIter));
					}
				}
			}
			if (Set.Num() != Data.NumInSet)
			{
				FAIL_CHECK(FString::Printf(TEXT("Set Num(%d) != expected num(%d)."), Set.Num(), Data.NumInSet));
			}
		};

	Data.ValueInSet.Reserve(Data.EndValue);
	Data.TypeHashes.Reserve(Data.EndValue);
	for (uint32 n = 0; n < Data.EndValue; ++n)
	{
		Data.TypeHashes.Add((uint32)-1);
		Data.ValueInSet.Add(false);
	}
	Data.ValueInSetCopiedFromSet.SetNum(Data.EndValue);
	for (uint32 V = 50; V < 150; ++V)
	{
		check(V < Data.EndValue && V + 200 < Data.EndValue);
		Data.ValuesA.Add(V);
		Data.TypeHashes[V] = V + 1000;
	}
	for (uint32 V = 250; V < 350; ++V)
	{
		check(V < Data.EndValue);
		Data.ValuesB.Add(V);
		Data.TypeHashes[V] = (V - 200) + 1000;
	}
	for (uint32 V = 450; V< 500; ++V)
	{
		check(V< Data.EndValue);
		Data.TypeHashes[V] = V + 2000;
	}

	for (int32 Trial = 0; Trial < 6; ++Trial)
	{
		Data.RemainingValues.Reset();
		for (uint32 V = 0; V < Data.EndValue; ++V)
		{
			Data.ValueInSet[V] = false;
		}
		Data.NumInSet = 0;

		switch (Trial)
		{
		default:
			Set.Reset();
			break;
		case 2:
			Set.Empty(1000);
			break;
		case 3:
			Set.Empty(10);
			break;
		case 4:
			Set.Empty();
			Set.Reserve(50);
			break;
		}

		for (uint32 V : Data.ValuesA)
		{
			Set.Add(V);
			Data.ValueInSet[V] = true;
			++Data.NumInSet;
			ValidateExpectedSetContents();
		}

		switch (Trial)
		{
		default:
			break;
		case 3:
			Set.ResizeToTargetSize();
			break;
		case 4:
		{
			TSetKeyFuncs<uint32, FKeyFuncs1> MovedSet(MoveTemp(Set));
			Set = MoveTemp(MovedSet);
			break;
		}
		case 5:
		{
			TSetKeyFuncs<uint32, FKeyFuncs1> CopySet(Set);
			CopySet.SetKeyFuncs(FKeyFuncs1{ &Data });
			Set.Empty();
			Set = CopySet;
			break;
		}
		}

		for (uint32 V : Data.ValuesB)
		{
			Set.Add(V);
			Data.ValueInSet[V] = true;
			++Data.NumInSet;
			ValidateExpectedSetContents();
		}

		Data.RemainingValues.Append(Data.ValuesA);
		uint32 RemoveIndex = 7;
		while (!Data.RemainingValues.IsEmpty())
		{
			RemoveIndex = (RemoveIndex + 13) % Data.RemainingValues.Num();
			uint32 RemoveValue = Data.RemainingValues[RemoveIndex];
			Data.RemainingValues.RemoveAtSwap(RemoveIndex);
			Data.ValueInSet[RemoveValue] = false;
			--Data.NumInSet;
			Set.Remove(RemoveValue);
			ValidateExpectedSetContents();
		}

		for (uint32 V : Data.ValuesA)
		{
			Set.Add(V);
			Data.ValueInSet[V] = true;
			++Data.NumInSet;
			ValidateExpectedSetContents();
		}

		FSetKeyFuncsStats Stats = Set.GetStats();
		CHECK_MESSAGE(TEXT("AverageSearch >= 1"), Stats.AverageSearch >= 1.f);
		CHECK_MESSAGE(TEXT("LongestSearch >= 1"), Stats.LongestSearch >= 1);
		CHECK_MESSAGE(TEXT("LongestSearch >= 1"), Set.GetAllocatedSize() >= Set.Num() * 1);
	}

	TSetKeyFuncs<uint32, FKeyFuncs1> EmptySet(FKeyFuncs1{ &Data });
	FSetKeyFuncsStats Stats = EmptySet.GetStats();
	CHECK_MESSAGE(TEXT("Empty AverageSearch == 0.0"), Stats.AverageSearch == 0.0f);
	CHECK_MESSAGE(TEXT("Empty LongestSearch == 0"), Stats.AverageSearch == 0);
}

#endif // WITH_TESTS