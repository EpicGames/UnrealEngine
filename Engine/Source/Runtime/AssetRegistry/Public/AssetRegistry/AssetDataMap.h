// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/Optional.h"
#include "Misc/StringBuilder.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "Templates/TypeCompatibleBytes.h"
#include "UObject/NameTypes.h"

struct FSoftObjectPath;
template <typename ElementType, typename KeyFuncs> class TSetKeyFuncs;

/**
 * UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS: Save memory in the AssetRegistryState in 64-bit systems.
 * AssetRegistryState has multiple containers of FAssetData*. Store these containers as int32 indexes into an array
 * of FAssetData* rather than storing the fullsize int64 pointer in each container.
 * 
 * Only useful for tight runtime systems since the savings is small.
 * Increases the cputime cost of AssetRegistry queries since every read of an FAssetData* during a query now adds
 * an extra memory read (and frequently this is a cache miss).
 * 
 * Defined or not, The FAssetDataMap CachedAssets structure is the authoritative list of all the FAssetData* that are
 * present in the FAssetRegistryState.
 * 
 * When not defined, the FAssetDataMap structure is a TSet<FAssetData*>, with a KeyFuncs that knows how to lookup an
 * FAssetData by its objectpath name (FCachedAssetKey). All of the other query structures use the FAssetData*
 * pointer directly as the key.
 * 
 * When defined, the FAssetDataMap structure has a TSetKeyFuncs that maps from FCachedAssetKey to index
 * (FAssetDataPtrIndex), and it has a TArray of FAssetData* that FAssetDataPtrIndex addresses. All of the other query
 * structures use FAssetDataPtrIndex as the key.
 * 
 * Because of the complexity of FAssetDataMap, it should not be used outside of the implementation of
 * FAssetRegistryState.
 * 
 * This optimization is disabled by default because of the increased cputime cost of queries.
 */
#ifndef UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
#define UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS 0
#endif

namespace UE::AssetRegistry::Private
{

/* 
* Key type for TSet<FAssetData*> in the asset registry.
* Top level assets are searched for by their asset path as two names (e.g. '/Path/ToPackageName' + 'AssetName')
* Other assets (e.g. external actors) are searched for by their full path with the whole outer chain as a single name. 
* (e.g. '/Path/To/Package.TopLevel:Subobject' + 'DeeperSubobject')
*/
struct FCachedAssetKey
{
	explicit FCachedAssetKey(const FAssetData* InAssetData);
	explicit FCachedAssetKey(const FAssetData& InAssetData);
	explicit FCachedAssetKey(FTopLevelAssetPath InAssetPath);
	explicit FCachedAssetKey(const FSoftObjectPath& InObjectPath);

	FString ToString() const;
	int32 Compare(const FCachedAssetKey& Other) const;	// Order asset keys with fast non-lexical comparison
	void AppendString(FStringBuilderBase& Builder) const;

	FName OuterPath = NAME_None;
	FName ObjectName = NAME_None;
};

inline FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FCachedAssetKey& Key);
inline bool operator==(const FCachedAssetKey& A, const FCachedAssetKey& B);
inline bool operator!=(const FCachedAssetKey& A, const FCachedAssetKey& B);
inline uint32 GetTypeHash(const FCachedAssetKey& A);

/* 
* Policy type for TSet<FAssetData*> to use FCachedAssetKey for hashing/equality.
* This allows us to store just FAssetData* in the map without storing an extra copy of the key fields to save memory.
*/
struct FCachedAssetKeyFuncs
{
	using KeyInitType = FCachedAssetKey;
	using ElementInitType = void; // TSet doesn't actually use this type 

	enum { bAllowDuplicateKeys = false };

	static inline KeyInitType GetSetKey(const FAssetData* Element)
	{
		return FCachedAssetKey(*Element);
	}

	static inline bool Matches(KeyInitType A, KeyInitType B)
	{
		return A == B;
	}

	static inline uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(Key);
	}
};

#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS

/**
 * When not using indirection, an FAssetDataMap is a TSet, but with the complication that it stores pointers to
 * FAssetData while supporting lookup by FCachedAssetKey. @see FCachedAssetKey.
 */
using FAssetDataMap = TSet<FAssetData*, FCachedAssetKeyFuncs>;
using FConstAssetDataMap = TSet<const FAssetData*, FCachedAssetKeyFuncs>;

#else

/** A 32-bit index to a 64-bit pointer, to an FAssetData. */
using FAssetDataPtrIndex = uint32;
constexpr FAssetDataPtrIndex AssetDataPtrIndexInvalid = MAX_uint32;

struct FAssetObjectNameKeyFuncs;
using FAssetObjectNameSet = TSetKeyFuncs<FAssetDataPtrIndex, FAssetObjectNameKeyFuncs>;

/**
 * Maps FCachedAssetKey to the FAssetDataPtrIndex.
 * Maps FAssetDataPtrIndex to FAssetData*.
 * This indirection is used to save memory in other query structures;
 * @see UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS.
 */
class FAssetDataMap
{
public:
	struct FIterator;

	ASSETREGISTRY_API FAssetDataMap();
	ASSETREGISTRY_API FAssetDataMap(FAssetDataMap&& Other);
	ASSETREGISTRY_API FAssetDataMap& operator=(FAssetDataMap&& Other);
	ASSETREGISTRY_API ~FAssetDataMap();

	ASSETREGISTRY_API void Empty(int32 ReservedSize = 0);
	ASSETREGISTRY_API void Shrink();
	ASSETREGISTRY_API FAssetDataPtrIndex Add(FAssetData* AssetData, bool* bAlreadyInSet = nullptr);
	/**
	 * Add the given AssetIndex to AssetByObjectName, when it is already in the AssetDatas array.
	 * Used for updates of the AssetData that change its name but not its pointer.
	 */
	ASSETREGISTRY_API void AddKeyLookup(FAssetData* AssetData, FAssetDataPtrIndex AssetIndex, bool* bAlreadyInSet = nullptr);
	ASSETREGISTRY_API int32 Remove(const FCachedAssetKey& Key);
	/**
	 * Remove the AssetData* with the given key from AssetByObjectName, but keep it in the AssetDatas array.
	 * Used for updates of the AssetData that change its name but not its pointer.
	 */
	ASSETREGISTRY_API int32 RemoveOnlyKeyLookup(const FCachedAssetKey& Key);

	ASSETREGISTRY_API int32 Num() const;
	ASSETREGISTRY_API SIZE_T GetAllocatedSize() const;
	ASSETREGISTRY_API TArray<FAssetData*> Array() const;
	ASSETREGISTRY_API int32 GetMaxIndex() const;
	ASSETREGISTRY_API bool IsValidId(FSetElementId SetId) const;
	ASSETREGISTRY_API FAssetData* Get(FSetElementId SetId) const;

	ASSETREGISTRY_API bool Contains(const FCachedAssetKey& Key) const;
	ASSETREGISTRY_API FAssetData* const* Find(const FCachedAssetKey& Key) const;
	ASSETREGISTRY_API FAssetDataPtrIndex FindId(const FCachedAssetKey& Key) const;
	ASSETREGISTRY_API FAssetData* operator[](FAssetDataPtrIndex AssetIndex) const;

	/** Iterator of the FAssetData* that also provides the index of each FAssetData to the calling code. */
	ASSETREGISTRY_API void Enumerate(TFunctionRef<bool(FAssetData& AssetData, FAssetDataPtrIndex AssetIndex)> Callback) const;

	ASSETREGISTRY_API FIterator begin() const;
	ASSETREGISTRY_API FIterator end() const;

public:
	/** Ranged for-loop iterator for the FAssetData*, do not use directly. */
	struct FIterator
	{
		ASSETREGISTRY_API FIterator(const FAssetDataMap& InOwner, int32 InIndex);

		ASSETREGISTRY_API FAssetData* operator*() const;
		ASSETREGISTRY_API FIterator& operator++();
		ASSETREGISTRY_API bool operator!=(const FIterator& Other) const;

		const FAssetDataMap& Owner;
		int32 Index;
	};

private:
	/**
	 * We keep a FreeList of FAssetData* that are no longer in use, because we need to have stable indices for
	 * AssetData* in AssetDatas. To save memory, we keep the FreeList as a linked list in the AssetDatas structure;
	 * FAssetData* elements that are not in use are in the freelist and we reinterpret them as uint32 NextIndex.
	 * They set the low-bit to distinguish them from inuse pointers (which are 4-byte aligned and so have 0 in that
	 * bit; IsInUse reads this low bit.
	 */
	static bool IsInUse(const FAssetData* DataFromAssetDatas);
	void AddToFreeList(FAssetDataPtrIndex Index);
	FAssetDataPtrIndex PopFreeIndex();
	uint32 AssetByObjectNameValueToTypeHash(FAssetDataPtrIndex Value) const;
	bool AssetByObjectNameValueMatches(FAssetDataPtrIndex Value, const FCachedAssetKey& Key) const;

private:
	TUniquePtr<FAssetObjectNameSet> AssetByObjectName; // TUniquePtr for implementation hiding
	TArray<FAssetData*> AssetDatas;
	FAssetDataPtrIndex FreeIndex = AssetDataPtrIndexInvalid;
	int32 NumFree = 0;

	friend struct FAssetObjectNameKeyFuncs;
};

/** Integer type used to index FAssetRegistryState.AssetDataArrays. */
using FAssetDataArrayIndex = uint32;

/**
 * Used to optimize for memory a container of FAssetDataPtrIndex arrays, where most of the arrays contain only a single
 * element. This type is a union that is either the single FAssetDataPtrIndex, or is an FAssetDataArrayIndex, which
 * indexes into an external array of TArray<FAssetDataPtrIndex> in the same way that an FAssetDataPtrIndex indexes
 * into an external array of FAssetData*.
 */
struct FAssetDataOrArrayIndex
{
	/** Default constructor returns the same result as CreateEmptyList(). */
	FAssetDataOrArrayIndex() = default; 
	static FAssetDataOrArrayIndex CreateEmptyList();
	static FAssetDataOrArrayIndex CreateAssetDataPtrIndex(FAssetDataPtrIndex AssetIndex);
	static FAssetDataOrArrayIndex CreateArrayIndex(FAssetDataArrayIndex ArrayIndex);

	bool IsEmptyList() const;
	bool IsAssetDataPtrIndex() const;
	bool IsAssetDataArrayIndex() const;
	bool operator==(const FAssetDataOrArrayIndex& Other) const;
	bool operator!=(const FAssetDataOrArrayIndex& Other) const;

	FAssetDataPtrIndex AsAssetDataPtrIndex() const;
	FAssetDataArrayIndex AsAssetDataArrayIndex() const;

public:
	// Implementation details for classes that need to make assumptions about the conversion
	static constexpr uint32 EmptyList		= 0xffff'ffff;
	static constexpr uint32 TypeMask		= 0x8000'0000;
	static constexpr uint32 AssetDataType	= 0x0000'0000;
	static constexpr uint32 ArrayType		= 0x8000'0000;

private:
	uint32 Value = EmptyList;
};

/**
 * Contains arrays of AssetDataPtrIndex that are referred to from the PackageNameMap by an index.
 * Provides an API for editing and reading as a TArrayView an FAssetDataOrArrayIndex,
 * regardless of whether that FAssetDataOrArrayIndex is a single FAssetDataPtrIndex or an
 * FAssetDataArrayIndex that points to an array of FAssetDataPtrIndex.
 */
class FIndirectAssetDataArrays
{
public:
	struct FIterator;

	ASSETREGISTRY_API void AddElement(FAssetDataOrArrayIndex& Array, FAssetDataPtrIndex AssetIndex);
	ASSETREGISTRY_API void RemoveElement(FAssetDataOrArrayIndex& Array, FAssetDataPtrIndex AssetIndex);
	ASSETREGISTRY_API void RemoveAllElements(FAssetDataOrArrayIndex& Array);
	ASSETREGISTRY_API TConstArrayView<FAssetDataPtrIndex> Iterate(const FAssetDataOrArrayIndex* ArrayPtr) const;

	ASSETREGISTRY_API SIZE_T GetAllocatedSize() const;
	ASSETREGISTRY_API void Empty();
	ASSETREGISTRY_API void Shrink();

private:
	static constexpr uint32 UnusedIndex = static_cast<uint32>(~0);

	/**
	 * Since the elements in Arrays need to have a stable index, we keep track of a freelist
	 * whenever one of those elements is freed. To store the freelist without a wasteful additional
	 * container, we use a linked list, with bytes used in the array reinterpreted as an index of the
	 * next element of the freelist. This requires a union that can be either a TArray or a uint32
	 * NextIndex, which is what FArrayOrNextIndex provides.
	 */
	struct FArrayOrNextIndex
	{
		union
		{
			TArray<FAssetDataPtrIndex> Array;
			uint32 NextIndex;
		};
		bool bArray;

		FArrayOrNextIndex();
		FArrayOrNextIndex(FArrayOrNextIndex&& Other);
		FArrayOrNextIndex(const FArrayOrNextIndex& Other) = delete;
		FArrayOrNextIndex& operator=(const FArrayOrNextIndex& Other) = delete;
		FArrayOrNextIndex& operator=(FArrayOrNextIndex&& Other) = delete;
		~FArrayOrNextIndex();
	};

	int32 AllocateArrayIndex();
	void ReleaseArrayIndex(int32 Index);

private:
	TArray<FArrayOrNextIndex> Arrays;
	uint32 FreeList = UnusedIndex;
};

struct FAssetPackageNameKeyFuncs;
using FAssetPackageNameSet = TSetKeyFuncs<FAssetDataOrArrayIndex, FAssetPackageNameKeyFuncs>;
/**
 * Structure that is the same size as TSetKeyFuncs::FIterator. This allows us to have a copy of TSetKeyFuncs::FIterator
 * with a forward declare rather than needing to include the header. The downside is we have to manually keep the size
 * in sync (which we enforce with a static_assert in cpp).
 */
using FAssetPackageNameSetIteratorBytes = TAlignedBytes<16, 8>;

/**
 * An API similar to TMap<FName, FAssetData*> that internally handles the compact data representation of actually used
 * for CachedAssetsByPackageName when UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS is on.
 * The actual values in the map in that case are FAssetDataOrArrayIndex, which need to be converted to a
 * TArrayView<FAssetData*>.
 */
class FAssetPackageNameMap
{
public:
	struct FIterator;
	struct FIterationSentinel;
	using KeyType = FName;

	ASSETREGISTRY_API FAssetPackageNameMap(FAssetDataMap& InAssetDataMap, FIndirectAssetDataArrays& InIndirectAssetDataArrays);
	ASSETREGISTRY_API FAssetPackageNameMap& operator=(FAssetPackageNameMap&& Other);
	ASSETREGISTRY_API ~FAssetPackageNameMap();

	ASSETREGISTRY_API void Empty(int32 ReservedSize=0);
	ASSETREGISTRY_API void Add(FName PackageName, FAssetDataPtrIndex AssetIndex);
	ASSETREGISTRY_API int32 Remove(FName PackageName, FAssetDataPtrIndex AssetIndex);
	ASSETREGISTRY_API void Shrink();

	ASSETREGISTRY_API int32 Num() const;
	ASSETREGISTRY_API SIZE_T GetAllocatedSize() const;

	ASSETREGISTRY_API void GenerateKeyArray(TArray<FName>& OutKeys) const;
	/**
	 * Return an arrayview to the list of FAssetDataPtrIndex stored for the given PackageName, or an empty optional
	 * if not found. The memory at the array view can be invalidated if any non-const functions on this,
	 * or on IndirectArrays, is called. The arrayview should be no longer used after calling any of those functions.
	 */
	ASSETREGISTRY_API TOptional<TConstArrayView<FAssetDataPtrIndex>> Find(FName PackageName) const;
	ASSETREGISTRY_API bool Contains(FName PackageName) const;
	ASSETREGISTRY_API FIterator begin() const;
	ASSETREGISTRY_API FIterationSentinel end() const;

public:
	// We don't handle these copy/moves because we rely on references to some other containers
	FAssetPackageNameMap(const FAssetPackageNameMap& Other) = delete;
	FAssetPackageNameMap(FAssetPackageNameMap&& Other) = delete;
	FAssetPackageNameMap& operator=(const FAssetPackageNameMap&& Other) = delete;

	/**
	 * Provides the key in the same API as TPair<FName, TArray<FAssetData*>> returned from a
	 * TMap<FName, TArray<FAssetData*>> ranged for loop. The TArray<FAssetData*> value is not provided since
	 * it is expensive to create.
	 */
	struct FIteratorValue
	{
		FName Key;
	};
	/** Ranged for-loop iterator, do not use directly. */
	struct FIterator
	{
	public:
		ASSETREGISTRY_API explicit FIterator(const FAssetPackageNameMap& InOwner);
		ASSETREGISTRY_API ~FIterator();
		ASSETREGISTRY_API FIteratorValue operator*() const;
		ASSETREGISTRY_API FIterator& operator++();
		ASSETREGISTRY_API bool operator!=(FIterationSentinel) const;
	private:
		const FAssetPackageNameMap& Owner;
		FAssetPackageNameSetIteratorBytes HashIterBytes;
	};
	struct FIterationSentinel
	{
	};

private:
	uint32 AssetOrArrayByPackageNameValueToTypeHash(FAssetDataOrArrayIndex Value) const;
	bool AssetOrArrayByPackageNameValueMatches(FAssetDataOrArrayIndex Value, FName PackageName) const;
	FAssetData* AssetOrArrayIndexToFirstAssetDataPtr(FAssetDataOrArrayIndex DataOrArrayIndex) const;

private:
	TUniquePtr<FAssetPackageNameSet> AssetOrArrayByPackageName; // TUniquePtr for implementation hiding
	FAssetDataMap& AssetDataMap;
	FIndirectAssetDataArrays& IndirectArrays;

	friend struct FAssetPackageNameKeyFuncs;
};

#endif // UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS

} // namespace UE::AssetRegistry::Private


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////


namespace UE::AssetRegistry::Private
{

inline uint32 HashCombineQuick(uint32 A, uint32 B)
{
	return A ^ (B + 0x9e3779b9 + (A << 6) + (A >> 2));
}

inline FCachedAssetKey::FCachedAssetKey(const FAssetData* InAssetData)
{
	if (!InAssetData)
	{
		return;
	}

#if WITH_EDITORONLY_DATA
	if (!InAssetData->GetOptionalOuterPathName().IsNone())
	{
		OuterPath = InAssetData->GetOptionalOuterPathName();
	}
	else
#endif
	{
		OuterPath = InAssetData->PackageName;
	}
	ObjectName = InAssetData->AssetName;
}

inline FCachedAssetKey::FCachedAssetKey(const FAssetData& InAssetData)
	: FCachedAssetKey(&InAssetData)
{
}

inline FCachedAssetKey::FCachedAssetKey(FTopLevelAssetPath InAssetPath)
	: OuterPath(InAssetPath.GetPackageName())
	, ObjectName(InAssetPath.GetAssetName())
{
}

inline FCachedAssetKey::FCachedAssetKey(const FSoftObjectPath& InObjectPath)
{
	if (InObjectPath.GetAssetFName().IsNone())
	{
		// Packages themselves never appear in the asset registry
		return;
	}
	else if (InObjectPath.GetSubPathString().IsEmpty())
	{
		// If InObjectPath represents a top-level asset we can just take the existing FNames.
		OuterPath = InObjectPath.GetLongPackageFName();
		ObjectName = InObjectPath.GetAssetFName();
	}
	else
	{
		// If InObjectPath represents a subobject we need to split the path into the path of the outer and the name of the innermost object.
		TStringBuilder<FName::StringBufferSize> Builder;
		InObjectPath.ToString(Builder);

		const FAssetPathParts Parts = SplitIntoOuterPathAndAssetName(Builder);

		// This should be impossible as at bare minimum concatenating the package name and asset name should add a separator
		check(!Parts.OuterPath.IsEmpty() && !Parts.InnermostName.IsEmpty()); 

		// Don't create FNames for this query struct. If the AssetData exists to find, the FName will already exist due to OptionalOuterPath on FAssetData.
		OuterPath = FName(Parts.OuterPath, FNAME_Find); 
		ObjectName = FName(Parts.InnermostName);
	}
}
inline FString FCachedAssetKey::ToString() const
{
	TStringBuilder<FName::StringBufferSize> Builder;
	AppendString(Builder);
	return FString(Builder);
}

inline int32 FCachedAssetKey::Compare(const FCachedAssetKey& Other) const
{
	if (OuterPath == Other.OuterPath)
	{
		return ObjectName.CompareIndexes(Other.ObjectName);
	}
	else
	{
		return OuterPath.CompareIndexes(Other.OuterPath);
	}
}

inline void FCachedAssetKey::AppendString(FStringBuilderBase& Builder) const
{
	ConcatenateOuterPathAndObjectName(Builder, OuterPath, ObjectName);
}

inline FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FCachedAssetKey& Key)
{
	Key.AppendString(Builder);
	return Builder;
}

inline bool operator==(const FCachedAssetKey& A, const FCachedAssetKey& B)
{
	return A.OuterPath == B.OuterPath && A.ObjectName == B.ObjectName;
}

inline bool operator!=(const FCachedAssetKey& A, const FCachedAssetKey& B)
{
	return A.OuterPath != B.OuterPath || A.ObjectName != B.ObjectName;
}

inline uint32 GetTypeHash(const FCachedAssetKey& A)
{
	return HashCombineQuick(GetTypeHash(A.OuterPath), GetTypeHash(A.ObjectName));
}

#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS

inline FAssetDataOrArrayIndex FAssetDataOrArrayIndex::CreateEmptyList()
{
	return FAssetDataOrArrayIndex();
}

inline FAssetDataOrArrayIndex FAssetDataOrArrayIndex::CreateAssetDataPtrIndex(FAssetDataPtrIndex AssetIndex)
{
	uint32 InValue = static_cast<uint32>(AssetIndex);
	checkf((InValue & TypeMask) == 0,
		TEXT("FAssetDataPtrIndex value is too large. Value == %u. Maximum supported value == 0x7fffffff."),
		InValue);
	FAssetDataOrArrayIndex Result;
	Result.Value = static_cast<uint32>(InValue) | AssetDataType;
	return Result;
}

inline FAssetDataOrArrayIndex FAssetDataOrArrayIndex::CreateArrayIndex(FAssetDataArrayIndex ArrayIndex)
{
	uint32 InValue = static_cast<uint32>(ArrayIndex);
	checkf((InValue & TypeMask) == 0,
		TEXT("FAssetDataArrayIndex value is too large. Value == %u. Maximum supported value == 0x7fffffff."),
		InValue);
	FAssetDataOrArrayIndex Result;
	Result.Value = static_cast<uint32>(InValue) | ArrayType;
	return Result;
}

inline bool FAssetDataOrArrayIndex::IsEmptyList() const
{
	return Value == EmptyList;
}

inline bool FAssetDataOrArrayIndex::IsAssetDataPtrIndex() const
{
	return (Value & TypeMask) == AssetDataType;
}

inline bool FAssetDataOrArrayIndex::IsAssetDataArrayIndex() const
{
	return (Value != EmptyList) & ((Value & TypeMask) == ArrayType);
}

inline bool FAssetDataOrArrayIndex::operator==(const FAssetDataOrArrayIndex& Other) const
{
	return Value == Other.Value;
}

inline bool FAssetDataOrArrayIndex::operator!=(const FAssetDataOrArrayIndex& Other) const
{
	return Value != Other.Value;
}

inline FAssetDataPtrIndex FAssetDataOrArrayIndex::AsAssetDataPtrIndex() const
{
	return static_cast<FAssetDataPtrIndex>(Value & ~TypeMask);
}

inline FAssetDataArrayIndex FAssetDataOrArrayIndex::AsAssetDataArrayIndex() const
{
	return static_cast<FAssetDataPtrIndex>(Value & ~TypeMask);
}

inline FIndirectAssetDataArrays::FArrayOrNextIndex::FArrayOrNextIndex()
	: NextIndex(UnusedIndex)
	, bArray(false)
{
}

inline FIndirectAssetDataArrays::FArrayOrNextIndex::FArrayOrNextIndex(FArrayOrNextIndex&& Other)
{
	if (Other.bArray)
	{
		bArray = true;
		new (&Array) TArray<FAssetDataPtrIndex>(MoveTemp(Other.Array));

		Other.Array.~TArray<FAssetDataPtrIndex>();
		Other.bArray = false;
		Other.NextIndex = UnusedIndex;
	}
	else
	{
		bArray = false;
		NextIndex = Other.NextIndex;
		Other.NextIndex = UnusedIndex;
	}
}

inline FIndirectAssetDataArrays::FArrayOrNextIndex::~FArrayOrNextIndex()
{
	if (bArray)
	{
		Array.~TArray<FAssetDataPtrIndex>();
		bArray = false;
		NextIndex = UnusedIndex;
	}
}

#endif // UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS

} // namespace UE::AssetRegistry::Private