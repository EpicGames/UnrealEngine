// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "Containers/Array.h"
#include "Containers/List.h"
#include "HAL/PreprocessorHelpers.h"
#include "Misc/AssetRegistryInterface.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

class FAssetRegistryState;
class FString;
class UClass;
struct FARCompiledFilter;
struct FARFilter;
struct FAssetData;
class FPathTree; 
template <typename FuncType> class TFunctionRef;

namespace UE::AssetDependencyGatherer::Private
{
	class FRegisteredAssetDependencyGatherer;
};

namespace UE::AssetRegistry
{
	class FAssetRegistryImpl;
}
/**
 * Interface class for functions that return extra dependencies to add to Assets of a given Class, 
 * to e.g. add dependencies on external Assets in a subdirectory. 
 */
class IAssetDependencyGatherer
{
public:
	struct FGathereredDependency
	{
		FName PackageName;
		UE::AssetRegistry::EDependencyProperty Property;
	};
		
	/* The parameters passed to GatherDependencies
	 *
	 * @param AssetData The Asset to which the dependencies will be added
	 * @param AssetRegistryState Helper object: The AssetRegistry's internal state which can be used to run queries
	 *        in the callback. This must be used instead of the usual interface functions on
	 *        IAssetRegistry::GetChecked(), because the callback executes inside the critical section.
	 * @param CompileFilterFunc Helper object: Used to compile FARFilter into FARCompiledFilter for queries sent to
	 *        AssetRegistryState. The same functionality that is normally provided by
	 *        IAssetRegistry::GetChecked().CompileFilter.
	 * @param CachedPathTree The path tree cache inside the AssetRegitry can be used to quickly interogate the path 
	 *        hierarchy. 
	 * @param OutDependencies List of gathered dependencies to add for the asset.
	 * @param OutDependencyDirectories List of directories that the callback queried. The AssetRegistry will call the callback
	 *        again if any assets are added to any of these directories.	 
	 */
	struct FGatherDependenciesContext
	{
		friend class UE::AssetRegistry::FAssetRegistryImpl;
		friend class IAssetDependencyGatherer;
		friend class UE::AssetDependencyGatherer::Private::FRegisteredAssetDependencyGatherer;
		
	private:
		
		const FAssetData& AssetData;
		const FAssetRegistryState& AssetRegistryState;
		const FPathTree& CachedPathTree;
		const TFunctionRef<FARCompiledFilter(const FARFilter&)> CompileFilterFunc;
		TArray<FGathereredDependency>& OutDependencies;
		TArray<FString>& OutDependencyDirectories;

		// helper flag to detect old implementations of IAssetDependencyGatherer
		bool bInterfaceGatherDependenciesCalled = false;
		
	public:

		FGatherDependenciesContext(	const FAssetData& InAssetData, const FAssetRegistryState& InAssetRegistryState, 
									const FPathTree& InCachedPathTree, const TFunctionRef<FARCompiledFilter(const FARFilter&)> InCompileFilterFunc, 
									TArray<FGathereredDependency>& InReturnedDependencies, TArray<FString>& InReturnedDependencyDirectories) :
									AssetData(InAssetData),
									AssetRegistryState(InAssetRegistryState),
									CachedPathTree(InCachedPathTree),
									CompileFilterFunc(InCompileFilterFunc),
									OutDependencies(InReturnedDependencies),
									OutDependencyDirectories(InReturnedDependencyDirectories)
		{
		}

		const FAssetData& GetAssetData() const { return AssetData; }
		const FAssetRegistryState& GetAssetRegistryState() const { return AssetRegistryState; }
		const class FPathTree& GetCachedPathTree() const { return CachedPathTree; }
		TFunctionRef<FARCompiledFilter(const FARFilter&)> GetCompileFilterFunc() const { return CompileFilterFunc;}
		FARCompiledFilter CompileFilter(FARFilter& Filter) { return CompileFilterFunc(Filter); }

		TArray<FGathereredDependency>& GetOutDependencies() { return OutDependencies; }
		TArray<FString>& GetOutDependencyDirectories() { return OutDependencyDirectories; }
	};

	/**
	 * Return the extra dependencies to add to the given AssetData, which is guaranteed to 
	 * have ClassType equal to the class for which the Gatherer was registered.
	 * 
	 * WARNING: For high performance these callbacks are called inside the critical section of the AssetRegistry. 
	 * Attempting to call public functions on the AssetRegistry will deadlock. 
	 * To send queries about what assets exist, used the passed-in interface functions instead.
	 * 
	 * @param Context the various inputs/outputs provided to/from the gatherer
	 
	 */
	 ASSETREGISTRY_API virtual void GatherDependencies(FGatherDependenciesContext& Context) const /*= 0*/;

	 /**
	  * Deprecated version of GatherDependencies, provided for code compatibility. Will be removed
	  * in UE 5.8. 
	  */
	 UE_DEPRECATED(5.8, "Implement GatherDependencies(FGatherDependenciesContext& Context) instead")
	 ASSETREGISTRY_API virtual void GatherDependencies(const FAssetData& AssetData, const FAssetRegistryState& AssetRegistryState, 
		TFunctionRef<FARCompiledFilter(const FARFilter&)> CompileFilterFunc, TArray<IAssetDependencyGatherer::FGathereredDependency>& OutDependencies, 
		TArray<FString>& OutDependencyDirectories) const /*= 0*/;

};

namespace UE::AssetDependencyGatherer::Private
{
	class FRegisteredAssetDependencyGatherer
	{
	public:
		DECLARE_MULTICAST_DELEGATE(FOnAssetDependencyGathererRegistered);
		static ASSETREGISTRY_API FOnAssetDependencyGathererRegistered OnAssetDependencyGathererRegistered;

		ASSETREGISTRY_API FRegisteredAssetDependencyGatherer();
		ASSETREGISTRY_API virtual ~FRegisteredAssetDependencyGatherer();

		virtual UClass* GetAssetClass() const = 0;
		virtual void GatherDependencies(IAssetDependencyGatherer::FGatherDependenciesContext& Context) const = 0;

		static ASSETREGISTRY_API void ForEach(TFunctionRef<void(FRegisteredAssetDependencyGatherer*)> Func);
		static bool IsEmpty() { return !GetRegisteredList(); }

		ASSETREGISTRY_API void GatherDependenciesForGatherer(IAssetDependencyGatherer* GathererInterface, IAssetDependencyGatherer::FGatherDependenciesContext& Context, const char* InGathererClass) const;
	private:

		static ASSETREGISTRY_API TLinkedList<FRegisteredAssetDependencyGatherer*>*& GetRegisteredList();
		TLinkedList<FRegisteredAssetDependencyGatherer*> GlobalListLink;
		mutable bool bCalledOnce = false;
	};
}


/**
 * Used to Register an IAssetDependencyGatherer for a class
 *
 * Example usage:
 *
 * class FMyAssetDependencyGatherer : public IAssetDependencyGatherer { ... }
 * REGISTER_ASSETDEPENDENCY_GATHERER(FMyAssetDependencyGatherer, UMyAssetClass);
 */
#define REGISTER_ASSETDEPENDENCY_GATHERER(GathererClass, AssetClass) \
class UE_JOIN(GathererClass, AssetClass, _Register) : public UE::AssetDependencyGatherer::Private::FRegisteredAssetDependencyGatherer \
{ \
	virtual UClass* GetAssetClass() const override { return AssetClass::StaticClass(); } \
	virtual void GatherDependencies(IAssetDependencyGatherer::FGatherDependenciesContext& Context) const override \
	{ \
		GathererClass Gatherer; \
		GatherDependenciesForGatherer(&Gatherer, Context, #GathererClass); \
	} \
}; \
namespace UE_JOIN(GathererClass, AssetClass) \
{ \
	static UE_JOIN(GathererClass, AssetClass, _Register) DefaultObject; \
}

#endif // WITH_EDITOR 
