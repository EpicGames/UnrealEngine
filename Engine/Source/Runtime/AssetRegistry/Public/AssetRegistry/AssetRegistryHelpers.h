// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptInterface.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"
#include "Containers/ArrayView.h"
#include "AssetRegistryHelpers.generated.h"

class IAssetRegistry;
class UClass;
struct FFrame;
struct FAssetIdentifier;

USTRUCT(BlueprintType)
struct FTagAndValue
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Transient, Category = AssetData)
	FName Tag;

	UPROPERTY(BlueprintReadWrite, Transient, Category = AssetData)
	FString Value;
};

UENUM(BlueprintType, DisplayName = "Sort Order")
enum class EAssetRegistrySortOrder : uint8
{
	Ascending,
	Descending
};

enum class EAssetsWithOuterForPathsFlags
{
	None					= 0,
	// If we should scan recursively or not
	RecursivePaths			= (1 << 0),
	// If we should only include on disk assets
	IncludeOnlyOnDiskAsset	= (1 << 1),
	// If we should only return assets directly outered to the provided outer
	ExactOuter				= (1 << 2),
	// Scan the AssetRegistry for each path passed in
	ScanPaths				= (1 << 3)
};
ENUM_CLASS_FLAGS(EAssetsWithOuterForPathsFlags);

UCLASS(transient)
class UAssetRegistryHelpers : public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, Category = "Asset Registry")
	static TScriptInterface<IAssetRegistry> GetAssetRegistry();

	/** 
	 * Creates asset data from a UObject. 
	 *
	 * @param InAsset	The asset to create asset data for
	 * @param bAllowBlueprintClass	By default trying to create asset data for a blueprint class will create one for the UBlueprint instead
	 */
	UFUNCTION(BlueprintPure, Category = "Asset Data")
	static FAssetData CreateAssetData(const UObject* InAsset, bool bAllowBlueprintClass = false);

	/** Checks to see if this AssetData refers to an asset or is NULL */
	UFUNCTION(BlueprintPure, Category = "Asset Data", meta=(DisplayName = "Is Valid Asset Data", ScriptMethod))
	static bool IsValid(const FAssetData& InAssetData);

	/** Returns true if this is the primary asset in a package, true for maps and assets but false for secondary objects like class redirectors */
	UFUNCTION(BlueprintPure, Category = "Asset Data", meta=(ScriptMethod))
	static bool IsUAsset(const FAssetData& InAssetData);

	/** Returns true if the this asset is a redirector. */
	UFUNCTION(BlueprintPure, Category = "Asset Data", meta=(ScriptMethod))
	static bool IsRedirector(const FAssetData& InAssetData);

	/** Returns the full name for the asset in the form: Class ObjectPath */
	UFUNCTION(BlueprintPure, Category = "Asset Data", meta=(ScriptMethod))
	static FString GetFullName(const FAssetData& InAssetData);

	/** Convert to a SoftObjectPath for loading */
	UFUNCTION(BlueprintPure, Category = "Asset Data", meta=(ScriptMethod))
	static FSoftObjectPath ToSoftObjectPath(const FAssetData& InAssetData);

	UFUNCTION(BlueprintPure, Category = "Asset Data", meta=(ScriptMethod))
	static UClass* GetClass(const FAssetData& InAssetData);

	/** Returns the asset UObject if it is loaded or loads the asset if it is unloaded then returns the result */
	UFUNCTION(BlueprintPure, Category = "Asset Data", meta=(ScriptMethod))
	static UObject* GetAsset(const FAssetData& InAssetData);

	/** Returns true if the asset is loaded */
	UFUNCTION(BlueprintPure, Category = "Asset Data", meta=(ScriptMethod))
	static bool IsAssetLoaded(const FAssetData& InAssetData);

#if WITH_EDITOR
	/** Returns true if the asset is cooked */
	UFUNCTION(BlueprintPure, Category = "Asset Data", meta=(ScriptMethod="IsCooked"))
	static bool IsAssetCooked(const FAssetData& InAssetData);

	/** Returns true if the asset has its editor-only data */
	UFUNCTION(BlueprintPure, Category = "Asset Data", meta=(ScriptMethod="HasEditorOnlyData"))
	static bool AssetHasEditorOnlyData(const FAssetData& InAssetData);
#endif	// WITH_EDITOR

	/** Returns the name for the asset in the form: Class'ObjectPath' */
	UFUNCTION(BlueprintPure, Category = "Asset Data", meta=(ScriptMethod))
	static FString GetExportTextName(const FAssetData& InAssetData);

	/** Gets the value associated with the given tag as a string */
	UFUNCTION(BlueprintPure, Category = "Asset Data", meta=(ScriptMethod))
	static bool GetTagValue(const FAssetData& InAssetData, const FName& InTagName, FString& OutTagValue);

	/**
	 * Populates the FARFilters tags and values map with the passed in tags and values
	 */
	UFUNCTION(BlueprintPure, Category = "Asset Registry")
	static FARFilter SetFilterTagsAndValues(const FARFilter& InFilter, const TArray<FTagAndValue>& InTagsAndValues);

	/** Gets asset data for all blueprint assets that match the filter. ClassPaths in the filter specify the blueprint's parent class. */
	UFUNCTION(BlueprintPure, Category = "Asset Registry", meta=(ScriptMethod))
	static ASSETREGISTRY_API void GetBlueprintAssets(const FARFilter& InFilter, TArray<FAssetData>& OutAssetData);

	/** 
	 * Gets a list of FAssetData for all child classes, loaded or unloaded, if you have a native UClass* convert it
	 * to a FTopLevelAssetPath using UClass::GetClassPathName. The list of derived types can be quite large, but
	 * this function attempts to be as efficient as possible for large asset sets.
	 */
	UFUNCTION(BlueprintCallable, Category = "Asset Registry")
	static ASSETREGISTRY_API void GetDerivedClassAssetData(const TArray<FTopLevelAssetPath>& InBaseClasses, TArray<FAssetData>& OutDerivedClassAssetData);

	/**
	 * Returns the first native class of the asset type that can be found.  Normally this is just the FAssetData::GetClass(),
	 * however if the class is a blueprint generated class it may not be loaded.  In which case GetAncestorClassNames will
	 * be used to find the first native super class.  This can be slow if temporary caching mode is not on.
	 */
	UFUNCTION(BlueprintPure, Category = "Asset Registry", meta=(ScriptMethod))
	static ASSETREGISTRY_API UClass* FindAssetNativeClass(const FAssetData& AssetData);

	DECLARE_DYNAMIC_DELEGATE_RetVal_TwoParams(bool, FSortingPredicate, const FAssetData&, Left, const FAssetData&, Right);
	/**
	 * Sorts the assets based on a custom Blueprint delegate.
	 * 
	 * @param Assets The assets to sort
	 * @param SortingPredicate Implements a Left <= Right relation
	 * @param SortOrder Whether to sort ascending or descending
	 */
	UFUNCTION(BlueprintCallable, Category = "Asset Registry")
	static ASSETREGISTRY_API void SortByPredicate(UPARAM(Ref) TArray<FAssetData>& Assets, FSortingPredicate SortingPredicate, EAssetRegistrySortOrder SortOrder);

	/**
	 * Sorts the assets by their asset name.
	 *
	 * @param Assets The assets to sort
	 * @param SortOrder Whether to sort ascending or descending
	 */
	UFUNCTION(BlueprintCallable, Category = "Asset Registry")
	static ASSETREGISTRY_API void SortByAssetName(UPARAM(Ref) TArray<FAssetData>& Assets, EAssetRegistrySortOrder SortOrder);
	
	/**
	 * Finds references of the provided asset that are of the a class contained in the InMatchClasses set.
	 */
	static ASSETREGISTRY_API void FindReferencersOfAssetOfClass(UObject* AssetInstance, TConstArrayView<UClass*> InMatchClasses, TArray<FAssetData>& OutAssetDatas);

	/**
	 * Finds references of the provided asset that are of the a class contained in the InMatchClasses set.
	 */
	static ASSETREGISTRY_API void FindReferencersOfAssetOfClass(const FAssetIdentifier& InAssetIdentifier, TConstArrayView<UClass*> InMatchClasses, TArray<FAssetData>& OutAssetDatas);
	
	/** Enable/disable asset registry caching mode for the duration of the scope */
	struct FTemporaryCachingModeScope
	{
		ASSETREGISTRY_API FTemporaryCachingModeScope(bool InTempCachingMode);
		ASSETREGISTRY_API ~FTemporaryCachingModeScope();

	private:
		bool PreviousCachingMode;
	};

	/** Checks to see if the given asset data is a blueprint with a base class in the ClassNameSet. This checks the parent asset tag */
	static bool ASSETREGISTRY_API IsAssetDataBlueprintOfClassSet(const FAssetData& AssetData, const TSet<FTopLevelAssetPath>& ClassNameSet);

	/**
	 * Resolves the provided asset path using asset redirectors.
	 * @param	InOutAssetPath	Asset path to resolve [In/Out].
	 */
	static ASSETREGISTRY_API void FixupRedirectedAssetPath(FName& InOutAssetPath);
	
	/**
	 * Resolves the provided soft object asset path using asset redirectors.
	 * @param	InOutSoftObjectPath	Asset path to resolve [In/Out].
	 */
	static ASSETREGISTRY_API void FixupRedirectedAssetPath(FSoftObjectPath& InOutSoftObjectPath);

#if WITH_EDITOR
	/**
	 * Returns all the corresponding assets from the provided paths that are outered to the provided outer name.
	 * @param	InPackagePaths List of package paths to scan for assets.
	 * @param	InOuterPath The outer path to filter asset with.
	 * @param	bInRecursivePaths If we should scan recursively or not.
	 * @param	bInIncludeOnlyOnDiskAsset If we should only include on disk assets.
	 * @param	bInExactOuter If we should only return assets directly outered to the provided outer.
	 */
	UE_DEPRECATED(5.7, "Please use GetAssetsWithOuterForPaths with enum flags")
	static ASSETREGISTRY_API TArray<FAssetData> GetAssetsWithOuterForPaths(TConstArrayView<FName> InPackagePaths, FName InOuterPath, bool bInRecursivePaths = false, bool bInIncludeOnlyOnDiskAsset = false, bool bInExactOuter = true);

	/**
	 * Returns all the corresponding assets from the provided paths that are outered to the provided outer name.
	 * @param	InPackagePaths List of package paths to scan for assets.
	 * @param	InOuterPath The outer path to filter asset with.
	 * @param	InFlags The flags to control the behavior of this function, see EAssetsWithOuterForPathsFlags
	 */
	static ASSETREGISTRY_API TArray<FAssetData> GetAssetsWithOuterForPaths(TConstArrayView<FName> InPackagePaths, FName InOuterPath, EAssetsWithOuterForPathsFlags InFlags = (EAssetsWithOuterForPathsFlags::ExactOuter | EAssetsWithOuterForPathsFlags::ScanPaths));
#endif
};
