// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/AssetRegistryInterface.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"

class UPackage;

namespace AssetRegistryConstants
{
	const FName ModuleName("AssetRegistry");
}

/**
 * Asset registry module
 */
class FAssetRegistryModule : public IModuleInterface, public IAssetRegistryInterface
{

public:

	/** Called right after the module DLL has been loaded and the module object has been created */
	virtual void StartupModule() override;

	virtual IAssetRegistry& Get() const
	{
		return IAssetRegistry::GetChecked();
	}

	/** Reports whether Get is valid to call. Will be true except during engine shutdown. */
	bool IsValid() const
	{
		return IAssetRegistry::Get() != nullptr;
	}

	/** Returns AssetRegistry pointer if valid, or nullptr. Will be non-null except during engine shutdown. */
	IAssetRegistry* TryGet() const
	{
		return IAssetRegistry::Get();
	}

	static IAssetRegistry& GetRegistry()
	{
		return IAssetRegistry::GetChecked();
	}

	static void TickAssetRegistry(float DeltaTime)
	{
		IAssetRegistry::GetChecked().Tick(DeltaTime);
	}

	static void AssetCreated(UObject* NewAsset)
	{
		IAssetRegistry::GetChecked().AssetCreated(NewAsset);
	}

	static void AssetDeleted(UObject* DeletedAsset)
	{
		IAssetRegistry::GetChecked().AssetDeleted(DeletedAsset);
	}

	static void AssetRenamed(const UObject* RenamedAsset, const FString& OldObjectPath)
	{
		IAssetRegistry::GetChecked().AssetRenamed(RenamedAsset, OldObjectPath);
	}

	static void AssetsSaved(TArray<FAssetData>&& SavedAssets)
	{
		IAssetRegistry::GetChecked().AssetsSaved(MoveTemp(SavedAssets));
	}

	static void PackageDeleted(UPackage* DeletedPackage)
	{
		IAssetRegistry::GetChecked().PackageDeleted(DeletedPackage);
	}

	/** Access the dependent package names for a given source package */
	virtual void GetDependencies(FName InPackageName, TArray<FName>& OutDependencies, UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::Package, const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) override
	{
		IAssetRegistry::GetChecked().GetDependencies(InPackageName, OutDependencies, Category, Flags);
	}

	virtual UE::AssetRegistry::EExists TryGetAssetByObjectPath(const FSoftObjectPath& ObjectPath, FAssetData& OutAssetData) const override
	{
		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		if (!AssetRegistry)
		{
			return UE::AssetRegistry::EExists::Unknown;
		}
		return AssetRegistry->TryGetAssetByObjectPath(ObjectPath, OutAssetData);
	}

	virtual UE::AssetRegistry::EExists TryGetAssetPackageData(FName PackageName, FAssetPackageData& OutAssetPackageData, bool bFailIfLockHeld) const override
	{
		FName OutCorrectCasePackageName;
		return TryGetAssetPackageData(PackageName, OutAssetPackageData, OutCorrectCasePackageName, bFailIfLockHeld);
	}
	
	virtual UE::AssetRegistry::EExists TryGetAssetPackageData(FName PackageName, FAssetPackageData& OutAssetPackageData, FName& OutCorrectCasePackageName, bool bFailIfLockHeld) const override
	{
		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		if (!AssetRegistry)
		{
			return UE::AssetRegistry::EExists::Unknown;
		}
		return AssetRegistry->TryGetAssetPackageData(PackageName, OutAssetPackageData, OutCorrectCasePackageName, bFailIfLockHeld);
	}

	virtual bool EnumerateAssets(const FARFilter& Filter, TFunctionRef<bool(const FAssetData&)> Callback,
		UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags = UE::AssetRegistry::EEnumerateAssetsFlags::None) const override
	{
		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		if (!AssetRegistry)
		{
			return false;
		}
		return AssetRegistry->EnumerateAssets(Filter, Callback, InEnumerateFlags);
	}

	virtual bool RegisterOnAssetsAddedDelegate(const TFunction<void(TConstArrayView<FAssetData>)>& Function, FDelegateHandle& OutHandle) override
	{
		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		if (!AssetRegistry)
		{
			return false;
		}
		OutHandle = AssetRegistry->OnAssetsAdded().AddLambda(Function);
		return true;
	}

	virtual bool UnregisterOnAssetsAddedDelegate(const FDelegateHandle& Handle) override
	{
		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		if (!AssetRegistry)
		{
			return false;
		}
		return AssetRegistry->OnAssetsAdded().Remove(Handle);
	}

	virtual bool RegisterOnAssetsRemovedDelegate(const TFunction<void(TConstArrayView<FAssetData>)>& Function, FDelegateHandle& OutHandle) override
	{
		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		if (!AssetRegistry)
		{
			return false;
		}
		OutHandle = AssetRegistry->OnAssetsRemoved().AddLambda(Function);
		return true;
	}

	virtual bool UnregisterOnAssetsRemovedDelegate(const FDelegateHandle& Handle) override
	{
		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		if (!AssetRegistry)
		{
			return false;
		}
		return AssetRegistry->OnAssetsRemoved().Remove(Handle);
	}
};
