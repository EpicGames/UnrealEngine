// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetDataMap.h"
#include "AssetRegistry/AssetRegistryStateSerialization.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/TopLevelAssetPath.h"

/**
 * UE_ASSETREGISTRY_CACHEDASSETSBYTAG: If non-zero, the CachedAssetsByTag field is defined and used. If zero,
 * CachedClassesByTag is defined and used. Both of these are used only in queries for assets by tag.
 * 
 * If CachedAssetsByTag is defined, the queries are as fast as possible, but a large amount of memory is used.
 * If not, then the queries are instead executed using a three step process: CachedClassesByTag to find all classes
 * with the tag, CachedAssetsByClass to find all assets in those classes, and then a filtering step
 * on the resulting list of assets.
 * 
 * The amount of memory used for each:
 * CachedAssetsByTag:  (number of assets) *(average number of tags per asset)*sizeof(Pointer)
 * CachedClassesByTag: (number of classes)*(average number of tags per class)*sizeof(FTopLevelAssetPath)
 * CachedClassesByTag is much smaller because number of classes is smaller than number of assets.
 */
#ifndef UE_ASSETREGISTRY_CACHEDASSETSBYTAG
#define UE_ASSETREGISTRY_CACHEDASSETSBYTAG WITH_EDITORONLY_DATA
#endif 

class FArchive;
class FAssetDataTagMap;
class FAssetDataTagMapSharedView;
class FAssetRegistryGenerator;
class FDependsNode;
class FString;
namespace UE::AssetRegistry { class FAssetRegistryImpl; }
struct FARCompiledFilter;
struct FAssetRegistryHeader;
struct FPrimaryAssetId;
template <typename FuncType> class TFunctionRef;
class FAssetDataGatherer;

/**
 * Load/Save options used to modify how the cache is serialized.
 * These are read out of the AssetRegistry section of Engine.ini and can be changed per platform.
 */
struct FAssetRegistrySerializationOptions
{
	FAssetRegistrySerializationOptions(
		UE::AssetRegistry::ESerializationTarget Target = UE::AssetRegistry::ESerializationTarget::ForGame);

	/** True rather to load/save registry at all */
	bool bSerializeAssetRegistry = false;

	/** True rather to load/save dependency info. If true this will handle hard and soft package references. */
	bool bSerializeDependencies = false;

	/** True rather to load/save dependency info for Name references,  */
	bool bSerializeSearchableNameDependencies = false;

	/** True rather to load/save dependency info for Manage references,  */
	bool bSerializeManageDependencies = false;

	/** If true will read/write FAssetPackageData */
	bool bSerializePackageData = false;

	/** True if CookFilterlistTagsByClass is an allow list. False if it is a deny list. */
	bool bUseAssetRegistryTagsAllowListInsteadOfDenyList = false;

	/**
	 * True if we want to only write out asset data if it has valid tags.
	 * This saves memory by not saving data for things like textures.
	 */
	bool bFilterAssetDataWithNoTags = false;

	/**
	 * True if we also want to filter out dependency data for assets that have no tags.
	 * Only filters if bFilterAssetDataWithNoTags is also true.
	 */
	bool bFilterDependenciesWithNoTags = false;

	/** Filter out searchable names from dependency data */
	bool bFilterSearchableNames = false;

	/**
	 * Keep tags intended for the cooker's output DevelopmentAssetRegistry. This flag defaults to false and is set to
	 * true only by the cooker.
	 */
	bool bKeepDevelopmentAssetRegistryTags = false;

	/**
	 * The map of class pathname to tag set of tags that are allowed in cooked builds.
	 * This is either an allow list or deny list depending on bUseAssetRegistryTagsAllowListInsteadOfDenyList.
	 */
	TMap<FTopLevelAssetPath, TSet<FName>> CookFilterlistTagsByClass;

	/** Tag keys whose values should be stored as FName in cooked builds. */
	TSet<FName> CookTagsAsName;

	/** Tag keys whose values should be stored as FRegistryExportPath in cooked builds. */
	TSet<FName> CookTagsAsPath;

	/** List of asset bundles (could contain wildcards) that will be removed from the AssetRegistry */
	TSet<FString> AssetBundlesDenyList;

	/** Disable all filters */
	void DisableFilters();

private:
	void InitForDevelopment();
};

struct FAssetRegistryLoadOptions
{
	FAssetRegistryLoadOptions() = default;
	explicit FAssetRegistryLoadOptions(const FAssetRegistrySerializationOptions& Options);

	/**
	 *  If non-empty, called on every FAssetData, AssetPackageData, and FDependsNode loaded from
	 * disk, keeping them only if ShouldKeepPackage returns true on their packagename.
	 */
	TFunction<bool(FName PackageName)> ShouldKeepPackage;
	/**
	 * Optional pointer to the FStore previously created by Load, in cases where the same AssetRegistry.bin
	 * file is being reloaded. Caution: If this Store is not identical to the one that would be loaded from disk,
	 * chaotic crashes can occur during loading or afterwards during reading of AsssetDatas.
	 * If non-null, this FStore will be used for all TagData lookups and the serialization of the bytes for it
	 * on disk will be skipped.
	 */
	FixedTagPrivate::FWeakStorePtr Store;
	int32 ParallelWorkers = 0;
	bool bLoadDependencies = true;
	bool bLoadPackageData = true;
	/** 
	 * If true, AssetDatas, AssetPackageDatas and DependsNodes will be individually allocated rather than allocated in a single block,
	 * so that they can be individually freed when their mountpoint is unloaded.
	 */
	bool bMountPointsRequireSeparateAllocationBlocks = false;
};

/** Extra data about the load of an AssetRegistryState from disk that is optionally returned from Load. */
struct FAssetRegistryLoadResults
{
	FAssetRegistryVersion::Type Version;
	FixedTagPrivate::FWeakStorePtr Store;
};

struct FAssetRegistryPruneOptions
{
	TSet<FName> RequiredPackages;
	TSet<FName> RemovePackages;
	TSet<int32> ChunksToKeep;
	FAssetRegistrySerializationOptions Options;

	/* Remove FDependsNodes that do not point to packages */
	bool bRemoveDependenciesWithoutPackages = false;

	/* List of types that should not be pruned because they do not have a package */
	TSet<FPrimaryAssetType> RemoveDependenciesWithoutPackagesKeepPrimaryAssetTypes;
};

struct FAssetRegistryAppendResult
{
	TArray<const FAssetData*> AddedAssets;
	TArray<const FAssetData*> UpdatedAssets;
};

/**
 * The state of an asset registry, this is used internally by IAssetRegistry to represent the disk cache,
 * and is also accessed directly to save/load cooked caches.
 */
class FAssetRegistryState
{
private:
	using FCachedAssetKey = UE::AssetRegistry::Private::FCachedAssetKey;
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	using FAssetDataMap = UE::AssetRegistry::Private::FAssetDataMap;
	using FAssetDataPtrIndex = UE::AssetRegistry::Private::FAssetDataPtrIndex;
	using FAssetDataArrayIndex = UE::AssetRegistry::Private::FAssetDataArrayIndex;
	using FAssetDataOrArrayIndex = UE::AssetRegistry::Private::FAssetDataOrArrayIndex;
#endif

public:
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	// These types are an implementation detail and they and the functions which take/return them are subject to change
	// without deprecation warnings.
	using FAssetDataMap = UE::AssetRegistry::Private::FAssetDataMap;
	using FConstAssetDataMap = UE::AssetRegistry::Private::FConstAssetDataMap;
#endif

	FAssetRegistryState();
	FAssetRegistryState(const FAssetRegistryState&) = delete;
	FAssetRegistryState(FAssetRegistryState&& Rhs);
	ASSETREGISTRY_API ~FAssetRegistryState();

	FAssetRegistryState& operator=(const FAssetRegistryState&) = delete;
	ASSETREGISTRY_API FAssetRegistryState& operator=(FAssetRegistryState&& O);

	using EInitializationMode = UE::AssetRegistry::EAppendMode;
	using EEnumerateAssetsFlags = UE::AssetRegistry::EEnumerateAssetsFlags;

	/**
	 * Does the given path contain assets?
	 * @param bARFiltering Whether to apply filtering from UE::AssetRegistry::FFiltering (false by default)
	 * @note This function doesn't recurse into sub-paths.
	 */
	ASSETREGISTRY_API bool HasAssets(const FName PackagePath, bool bARFiltering=false) const;

	/**
	 * Gets asset data for all assets that match the filter.
	 * Assets returned must satisfy every filter component if there is at least one element in the component's array.
	 * Assets will satisfy a component if they match any of the elements in it.
	 *
	 * @param Filter filter to apply to the assets in the AssetRegistry
	 * @param PackageNamesToSkip explicit list of packages to skip, because they were already added
	 * @param OutAssetData the list of assets in this path
	 * @param bSkipARFilteredAssets If true, skip assets that are skipped by UE::AssetRegistry::FFiltering
	          (false by default)
	 */
	ASSETREGISTRY_API bool GetAssets(const FARCompiledFilter& Filter, const TSet<FName>& PackageNamesToSkip,
		TArray<FAssetData>& OutAssetData, bool bSkipARFilteredAssets = false) const;

	/**
	 * Enumerate asset data for all assets that match the filter.
	 * Assets returned must satisfy every filter component if there is at least one element in the component's array.
	 * Assets will satisfy a component if they match any of the elements in it.
	 *
	 * @param Filter filter to apply to the assets in the AssetRegistry
	 * @param PackageNamesToSkip explicit list of packages to skip, because they were already added
	 * @param Callback function to call for each asset data enumerated
	 * @param InEnumerateFlags flags to control enumeration and filtering. Note that the default
	 *        does not include AllowUnmountedPaths; they will be skipped by default. @see EEnumerateAssetsFlags.
	 */
	ASSETREGISTRY_API bool EnumerateAssets(const FARCompiledFilter& Filter, const TSet<FName>& PackageNamesToSkip, 
		TFunctionRef<bool(const FAssetData&)> Callback,
		EEnumerateAssetsFlags InEnumerateFlags = EEnumerateAssetsFlags::AllowUnfilteredArAssets) const;

	/**
	 * Gets asset data for all assets in the registry state.
	 *
	 * @param PackageNamesToSkip explicit list of packages to skip, because they were already added
	 * @param OutAssetData the list of assets
	 * @param bARFiltering Whether to apply filtering from UE::AssetRegistry::FFiltering (false by default)
	 */
	ASSETREGISTRY_API bool GetAllAssets(const TSet<FName>& PackageNamesToSkip, TArray<FAssetData>& OutAssetData,
		bool bARFiltering = false) const;

	/**
	 * Enumerates asset data for all assets in the registry state.
	 *
	 * @param Callback function to call for each asset data enumerated
	 * @param InEnumerateFlags flags to control enumeration and filtering. The default includes
	 *        all AllowFlags and no assets will be skipped by default. @see EEnumerateAssetsFlags.
	 */
	ASSETREGISTRY_API void EnumerateAllAssets(TFunctionRef<void(const FAssetData&)> Callback,
		EEnumerateAssetsFlags InEnumerateFlags = EEnumerateAssetsFlags::AllowAll) const;

	/**
	 * Enumerates asset data for all assets in the registry state, except for assets in packages that are
	 * in PackageNamesToSkip. Used by the AssetRegistry for scans after an in-memory scan has already been
	 * completed.
	 *
	 * @param PackageNamesToSkip explicit list of packages to skip, because they were already added
	 * @param Callback function to call for each asset data enumerated
	 * @param InEnumerateFlags flags to control enumeration and filtering. Note that the default
	 *        does not include AllowUnmountedPaths; they will be skipped by default. @see EEnumerateAssetsFlags.
	 */
	ASSETREGISTRY_API bool EnumerateAllAssets(const TSet<FName>& PackageNamesToSkip,
		TFunctionRef<bool(const FAssetData&)> Callback,
		EEnumerateAssetsFlags InEnumerateFlags = EEnumerateAssetsFlags::AllowUnfilteredArAssets) const;

	/**
	 * Calls the callback with the LongPackageName of each path that has assets as direct children.
	 * Callback will not be called for parent paths that have childpaths with direct children but do not
	 * have direct children of their own.
	 */
	ASSETREGISTRY_API void EnumerateAllPaths(TFunctionRef<void(FName PathName)> Callback) const;

	/**
	 * Gets the LongPackageNames for all packages with the given PackageName.
	 * Call to check existence of a LongPackageName or find all packages with a ShortPackageName.
	 *
	 * @param PackageName Name of the package to find, may be a LongPackageName or ShortPackageName.
	 * @param OutPackageNames All discovered matching LongPackageNames are appended to this array.
	 */
	ASSETREGISTRY_API void GetPackagesByName(FStringView PackageName, TArray<FName>& OutPackageNames) const;

	/**
	 * Returns the first LongPackageName found for the given PackageName.
	 * Issues a warning and returns the first (sorted lexically) if there is more than one.
	 * Call to check existence of a LongPackageName or find a package with a ShortPackageName.
	 *
	 * @param PackageName Name of the package to find, may be a LongPackageName or ShortPackageName.
	 * @return The first LongPackageName of the matching package, or NAME_None if not found.
	 */
	ASSETREGISTRY_API FName GetFirstPackageByName(FStringView PackageName) const;

	/**
	 * Appends a list of packages and searchable names that are referenced by the supplied package or name.
	 *
	 * @param AssetIdentifier	the name of the package/name for which to gather dependencies
	 * @param OutDependencies	a list of things that are referenced by AssetIdentifier
	 * @param Category	which category(ies) of dependencies to include in the output list.
	 *        Dependencies matching ANY of the OR'd categories will be returned.
	 * @param Flags	which flags are required present or not present on the dependencies.
	 *        Dependencies matching ALL required and NONE excluded bits will be returned.
	 *        For each potentially returned dependency, flags not applicable to their category are ignored.
	 */
	ASSETREGISTRY_API bool GetDependencies(const FAssetIdentifier& AssetIdentifier,
		TArray<FAssetIdentifier>& OutDependencies,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const;
	ASSETREGISTRY_API bool GetDependencies(const FAssetIdentifier& AssetIdentifier,
		TArray<FAssetDependency>& OutDependencies,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const;

	/**
	 * Return whether the supplied package references the query package.
	 *
	 * @param PackageName The name of the package for which to inspect dependencies (eg, /Game/MyFolder/MyAsset)
	 * @param QueryAsset Which package to look for in PackageName's dependencies
	 * @param Category Which category(ies) of dependencies to look in for QueryAsset. Dependencies matching
	 *        ANY of the OR'd categories will be considered.
	 * @param Flags Which flags are required present or not present on the dependencies. Dependencies matching ALL
	 *        required and NONE excluded bits will be considered. For each potentially considered dependency, flags not
	 *        applicable to their category are ignored.
	 */
	ASSETREGISTRY_API bool ContainsDependency(const FAssetIdentifier& AssetIdentifier,
		const FAssetIdentifier& QueryAsset,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const;

	/**
	 * Appends a list of packages and searchable names that reference the supplied package or name.
	 *
	 * @param AssetIdentifier	the name of the package/name for which to gather dependencies
	 * @param OutReferencers	a list of things that reference AssetIdentifier
	 * @param Category	which category(ies) of dependencies to include in the output list.
	 *        Dependencies matching ANY of the OR'd categories will be returned.
	 * @param Flags	which flags are required present or not present on the dependencies.
	 *        Dependencies matching ALL required and NONE excluded bits will be returned.
	 *        For each potentially returned dependency, flags not applicable to their category are ignored.
	 */
	ASSETREGISTRY_API bool GetReferencers(const FAssetIdentifier& AssetIdentifier,
		TArray<FAssetIdentifier>& OutReferencers,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const;
	ASSETREGISTRY_API bool GetReferencers(const FAssetIdentifier& AssetIdentifier,
		TArray<FAssetDependency>& OutReferencers,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const;

	/**
	 * Gets the asset data for the specified object path
	 *
	 * @param ObjectPath the path of the object to be looked up
	 * @return the assets data, null if not found
	 */
	const FAssetData* GetAssetByObjectPath(const FSoftObjectPath& ObjectPath) const;

	const FAssetData* GetAssetByObjectPath(const UE::AssetRegistry::Private::FCachedAssetKey& Key) const;

	/**
	 * Enumerates the asset datas for the specified package name
	 *
	 * @param PackageName the path of the package to be looked up
	 * Calls the callback once for each AssetData* in the package; callback is not called if nothing found.
	 */
	void EnumerateAssetsByPackageName(const FName PackageName,
		TFunctionRef<bool(const FAssetData* AssetData)> Callback) const;
	/** Gets the array of AssetData pointers for the package; does not copy the AssetDatas, just the pointers. */
	TArray<const FAssetData*> CopyAssetsByPackageName(const FName PackageName) const;
	int32 NumAssetsByPackageName(const FName PackageName) const;

	void EnumerateAssetsByPackagePath(FName LongPackagePathName,
		TFunctionRef<bool(const FAssetData* AssetData)> Callback) const;

	/**
	 * Enumerates the asset datas for the specified asset class
	 *
	 * @param ClassPathName the class path name of the assets to look for
	 * Calls the callback once with each AssetData*, callback is not called if nothing found
	 */
	void EnumerateAssetsByClassPathName(const FTopLevelAssetPath ClassPathName,
		TFunctionRef<bool(const FAssetData* AssetData)> Callback) const;

	/**
	 * Enumerates all tags of any asset in the AssetRegistry
	 *
	 * @param Callback the function for each tag
	 */
	void EnumerateTags(TFunctionRef<bool(FName TagName)> Callback) const;

	/** Return whether the given TagName occurs in the tags of any asset in the AssetRegistry */
	bool ContainsTag(FName TagName) const;

	/**
	 * Enumerates the asset datas with the specified asset tag
	 *
	 * @param TagName the tag name to search for
	 * Callst he callback once with each AssetData*, callback is not called if nothing found
	 */
	ASSETREGISTRY_API void EnumerateAssetsByTagName(const FName TagName,
		TFunctionRef<bool(const FAssetData* AssetData)> Callback) const;

	/**
	 * Enumerates all tags of any asset in the AssetRegistry including function that can be called to enumerate
	 * assets for each tag
	 *
	 * @param Callback the function for each tag pair
	 */
	ASSETREGISTRY_API void EnumerateTagToAssetDatas(
		TFunctionRef<bool(FName TagName, IAssetRegistry::FEnumerateAssetDatasFunc EnumerateAssets)> Callback) const;

	/** Returns const version of internal PackageName->PackageData map for fast iteration */
	const TMap<FName, const FAssetPackageData*>& GetAssetPackageDataMap() const;

	/** Get the set of primary assets contained in this state */
	ASSETREGISTRY_API void GetPrimaryAssetsIds(TSet<FPrimaryAssetId>& OutPrimaryAssets) const;

	/** Returns pointer to the asset package data */
	ASSETREGISTRY_API const FAssetPackageData* GetAssetPackageData(FName PackageName) const;
	ASSETREGISTRY_API const FAssetPackageData* GetAssetPackageData(FName PackageName,
		FName& OutCorrectCasePackageName) const;
	ASSETREGISTRY_API FAssetPackageData* GetAssetPackageData(FName PackageName);

	/** Returns all package names */
	void GetPackageNames(TArray<FName>& OutPackageNames) const;

	/** Finds an existing package data, or creates a new one to modify */
	ASSETREGISTRY_API FAssetPackageData* CreateOrGetAssetPackageData(FName PackageName);

	/** Removes existing package data */
	ASSETREGISTRY_API bool RemovePackageData(FName PackageName);

	/** Removes existing package datas, passed in as FAssetIdentifier(PackageName) for internal performance. */
	ASSETREGISTRY_API bool RemovePackageDatas(const TSet<FAssetIdentifier>& PackageNames);

	/** Adds the asset data to the lookup maps */
	ASSETREGISTRY_API void AddAssetData(FAssetData* AssetData);

	/** Add the given tags/values to the asset data associated with the given object path, if it exists */
	ASSETREGISTRY_API void AddTagsToAssetData(const FSoftObjectPath& InObjectPath, FAssetDataTagMap&& InTagsAndValues);

	/** Finds an existing asset data based on object path and updates it with the new value and updates lookup maps */
	ASSETREGISTRY_API void UpdateAssetData(const FAssetData& NewAssetData, bool bCreateIfNotExists=false);
	ASSETREGISTRY_API void UpdateAssetData(FAssetData&& NewAssetData, bool bCreateIfNotExists = false);

	/** Updates an existing asset data with the new value and updates lookup maps */
	ASSETREGISTRY_API void UpdateAssetData(FAssetData* AssetData, const FAssetData& NewAssetData,
		bool* bOutModified = nullptr);
	ASSETREGISTRY_API void UpdateAssetData(FAssetData* AssetData, FAssetData&& NewAssetData,
		bool* bOutModified = nullptr);

	/**
	 * Updates all asset data package flags in the specified package
	 *
	 * @param PackageName the package name
	 * @param PackageFlags the package flags to set
	 * @return True if any assets exists in the package
	 */
	ASSETREGISTRY_API bool UpdateAssetDataPackageFlags(FName PackageName, uint32 PackageFlags);

	/** Removes the asset data from the lookup maps */
	ASSETREGISTRY_API void RemoveAssetData(FAssetData* AssetData, bool bRemoveDependencyData,
		bool& bOutRemovedAssetData, bool& bOutRemovedPackageData);
	ASSETREGISTRY_API void RemoveAssetData(const FSoftObjectPath& SoftObjectPath, bool bRemoveDependencyData,
		bool& bOutRemovedAssetData, bool& bOutRemovedPackageData);
	/** Removes a set of asset data from the lookup maps. */
	ASSETREGISTRY_API void RemoveAssetDatas(TArrayView<FAssetData*> AssetDatas, bool bRemoveDependencyData,
		TBitArray<>& bOutRemovedAssetData, TBitArray<>& bOutRemovedPackageData);

	/**
	 * Clear all dependencies of the given category from the given AssetIdentifier (e.g. package).
	 * Also clears the referencer link from each of the dependencies.
	 */
	ASSETREGISTRY_API void ClearDependencies(const FAssetIdentifier& AssetIdentifier,
		UE::AssetRegistry::EDependencyCategory Category);
	/**
	 * Add the given dependencies to the given AssetIdentifier (e.g. package).
	 * Also adds a referencer link on each of the dependencies.
	 */
	ASSETREGISTRY_API void AddDependencies(const FAssetIdentifier& AssetIdentifier,
		TConstArrayView<FAssetDependency> Dependencies);
	/**
	 * Clears existing dependencies of the given Category(s) and assigns the input Dependencies. Gives an error if 
	 * any elements of Dependencies are outside of the Category(s).
	 */
	ASSETREGISTRY_API void SetDependencies(const FAssetIdentifier& AssetIdentifier,
		TConstArrayView<FAssetDependency> Dependencies,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All);
	/**
	 * Clear all referencers of the given category from the given AssetIdentifier (e.g. package).
	 * Also clears the dependency link from each of the referencers.
	 */
	ASSETREGISTRY_API void ClearReferencers(const FAssetIdentifier& AssetIdentifier,
		UE::AssetRegistry::EDependencyCategory Category);
	/**
	 * Add a dependency on the given AssetIdentifier (e.g. package) from each of the Referencers.
	 * Also adds a referencer link to each referencer on the AssetIdentifer's node.
	 */
	ASSETREGISTRY_API void AddReferencers(const FAssetIdentifier& AssetIdentifier,
		TConstArrayView<FAssetDependency> Referencers);
	/**
	 * Clears existing referencers of the given Category(s) and assigns the input Referencers. Gives an error if
	 * any elements of Referencers are outside of the Category(s).
	 */
	ASSETREGISTRY_API void SetReferencers(const FAssetIdentifier& AssetIdentifier,
		TConstArrayView<FAssetDependency> Referencers,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All);

	/** Resets to default state */
	ASSETREGISTRY_API void Reset();

	void InitializeFromExisting(const FAssetRegistryState& Existing, const FAssetRegistrySerializationOptions& Options,
		EInitializationMode InitializationMode = EInitializationMode::Rebuild, FAssetRegistryAppendResult* OutAppendResult = nullptr);
	/**
	 * Similar to InitializeFromExisting, but it moves the allocations over from InState into *this rather than
	 * copying them. It also shares the InState's existing FStore (if any) rather than creating a new one.
	 * This function is used for faster consumption of states loaded internally by the AssetRegistry.
	 * This function is less general than InitializeFromExisting to reduce complexity; it only supports
	 * EInitializationMode::OnlyUpdateNew and does not support filter flags on the input Options; all instances
	 * of FAssetDatas, FAssetPackageDatas, and FDependsNodes are kept
	 * The InState is reset to empty by this function.
	 */
	void ConsumeExisting(FAssetRegistryState&& Existing, const FAssetRegistrySerializationOptions& Options,
		FAssetRegistryAppendResult* OutAppendResult = nullptr);

	/** 
	 * Prunes an asset cache, this removes asset data, nodes, and package data that isn't needed. 
	 * @param RequiredPackages If set, only these packages will be maintained. If empty it will keep all
	 *        unless filtered by other parameters
	 * @param RemovePackages These packages will be removed from the current set
	 * @param ChunksToKeep The list of chunks that are allowed to remain. Any assets in other chunks are pruned.
	 *        If empty, all assets are kept regardless of chunk
	 * @param Options Serialization options to read filter info from
	 */
	ASSETREGISTRY_API void PruneAssetData(const TSet<FName>& RequiredPackages, const TSet<FName>& RemovePackages,
		const TSet<int32> ChunksToKeep, const FAssetRegistrySerializationOptions& Options);
	ASSETREGISTRY_API void PruneAssetData(const TSet<FName>& RequiredPackages, const TSet<FName>& RemovePackages,
		const FAssetRegistrySerializationOptions& Options);
	ASSETREGISTRY_API void Prune(const FAssetRegistryPruneOptions& PruneOptions);

	
	/**
	 * Initializes a cache from an existing using a set of filters.
	 * This is more efficient than calling InitializeFromExisting and then PruneAssetData.
	 *
	 * @param ExistingState State to use initialize from
	 * @param RequiredPackages If set, only these packages will be maintained.
	 *        If empty it will keep all unless filtered by other parameters.
	 * @param RemovePackages These packages will be removed from the current set
	 * @param ChunksToKeep The list of chunks that are allowed to remain. Any assets in other chunks are pruned.
	 *        If empty, all assets are kept regardless of chunk
	 * @param Options Serialization options to read filter info from
	 */
	ASSETREGISTRY_API void InitializeFromExistingAndPrune(const FAssetRegistryState& ExistingState,
		const TSet<FName>& RequiredPackages, const TSet<FName>& RemovePackages, const TSet<int32> ChunksToKeep,
		const FAssetRegistrySerializationOptions& Options);

	/** Edit every AssetData's Tags to remove Tags that are filtered out by the filtering rules in Options. */
	ASSETREGISTRY_API void FilterTags(const FAssetRegistrySerializationOptions& Options);
	/** Filter a given AssetData's Tags to remove Tags that are filtered out by the filtering rules in Options. */
	ASSETREGISTRY_API static void FilterTags(FAssetData& InOutAssetData,
		const FAssetRegistrySerializationOptions& Options);

	/** Serialize the registry to/from a file, skipping editor only data */
	ASSETREGISTRY_API bool Serialize(FArchive& Ar, const FAssetRegistrySerializationOptions& Options);

	/** Save without editor-only data */
	ASSETREGISTRY_API bool Save(FArchive& Ar, const FAssetRegistrySerializationOptions& Options);
	ASSETREGISTRY_API bool Load(FArchive& Ar,
		const FAssetRegistryLoadOptions& Options = FAssetRegistryLoadOptions(),
		FAssetRegistryVersion::Type* OutVersion = nullptr);
	ASSETREGISTRY_API bool Load(FArchive& Ar,
		const FAssetRegistryLoadOptions& Options,
		FAssetRegistryLoadResults& OutResults);

	/** 
	* Example Usage:
	*	FAssetRegistryState AR;
	*	bool bSucceeded = FAssetRegistryState::LoadFromDisk(TEXT("Path/To/AR"), FAssetRegistryLoadOptions(), AR);
	*/
	static ASSETREGISTRY_API bool LoadFromDisk(const TCHAR* InPath, const FAssetRegistryLoadOptions& InOptions,
		FAssetRegistryState& OutState, FAssetRegistryVersion::Type* OutVersion = nullptr);

	/** Returns memory size of entire registry, optionally logging sizes */
	ASSETREGISTRY_API SIZE_T GetAllocatedSize(bool bLogDetailed = false) const;

	/** Checks a filter to make sure there are no illegal entries */
	static ASSETREGISTRY_API bool IsFilterValid(const FARCompiledFilter& Filter);

	/** Returns the number of assets in this state */
	int32 GetNumAssets() const;

	/** Returns the number of packages in this state */
	int32 GetNumPackages() const;

#if ASSET_REGISTRY_STATE_DUMPING_ENABLED
	/**
	 * Writes out the state in textual form. Use arguments to control which segments to emit.
	 * @param Arguments List of segments to emit.
	 *        Possible values: 'ObjectPath', 'PackageName', 'Path', 'Class', 'Tag', 'Dependencies' and 'PackageData'.
	 * @param OutPages Textual representation will be written to this array;
	 *        each entry will have LinesPerPage lines of the full dump.
	 * @param LinesPerPage - how many lines should be combined into each string element of OutPages,
	 *        for e.g. breaking up the dump into separate files.
	 *        To facilitate diffing between similar-but-different registries, the actual number of lines per page will
	 *        be slightly less than LinesPerPage; we introduce partially deterministic pagebreaks near the end of
	 *        each page.
	 */
	ASSETREGISTRY_API void Dump(const TArray<FString>& Arguments, TArray<FString>& OutPages,
		int32 LinesPerPage=1) const;
#endif

private:
	// Accessors of mutable FAssetData*, for friend classes
	ASSETREGISTRY_API void EnumerateAllMutableAssets(TFunctionRef<void(FAssetData&)> Callback,
		EEnumerateAssetsFlags InEnumerateFlags = EEnumerateAssetsFlags::AllowAll) const;
	/**
	 * Copy the results of EnumerateAllMutableAssets into an array of pointers and sort it by the FAssetData's
	 * objectpaths. Do not use this unless you need them sorted; it is wasteful to sort. Use EnumerateAllMutableAssets
	 * instead, and make your own array. Uses ParallelSort, so do not call this function within functions that are
	 * already running in parallel.
	 */
	ASSETREGISTRY_API TArray<FAssetData*> GetMutableAssetsSortedByObjectPath();

	FAssetData* GetMutableAssetByObjectPath(const FSoftObjectPath& ObjectPath) const;
	FAssetData* GetMutableAssetByObjectPath(const UE::AssetRegistry::Private::FCachedAssetKey& Key) const;
	void EnumerateMutableAssetsByPackageName(const FName PackageName,
		TFunctionRef<bool(FAssetData* AssetData)> Callback) const;
	void EnumerateMutableAssetsByPackagePath(FName LongPackagePathName,
		TFunctionRef<bool(FAssetData* AssetData)> Callback) const;

	/** Initializes cache from existing set of asset data and depends nodes */
	ASSETREGISTRY_API void InitializeFromExisting(const FAssetDataMap& AssetDataMap,
		const TMap<FAssetIdentifier, FDependsNode*>& DependsNodeMap,
		const TMap<FName, FAssetPackageData*>& AssetPackageDataMap,
		const FAssetRegistrySerializationOptions& Options,
		EInitializationMode InitializationMode = EInitializationMode::Rebuild,
		FAssetRegistryAppendResult* OutAppendResult = nullptr);

	template<class Archive>
	void Load(Archive&& Ar, const FAssetRegistryHeader& Header, const FAssetRegistryLoadOptions& Options);

	/** Initialize the lookup maps */
	void SetAssetDatas(TConstArrayView<FAssetData*> AssetDatas, const FAssetRegistryLoadOptions& Options);

	/** Find the first non-redirector dependency node starting from InDependency. */
	FDependsNode* ResolveRedirector(FDependsNode* InDependency, const FAssetDataMap& InAllowedAssets,
		TMap<FDependsNode*, FDependsNode*>& InCache) const;

	/** Finds an existing node for the given package and returns it, or returns null if one isn't found */
	FDependsNode* FindDependsNode(const FAssetIdentifier& Identifier) const;

	/** Creates a node in the CachedDependsNodes map or finds the existing node and returns it */
	FDependsNode* CreateOrFindDependsNode(const FAssetIdentifier& Identifier, bool* bAlreadyExists = nullptr);

	/** Removes the depends node and updates the dependencies to no longer contain it as a referencer. */
	ASSETREGISTRY_API bool RemoveDependsNode(const FAssetIdentifier& Identifier);

	enum class ERemoveDependsFlags
	{
		None,
		/**
		 * If passed in, RemoveDependsNode functions will remove nodes that were not in the remove request
		 * but that had links to the removed nodes and no longer have any links after the removals.
		 */
		RemoveOrphans = 1 << 0,
	};
	FRIEND_ENUM_CLASS_FLAGS(ERemoveDependsFlags);

	/** Removes a set of depends nodes and updates their dependencies to no longer contain them as a referencer. */
	ASSETREGISTRY_API bool RemoveDependsNodes(const TSet<FAssetIdentifier>& Identifiers,
		ERemoveDependsFlags Flags = ERemoveDependsFlags::None);

	/** Removes a set of existing package data. Returns true if any were removed, false if all already did not exist. */
	template<typename SetKeyFuncs, typename SetAllocator>
	bool RemovePackageData(const TSet<FAssetIdentifier, SetKeyFuncs, SetAllocator>& AssetIdentifiers);

	/** Filter a set of tags and output a copy of the filtered set. */
	static void FilterTags(const FAssetDataTagMapSharedView& InTagsAndValues, FAssetDataTagMap& OutTagsAndValues,
		const TSet<FName>* ClassSpecificFilterList, const FAssetRegistrySerializationOptions & Options);

	/** Filter asset bundles and output a copy of the filtered set or input AssetBundles if nothing was filtered */
	static TSharedPtr<FAssetBundleData> FilterBundles(const TSharedPtr<FAssetBundleData>& InAssetBundles,
		const FAssetRegistrySerializationOptions& Options);

	void LoadDependencies(FArchive& Ar, const FAssetRegistryLoadOptions& Options, FAssetRegistryVersion::Type Version);
	template <bool bLatestVersion>
	void LoadDependenciesImpl(FArchive& Ar, const FAssetRegistryLoadOptions& Options, FAssetRegistryVersion::Type Version);
	void LegacyLoadDependencies_BeforeFlags(FArchive& Ar, bool bSerializeDependencies, FAssetRegistryVersion::Type Version);

	void SetTagsOnExistingAsset(FAssetData* AssetData, FAssetDataTagMap&& NewTags);

	void SetDependencyNodeSorting(bool bSortDependencies, bool bSortReferencers);

	void RemoveAssetData(FAssetData* AssetData, const FCachedAssetKey& Key, bool bRemoveDependencyData,
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		UE::AssetRegistry::Private::FAssetDataPtrIndex AssetIndex,
#endif
		bool& bOutRemovedAssetData, bool& bOutRemovedPackageData
	);

	template<typename BitArrayAllocator>
	void RemoveAssetDatasImpl(TArrayView<FAssetData*> AssetDatas, bool bRemoveDependencyData,
		TBitArray<BitArrayAllocator>& bOutRemovedAssetData, TBitArray<BitArrayAllocator>& bOutRemovedPackageData);

	template<typename BitArrayAllocator>
	void RemoveAssetDatasImpl(TArrayView<FAssetData*> AssetDatas, TArrayView<const FCachedAssetKey> Keys, bool bRemoveDependencyData,
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
		TArrayView<UE::AssetRegistry::Private::FAssetDataPtrIndex> AssetIndices,
#endif
		TBitArray<BitArrayAllocator>& bOutRemovedAssetDatas, TBitArray<BitArrayAllocator>& bOutRemovedPackageDatas
	);

	/** Free slack memory accumulated in cache containers. */
	void Shrink();

	/**
	 * Returns true if the given package should be filtered from the results because the package belongs
	 * to an unmounted content path.
	 * This can only happen when loading a cooked asset registry (@see bCookedGlobalAssetRegistryState),
	 * as it may contain state for plugins that are not currently loaded.
	 */
	bool IsPackageUnmountedAndFiltered(const FName PackageName) const;

	/**
	 * Set of asset data for assets saved to disk. Searched via path name types,
	 * implicitly converted to FCachedAssetKey.
	 */
	FAssetDataMap CachedAssets;
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	UE::AssetRegistry::Private::FIndirectAssetDataArrays IndirectAssetDataArrays;
#endif

#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	/** The map of package names to asset data for assets saved to disk */
	TMap<FName, TArray<FAssetData*, TInlineAllocator<1>> > CachedAssetsByPackageName;
	/** The map of long package path to asset data for assets saved to disk */
	TMap<FName, TArray<FAssetData*> > CachedAssetsByPath;
	/** The map of class name to asset data for assets saved to disk */
	TMap<FTopLevelAssetPath, TArray<FAssetData*> > CachedAssetsByClass;
#else
	/** The map of package names to asset data for assets saved to disk */
	UE::AssetRegistry::Private::FAssetPackageNameMap CachedAssetsByPackageName;
	/** The map of long package path to asset data for assets saved to disk */
	TMap<FName, TArray<FAssetDataPtrIndex> > CachedAssetsByPath;
	/** The map of class name to asset data for assets saved to disk */
	TMap<FTopLevelAssetPath, TArray<FAssetDataPtrIndex> > CachedAssetsByClass;
#endif

#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	/** The map of asset tag to asset data for assets saved to disk */
	TMap<FName, TSet<FAssetData*> > CachedAssetsByTag;
#else
	/** The map of asset tag to asset data for assets saved to disk */
	TMap<FName, TSet<FAssetDataPtrIndex> > CachedAssetsByTag;
#endif
#else
	/** The map of asset tag to asset data for assets saved to disk */
	TMap<FName, TSet<FTopLevelAssetPath> > CachedClassesByTag;
#endif

	/** A map of object names to dependency data */
	TMap<FAssetIdentifier, FDependsNode*> CachedDependsNodes;

	/** A map of Package Names to Package Data */
	TMap<FName, FAssetPackageData*> CachedPackageData;

	/**
	 * When loading a registry from disk, we can allocate all the FAssetData objects in one chunk,
	 * to save on 10s of thousands of heap allocations.
	 */
	TArray<FAssetData*> PreallocatedAssetDataBuffers;
	TArray<FDependsNode*> PreallocatedDependsNodeDataBuffers;
	TArray<FAssetPackageData*> PreallocatedPackageDataBuffers;

	/**
	 * Counters for asset/depends data memory allocation to ensure that every FAssetData and
	 * FDependsNode created is deleted.
	 */
	int32 NumAssets = 0;
	int32 NumDependsNodes = 0;
	int32 NumPackageData = 0;

	/** True if this asset registry state was loaded from a cooked asset registry */
	bool bCookedGlobalAssetRegistryState = false;

	friend class FAssetRegistryGenerator;
	friend class UAssetRegistryImpl;
	friend class UE::AssetRegistry::FAssetRegistryImpl;
	friend class FAssetDataGatherer;
};


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////


ENUM_CLASS_FLAGS(FAssetRegistryState::ERemoveDependsFlags);

inline FAssetRegistrySerializationOptions::FAssetRegistrySerializationOptions(
	UE::AssetRegistry::ESerializationTarget Target)
{
	if (Target == UE::AssetRegistry::ESerializationTarget::ForDevelopment)
	{
		InitForDevelopment();
	}
}

inline void FAssetRegistrySerializationOptions::DisableFilters()
{
	bFilterAssetDataWithNoTags = false;
	bFilterDependenciesWithNoTags = false;
	bFilterSearchableNames = false;
	bUseAssetRegistryTagsAllowListInsteadOfDenyList = false;
	CookFilterlistTagsByClass.Empty();
}

inline void FAssetRegistrySerializationOptions::InitForDevelopment()
{
	bSerializeAssetRegistry = true;
	bSerializeDependencies = true;
	bSerializeSearchableNameDependencies = true;
	bSerializeManageDependencies = true;
	bSerializePackageData = true;
	DisableFilters();
}

inline FAssetRegistryLoadOptions::FAssetRegistryLoadOptions(const FAssetRegistrySerializationOptions& Options)
	: bLoadDependencies(Options.bSerializeDependencies)
	, bLoadPackageData(Options.bSerializePackageData)
{
}

inline FAssetRegistryState::FAssetRegistryState()
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	: CachedAssetsByPackageName(CachedAssets, IndirectAssetDataArrays)
#endif
{
}

inline FAssetRegistryState::FAssetRegistryState(FAssetRegistryState&& Rhs)
#if UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	: CachedAssetsByPackageName(CachedAssets, IndirectAssetDataArrays)
#endif
{
	*this = MoveTemp(Rhs);
}

inline const FAssetData* FAssetRegistryState::GetAssetByObjectPath(const FSoftObjectPath& ObjectPath) const
{
	return GetAssetByObjectPath(UE::AssetRegistry::Private::FCachedAssetKey(ObjectPath));
}

inline const FAssetData* FAssetRegistryState::GetAssetByObjectPath(
	const UE::AssetRegistry::Private::FCachedAssetKey& Key) const
{
	return GetMutableAssetByObjectPath(Key);
}

inline FAssetData* FAssetRegistryState::GetMutableAssetByObjectPath(const FSoftObjectPath& ObjectPath) const
{
	return GetMutableAssetByObjectPath(UE::AssetRegistry::Private::FCachedAssetKey(ObjectPath));
}

inline FAssetData* FAssetRegistryState::GetMutableAssetByObjectPath(
	const UE::AssetRegistry::Private::FCachedAssetKey& Key) const
{
	FAssetData* const* FoundAsset = CachedAssets.Find(Key);
	return FoundAsset ? *FoundAsset : nullptr;
}

inline void FAssetRegistryState::EnumerateAssetsByPackageName(const FName PackageName,
	TFunctionRef<bool(const FAssetData* AssetData)> Callback) const
{
	EnumerateMutableAssetsByPackageName(PackageName, [&Callback](FAssetData* AssetData)
		{
			return Callback(AssetData);
		});
}

inline void FAssetRegistryState::EnumerateMutableAssetsByPackageName(const FName PackageName,
	TFunctionRef<bool(FAssetData* AssetData)> Callback) const
{
	using namespace UE::AssetRegistry::Private;

#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	if (const TArray<FAssetData*, TInlineAllocator<1>>* FoundAssetArray
		= CachedAssetsByPackageName.Find(PackageName))
	{
		for (FAssetData* AssetData : *FoundAssetArray)
		{
			if (!Callback(AssetData))
			{
				break;
			}
		}
	}
#else
	if (TOptional<TConstArrayView<FAssetDataPtrIndex>> AssetArray = CachedAssetsByPackageName.Find(PackageName))
	{
		for (FAssetDataPtrIndex Index : *AssetArray)
		{
			if (!Callback(CachedAssets[Index]))
			{
				break;
			}
		}
	}
#endif
}

inline TArray<const FAssetData*> FAssetRegistryState::CopyAssetsByPackageName(const FName PackageName) const
{
	TArray<const FAssetData*> Result;
	Result.Reserve(NumAssetsByPackageName(PackageName));
	EnumerateAssetsByPackageName(PackageName, [&Result](const FAssetData* AssetData)
		{
			Result.Add(AssetData);
			return true;
		});
	return Result;
}

inline int32 FAssetRegistryState::NumAssetsByPackageName(const FName PackageName) const
{
#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	const auto* Array = CachedAssetsByPackageName.Find(PackageName);
	return Array ? Array->Num() : 0;
#else
	const TOptional<TConstArrayView<FAssetDataPtrIndex>> Array = CachedAssetsByPackageName.Find(PackageName);
	return Array ? Array->Num() : 0;
#endif
}

inline void FAssetRegistryState::EnumerateAssetsByPackagePath(FName LongPackagePathName,
	TFunctionRef<bool(const FAssetData* AssetData)> Callback) const
{
	EnumerateMutableAssetsByPackagePath(LongPackagePathName, [&Callback](FAssetData* AssetData)
		{
			return Callback(AssetData);
		});
}

inline void FAssetRegistryState::EnumerateMutableAssetsByPackagePath(FName LongPackagePathName,
	TFunctionRef<bool(FAssetData* AssetData)> Callback) const
{
	using namespace UE::AssetRegistry::Private;

#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	const TArray<FAssetData*>* AssetsInPath = CachedAssetsByPath.Find(LongPackagePathName);
	if (AssetsInPath)
	{
		for (FAssetData* AssetData : *AssetsInPath)
		{
			if (!Callback(AssetData))
			{
				break;
			}
		}
	}
#else
	const TArray<FAssetDataPtrIndex>* AssetsInPath = CachedAssetsByPath.Find(LongPackagePathName);
	if (AssetsInPath)
	{
		for (FAssetDataPtrIndex AssetIndex : *AssetsInPath)
		{
			if (!Callback(CachedAssets[AssetIndex]))
			{
				break;
			}
		}
	}
#endif
}

inline void FAssetRegistryState::EnumerateAssetsByClassPathName(
	const FTopLevelAssetPath ClassPathName, TFunctionRef<bool(const FAssetData* AssetData)> Callback) const
{
	using namespace UE::AssetRegistry::Private;

#if !UE_ASSETREGISTRY_INDIRECT_ASSETDATA_POINTERS
	if (const TArray<FAssetData*>* FoundAssetArray = CachedAssetsByClass.Find(ClassPathName))
	{
		for (FAssetData* AssetData : *FoundAssetArray)
		{
			Callback(AssetData);
		}
	}
#else
	if (const TArray<FAssetDataPtrIndex>* FoundAssetArray = CachedAssetsByClass.Find(ClassPathName))
	{
		for (FAssetDataPtrIndex AssetIndex : *FoundAssetArray)
		{
			if (!Callback(CachedAssets[AssetIndex]))
			{
				break;
			}
		}
	}
#endif
}

inline void FAssetRegistryState::EnumerateTags(TFunctionRef<bool(FName TagName)> Callback) const
{
#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
	for (const auto& Pair : CachedAssetsByTag)
#else
	for (const TPair<FName, TSet<FTopLevelAssetPath>>& Pair : CachedClassesByTag)
#endif
	{
		if (!Callback(Pair.Key))
		{
			break;
		}
	}
}

inline bool FAssetRegistryState::ContainsTag(FName TagName) const
{
#if UE_ASSETREGISTRY_CACHEDASSETSBYTAG
	return CachedAssetsByTag.Contains(TagName);
#else
	return CachedClassesByTag.Contains(TagName);
#endif
}

inline const TMap<FName, const FAssetPackageData*>& FAssetRegistryState::GetAssetPackageDataMap() const
{
	return reinterpret_cast<const TMap<FName, const FAssetPackageData*>&>(CachedPackageData);
}

inline void FAssetRegistryState::GetPackageNames(TArray<FName>& OutPackageNames) const
{
	OutPackageNames.Reserve(CachedAssetsByPackageName.Num());
	for (const auto& It : CachedAssetsByPackageName)
	{
		OutPackageNames.Add(It.Key);
	}
}

inline int32 FAssetRegistryState::GetNumAssets() const
{
	return NumAssets;
}

inline int32 FAssetRegistryState::GetNumPackages() const
{
	return CachedAssetsByPackageName.Num();
}

inline void FAssetRegistryState::InitializeFromExisting(const FAssetRegistryState& Existing,
	const FAssetRegistrySerializationOptions& Options,
	EInitializationMode InitializationMode,
	FAssetRegistryAppendResult* OutAppendResult)
{
	InitializeFromExisting(Existing.CachedAssets, Existing.CachedDependsNodes,
		Existing.CachedPackageData, Options, InitializationMode, OutAppendResult);
}
