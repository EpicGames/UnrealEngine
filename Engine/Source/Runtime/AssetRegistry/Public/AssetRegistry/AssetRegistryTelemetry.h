// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/ArrayView.h"
#include "Misc/Guid.h"

struct FFileChangeData;

namespace UE::Telemetry::AssetRegistry
{
    // Information about very early asset registry startup.
    struct FStartupTelemetry
    {
        static constexpr inline FGuid TelemetryID = FGuid(0x903732ce, 0x8bd04eb9, 0x92170fde, 0x0a1c1562);
        
        // Time spent in synchronous initialization
        double StartupDuration = 0.0;
        // Whether async gather operation was started at this time
        bool bStartedAsyncGather = false;
    };

    // Information about a synchronous scan that was executed, blocking the calling thread.
    struct FSynchronousScanTelemetry
    {
        static constexpr inline FGuid TelemetryID = FGuid(0x2b4b9f1a, 0xdcfd4958, 0xbe43ba9d, 0xae309392);
        
        // List of directories that were scanned
        TConstArrayView<FString> Directories;
        // List of specific files that were scanned
        TConstArrayView<FString> Files;
        // Flags controlling scan behavior
        UE::AssetRegistry::EScanFlags Flags = UE::AssetRegistry::EScanFlags::None;
        // Number of assets found by this scan
        int32 NumFoundAssets = 0;
        // Duration of synchronous scan
        double Duration = 0.0;
        // Whether the main background async gather was started at this time
        bool bInitialSearchStarted = false;
        // Whether the main background async gather was completed at this time
        bool bInitialSearchCompleted = false;
		// Whether an additional main background async gather was started at this time
		bool bAdditionalMountSearchInProgress = false;
    };

    // Information about asset registry scanning either as the initial scan when the process starts,
    // or when an additional mount point is added. See the bIsInitialScan field to disambiguate.
    // Note that for subsequent scans, the times other than TotalSearchDurationSeconds will be cumulative totals including previous work.
    // This is also a good time to gather information about the contents of the asset registry.
    struct FGatherTelemetry
    {
        static constexpr inline FGuid TelemetryID = FGuid(0xafcec052, 0x5d2c4850, 0xbfc6d11d, 0x3163ccd5);
        
        // Total wall clock time between start of search and completion.
        double TotalSearchDurationSeconds = 0.0;
        // Total work time (includes work done in parallel stages)
        double TotalWorkTimeSeconds = 0.0;
        // Time spent discovering asset files on disk
        double DiscoveryTimeSeconds = 0.0;
        // Time spent gathering asset data from files on disk (or cache)
        double GatherTimeSeconds = 0.0;
        // Time spent storing asset data in the asset registry for searching
        double StoreTimeSeconds = 0.0;
        // Number of directories read from cache
        int32 NumCachedDirectories = 0;
        // Number of directories scanned from disk
        int32 NumUncachedDirectories = 0;
        // Number of asset files read from cache
        int32 NumCachedAssetFiles = 0;
        // Number of asset files read loose from disk
        int32 NumUncachedAssetFiles = 0;

        // True if this was the initial startup scan - this should be true only once per process.        
        bool bIsInitialScan = false;
    };
    
    // Information about an asset registry update that was triggered by the directory watcher module. 
    struct FDirectoryWatcherUpdateTelemetry
    {
        static constexpr inline FGuid TelemetryID = FGuid(0xa1da56e1, 0xe1314918, 0xba6fdb24, 0xb8911ce0);
        
        // File change data from watcher
        TConstArrayView<FFileChangeData> Changes;
        // Total time spent in update handler
        double DurationSeconds = 0.0;
        // Whether the main background async gather was started at this time
        bool bInitialSearchStarted = false;
        // Whether the main background async gather was completed at this time
        bool bInitialSearchCompleted = false;
		// Whether an additional main background async gather was started at this time
		bool bAdditionalMountSearchInProgress = false;
    };

	// Information about a failure in our FileJournal area
	struct FFileJournalErrorTelemetry
	{
		static constexpr inline FGuid TelemetryID = FGuid(0x761ebe19, 0x2f0c427e, 0xaec0f06d, 0xe116ff82);

		// Directory that failed for our FileJournal calls
		FString Directory;

		// The error string returned from our lower level FileJournal calls
		FString ErrorString;
	};

	struct FFileJournalWrappedTelemetry
	{
		static constexpr inline FGuid TelemetryID = FGuid(0x542713AA, 0x04724D77, 0x8F7E2319, 0xA0204815);

		FString VolumeName;
		// Size of journal that wrapped
		uint64 JournalMaximumSize;
	};
}
