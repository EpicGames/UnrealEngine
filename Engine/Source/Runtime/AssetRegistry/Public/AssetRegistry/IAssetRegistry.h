// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetIdentifier.h"
#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/StringFwd.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/Optional.h"
#include "Tasks/Task.h"
#include "Templates/Function.h"
#include "Templates/PimplPtr.h"
#include "UObject/Interface.h"
#include "IAssetRegistry.generated.h"

#ifndef ASSET_REGISTRY_STATE_DUMPING_ENABLED
#define ASSET_REGISTRY_STATE_DUMPING_ENABLED !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#endif

class FArchive;
class FAssetRegistryGenerator;
class FAssetRegistryState;
class FCbFieldView;
class FCbWriter;
class FDependsNode;
enum class EAssetRegistryTagsCaller : uint8;
namespace UE::AssetRegistry { struct FSetManageReferencesScratch; }
struct FARFilter;
struct FARCompiledFilter;
struct FAssetData;
struct FAssetRegistrySerializationOptions;
struct FPackageFileSummary;
struct FObjectExport;
struct FObjectImport;
class ITargetPlatform;

namespace EAssetAvailability
{
	enum Type
	{
		DoesNotExist,	// asset chunkid does not exist
		NotAvailable,	// chunk containing asset has not been installed yet
		LocalSlow,		// chunk containing asset is on local slow media (optical)
		LocalFast		// chunk containing asset is on local fast media (HDD)
	};
}

namespace EAssetAvailabilityProgressReportingType
{
	enum Type
	{
		ETA,					// time remaining in seconds
		PercentageComplete		// percentage complete in 99.99 format
	};
}

namespace UE::AssetRegistry
{

// Please update LexToString when modifying this enum
enum class EScanFlags : uint32
{
	None = 0,
	ForceRescan = 1 << 0,				// the paths will be scanned again, even if they were previously scanned
	IgnoreDenyListScanFilters = 1 << 1,	// ignore deny list scan filters
	WaitForInMemoryObjects = 1 << 2,	// update the tags of all assets that have loaded into memory before returning from the scan
	IgnoreInvalidPathWarning = 1 << 3,  // By default, scans of invalid paths (unmounted, or /Temp) log a warning. Suppress that warning.
};
ENUM_CLASS_FLAGS(EScanFlags);

ASSETREGISTRY_API FString LexToString(EScanFlags Flags);

/** Properties about each node in the graph search conducted from SetManageReferences. */
enum class EAssetSetManagerAssetFlags : uint32
{
	None = 0x0,
	ScriptPackage = 0x1,
};
ENUM_CLASS_FLAGS(EAssetSetManagerAssetFlags);

/** Information passed into ShouldSetManager calls from the graph search performed in SetManageReferences. */
struct FShouldSetManagerContext
{
	/** The top-level manager node for which the search is conducted. */
	const FAssetIdentifier& Manager;
	/** The source node of the edge in the graph search that is being asked about in the ShouldSetManager call. */
	const FAssetIdentifier& Source;
	/** The target node of the edge in the graph search that is being asked about in the ShouldSetManager call. */
	const FAssetIdentifier& Target;
	/** Properties of the manager node. */
	EAssetSetManagerAssetFlags ManagerAssetFlags;
	/** Properties of the source node. */
	EAssetSetManagerAssetFlags SourceAssetFlags;
	/** Properties of the source node. */
	EAssetSetManagerAssetFlags TargetAssetFlags;
	/* Properties of the edge, and of the target's status in the current phase of SetManageReferences. */
	EAssetSetManagerFlags::Type EdgeFlags;
	/** AssetRegistry category of the edge: which kind of AssetRegistry dependency the edge comes from. */
	EDependencyCategory EdgeARCategory;
	/** AssetRegistry Properties of the edge: which properties within its category the edge has. */
	EDependencyProperty EdgeARProperties;
};

/**
 * Information passed into SetManageReferences from ther caller. Includes arguments for the call as well as
 * scratch space data that is shared across multiple SetManageReferences calls conducted by the caller for different
 * priorities of managers.
 */
struct FSetManageReferencesContext
{
	/** Map from Managing asset to directly-managed assets for the current call to SetManageReferences.*/
	const TMultiMap<FAssetIdentifier, FAssetIdentifier>* ManagerMap = nullptr;
	/** If true, will clear any existing manage dependencies. Should be set only for the first call. */
	bool bClearExisting = false;
	/** Dependency types to follow during the graph search out from the starting nodes in ManagerMap. */
	EDependencyCategory RecurseType = EDependencyCategory::None;
	/**
	 * Predicate that is called on recursive dependencies if bound, returns whether the graph search should visit a
	 * target node from a source node and whether that visit should be recursive.
	 */
	TOptional<TFunctionRef<EAssetSetManagerResult::Type(FShouldSetManagerContext& Context)>> ShouldSetManager;

	/**
	 * Scratch variable that will be constructed during the call, and should not be modified by the caller but should
	* be preserved across a series of calls to SetManageReferences for managers of different priorities
	*/
	TPimplPtr<FSetManageReferencesScratch> Scratch;
};

/**
 * Enum controlling how FAssetRegistryStates are merged during Append functions, e.g. when adding plugin's serialized AssetRegistry into global AssetRegistry.
 */
enum class EAppendMode
{
	Rebuild,
	OnlyUpdateExisting,
	Append,
	OnlyUpdateNew,
};

} // namespace UE::AssetRegistry

USTRUCT(BlueprintType)
struct FAssetRegistryDependencyOptions
{
	GENERATED_BODY()

	bool GetPackageQuery(UE::AssetRegistry::FDependencyQuery& Flags) const;
	bool GetSearchableNameQuery(UE::AssetRegistry::FDependencyQuery& Flags) const;
	bool GetManageQuery(UE::AssetRegistry::FDependencyQuery& Flags) const;

	/** Dependencies which don't need to be loaded for the object to be used (i.e. soft object paths) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AssetRegistry")
	bool bIncludeSoftPackageReferences = true;

	/** Dependencies which are required for correct usage of the source asset, and must be loaded at the same time */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AssetRegistry")
	bool bIncludeHardPackageReferences = true;

	/** Dependencies which are needed in the game as well as the editor. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AssetRegistry")
	bool bIncludeGamePackageReferences = true;

	/** Dependencies which are only needed in the editor. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AssetRegistry")
	bool bIncludeEditorOnlyPackageReferences = true;

	/** References to specific SearchableNames inside a package */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AssetRegistry")
	bool bIncludeSearchableNames = false;

	/** Indirect management references, these are set through recursion for Primary Assets that manage packages or other primary assets */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AssetRegistry")
	bool bIncludeSoftManagementReferences = false;

	/** Reference that says one object directly manages another object, set when Primary Assets manage things explicitly */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AssetRegistry")
	bool bIncludeHardManagementReferences = false;
};

/** An output struct to hold both an AssetIdentifier and the properties of the dependency on that AssetIdentifier */
struct FAssetDependency
{
	FAssetIdentifier AssetId;
	UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::None;
	UE::AssetRegistry::EDependencyProperty Properties = UE::AssetRegistry::EDependencyProperty::None;

	bool operator==(const FAssetDependency& Other) const
	{
		return AssetId == Other.AssetId && Category == Other.Category && Properties == Other.Properties;
	}
	bool LexicalLess(const FAssetDependency& Other) const
	{
		if (!(AssetId == Other.AssetId))
		{
			return AssetId.LexicalLess(Other.AssetId);
		}
		if (Category != Other.Category)
		{
			return Category < Other.Category;
		}
		return Properties < Other.Properties;
	}

	static FAssetDependency PackageDependency(FName PackageName, UE::AssetRegistry::EDependencyProperty Properties)
	{
		return FAssetDependency{ FAssetIdentifier(PackageName), UE::AssetRegistry::EDependencyCategory::Package,
			Properties };
	}

	friend inline uint32 GetTypeHash(const FAssetDependency& InHash)
	{
		uint32 Hash = GetTypeHash(InHash.AssetId);
		Hash = HashCombine(Hash, GetTypeHash(InHash.Category));
		Hash = HashCombine(Hash, GetTypeHash(InHash.Properties));

		return Hash;
	}


	ASSETREGISTRY_API void WriteCompactBinary(FCbWriter& Writer) const;
private:
	friend FCbWriter& operator<<(FCbWriter& Writer, const FAssetDependency& Dependency)
	{
		// Hidden friend function needs to be inline, but call a subfunction to hide the implementation
		Dependency.WriteCompactBinary(Writer);
		return Writer;
	}
	// Load Cannot be inline because we need to hide implementation and copy-by-value is invalid without definition
	ASSETREGISTRY_API friend bool LoadFromCompactBinary(FCbFieldView Field, FAssetDependency& Dependency);
};

UINTERFACE(MinimalApi, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UAssetRegistry : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

/**
 * Global singleton interface for accessing a catalog of all packages (and some other content file types) that are
 * stored in any mounted directory. In editor this information is gathered from the package files on disk during a
 * gather step at editor startup. In cooked runtimes this information was calculated during cook and is serialized out
 * of a single file (after pruning information not necessary at runtime.)
 * 
 * Some API notes:
 *
 * bIncludeOnlyOnDiskAssets
 *     Most query functions that return FAssetData take this argument. If true, only data collected from disk and
 *     stored in the AssetRegistry will be returned. If false, and the object is loaded in memory, the returned
 *     AssetData will be calculated from the object in memory because the InMemoryData is more likely to be
 *     up-to-date. The InMemory data will sometimes vary from the DiskGatheredData.
 * 
 *     When InMemoryData is returned some categories of data that are always missing from the object in memory (e.g.
 *     GetAssetRegistryTags(EAssetRegistryTagsCaller::SavePackage)) are read from the DiskGatheredData and added to
 *     the InMemoryData.
 * 
 *     Setting this value to true will always be faster than setting it to false, because the same registry
 *     lookups are performed in either case, but the InMemoryData lookup is skipped in the true case.
 * 
 *     The default is usually false.
 */
class IAssetRegistry
{
	GENERATED_IINTERFACE_BODY()
public:
	static IAssetRegistry* Get()
	{
		return UE::AssetRegistry::Private::IAssetRegistrySingleton::Get();
	}
	static IAssetRegistry& GetChecked()
	{
		IAssetRegistry* Singleton = UE::AssetRegistry::Private::IAssetRegistrySingleton::Get();
		check(Singleton);
		return *Singleton;
	}

	/**
	 * Does the given path contain assets, optionally also testing sub-paths?
	 *
	 * @param PackagePath the path to query asset data in (eg, /Game/MyFolder)
	 * @param bRecursive if true, the supplied path will be tested recursively
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category = "AssetRegistry")
	virtual bool HasAssets(const FName PackagePath, const bool bRecursive = false) const = 0;

	/**
	 * Gets asset data for the assets in the package with the specified package name
	 *
	 * @param PackageName the package name for the requested assets (eg, /Game/MyFolder/MyAsset)
	 * @param OutAssetData the list of assets in this path
	 * @param bIncludeOnlyOnDiskAssets If true, use only DiskGatheredData, do not calculate from UObjects.
	 *        @see IAssetRegistry class header for bIncludeOnlyOnDiskAssets.
	 * @param bSkipARFilteredAssets If true, skips Objects that return true for IsAsset but are not assets in the current platform.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="AssetRegistry")
	virtual bool GetAssetsByPackageName(FName PackageName, TArray<FAssetData>& OutAssetData,
		bool bIncludeOnlyOnDiskAssets = false, bool bSkipARFilteredAssets=true) const = 0;

	/**
	 * Gets asset data for all assets in the supplied folder path
	 *
	 * @param PackagePath the path to query asset data in (eg, /Game/MyFolder)
	 * @param OutAssetData the list of assets in this path
	 * @param bIncludeOnlyOnDiskAssets If true, use only DiskGatheredData, do not calculate from UObjects.
	 *        @see IAssetRegistry class header for bIncludeOnlyOnDiskAssets.
	 * @param bRecursive if true, all supplied paths will be searched recursively
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category = "AssetRegistry")
	virtual bool GetAssetsByPath(FName PackagePath, TArray<FAssetData>& OutAssetData, bool bRecursive = false,
		bool bIncludeOnlyOnDiskAssets = false) const = 0;

	/**
	 * Gets asset data for all assets in any of the supplied folder paths
	 *
	 * @param PackagePaths the paths to query asset data in (eg, /Game/MyFolder)
	 * @param OutAssetData the list of assets in this path
	 * @param bIncludeOnlyOnDiskAssets If true, use only DiskGatheredData, do not calculate from UObjects.
	 *        @see IAssetRegistry class header for bIncludeOnlyOnDiskAssets.
	 * @param bRecursive if true, all supplied paths will be searched recursively
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "AssetRegistry")
	virtual bool GetAssetsByPaths(TArray<FName> PackagePaths, TArray<FAssetData>& OutAssetData, bool bRecursive = false,
		bool bIncludeOnlyOnDiskAssets = false) const = 0;

	/**
	 * Gets asset data for all assets with the supplied class
	 *
	 * @param ClassPathName the full path of the class name of the assets requested, in a TopLevelAssetPath structure.
	 * @param OutAssetData the list of assets in this path
	 * @param bSearchSubClasses if true, all subclasses of the passed in class will be searched as well
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category = "AssetRegistry")
	virtual bool GetAssetsByClass(FTopLevelAssetPath ClassPathName, TArray<FAssetData>& OutAssetData, bool bSearchSubClasses = false) const = 0;

	/**
	 * Gets asset data for all assets with the supplied tags, regardless of their value
	 *
	 * @param AssetTags the tags associated with the assets requested
	 * @param OutAssetData the list of assets with any of the given tags
	 */
	virtual bool GetAssetsByTags(const TArray<FName>& AssetTags, TArray<FAssetData>& OutAssetData) const = 0;

	/**
	 * Gets asset data for all assets with the supplied tags and values
	 *
	 * @param AssetTagsAndValues the tags and values associated with the assets requested
	 * @param OutAssetData the list of assets with any of the given tags and values
	 */
	virtual bool GetAssetsByTagValues(const TMultiMap<FName, FString>& AssetTagsAndValues, TArray<FAssetData>& OutAssetData) const = 0;

	/**
	 * Gets asset data for all assets that match the filter.
	 * Assets returned must satisfy every filter component if there is at least one element in the component's array.
	 * Assets will satisfy a component if they match any of the elements in it.
	 *
	 * @param Filter filter to apply to the assets in the AssetRegistry
	 * @param OutAssetData the list of assets in this path
	 * @param bSkipARFilteredAssets If true, skips Objects that return true for IsAsset but are not assets in the current platform.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="AssetRegistry")
	virtual bool GetAssets(const FARFilter& Filter, TArray<FAssetData>& OutAssetData, bool bSkipARFilteredAssets=true) const = 0;
	virtual bool GetAssets(const FARCompiledFilter& Filter, TArray<FAssetData>& OutAssetData, bool bSkipARFilteredAssets=true) const = 0;

	/**
	 * Gets asset data for in-memory assets only, that match the filter.
	 * Returns assets that would be excluded by calling GetAssets with bIncludeOnlyOnDiskAssets set to true.
	 * @note This method IGNORES the value of bIncludeOnlyOnDiskAssets on the provided filter. 
	 * Assets returned must satisfy every filter component if there is at least one element in the component's array.
	 * Assets will satisfy a component if they match any of the elements in it.
	 *
	 * @param Filter filter to apply to the assets in the AssetRegistry
	 * @param OutAssetData the list of assets in this path
	 * @param bSkipARFilteredAssets If true, skips Objects that return true for IsAsset but are not assets in the current platform.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="AssetRegistry")
	virtual bool GetInMemoryAssets(const FARFilter& Filter, TArray<FAssetData>& OutAssetData, bool bSkipARFilteredAssets=true) const = 0;
	virtual bool GetInMemoryAssets(const FARCompiledFilter& Filter, TArray<FAssetData>& OutAssetData, bool bSkipARFilteredAssets=true) const = 0;

	/**
	 * Enumerate asset data for all assets that match the filter.
	 * Assets returned must satisfy every filter component if there is at least one element in the component's array.
	 * Assets will satisfy a component if they match any of the elements in it.
	 *
	 * @param Filter filter to apply to the assets in the AssetRegistry
	 * @param Callback function to call for each asset data enumerated
	 * @param InEnumerateFlags flags to control enumeration and filtering.
	 *        @see EEnumerateAssetsFlags.
	 *        EEnumerateAssetsFlags::Parallel note: in-memory assets enumeration ignores the parallel
	 *        flag and is always single-threaded.
	 * @return False if filter is invalid, otherwise true.
	 */
	virtual bool EnumerateAssets(const FARFilter& Filter, TFunctionRef<bool(const FAssetData&)> Callback,
		UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags = UE::AssetRegistry::EEnumerateAssetsFlags::None) const = 0;
	virtual bool EnumerateAssets(const FARCompiledFilter& Filter, TFunctionRef<bool(const FAssetData&)> Callback,
		UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags = UE::AssetRegistry::EEnumerateAssetsFlags::None) const = 0;

	UE_DEPRECATED(5.6, "Asset path FNames have been deprecated, use Soft Object Path instead.")
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="AssetRegistry", meta=(DeprecatedFunction="Note", DeprecationMessage="Use other version that takes a SoftObjectPath"))
	virtual FAssetData GetAssetByObjectPath( const FName ObjectPath, bool bIncludeOnlyOnDiskAssets = false ) const = 0;

	/**
	 * Gets the asset data for the specified object path
	 *
	 * @param ObjectPath the path of the object to be looked up
	 * @param bIncludeOnlyOnDiskAssets If true, use only DiskGatheredData, do not calculate from UObjects.
	 *        @see IAssetRegistry class header for bIncludeOnlyOnDiskAssets.
	 * @param bSkipARFilteredAssets If true, skips Objects that return true for IsAsset but are not assets in the current platform.
	 * @return the assets data;Will be invalid if object could not be found
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category = "AssetRegistry", DisplayName="Get Asset By Object Path")
	ASSETREGISTRY_API virtual FAssetData K2_GetAssetByObjectPath(const FSoftObjectPath& ObjectPath,
		bool bIncludeOnlyOnDiskAssets = false, bool bSkipARFilteredAssets = true) const;

	/**
	 * Gets the asset data for the specified object path
	 *
	 * @param ObjectPath the path of the object to be looked up
	 * @param bIncludeOnlyOnDiskAssets If true, use only DiskGatheredData, do not calculate from UObjects.
	 *        @see IAssetRegistry class header for bIncludeOnlyOnDiskAssets.
	 * @param bSkipARFilteredAssets If true, skips Objects that return true for IsAsset but are not assets in the current platform.
	 * @return the assets data;Will be invalid if object could not be found
	 */
	virtual FAssetData GetAssetByObjectPath(const FSoftObjectPath& ObjectPath,
		bool bIncludeOnlyOnDiskAssets = false, bool bSkipARFilteredAssets = true) const = 0;

	/**
	 * Tries to get the asset data for the specified object path
	 * 
	 * @param ObjectPath the path of the object to be looked up
	 * @param OutAssetData out FAssetData 
	 * @return Enum return code
	 */
	virtual UE::AssetRegistry::EExists TryGetAssetByObjectPath(const FSoftObjectPath& ObjectPath,
		FAssetData& OutAssetData) const = 0;

	/**
	 * Tries to get the pacakge data for a specified path
	 *
	 * @param PackageName name of the package
	 * @param OutAssetPackageData out FAssetPackageData
	 * @param bFailIfLockHeld If the AssetRegistry is currently being written to by another thread this 
	 *	      function will return EExists::Unknown rather than waiting for safe read access.
	 * @return Enum return code
	 */
	virtual UE::AssetRegistry::EExists TryGetAssetPackageData(FName PackageName,
		FAssetPackageData& OutAssetPackageData, bool bFailIfLockHeld = false) const = 0;

	/**
	 * Tries to get the package data for the specified package name. If found, OutCorrectCasePackageName
	 * will be populated with the PackageName that matches the casing used by the filesystem
	 * 
	 * @param PackageName name of the package
	 * @param OutAssetPackageData out FAssetPackageData
	 * @param OutCorrectCasePackageName out FName matching filesystem casing
	 * @param bFailIfLockHeld If the AssetRegistry is currently being written to by another thread this 
	 *	      function will return EExists::Unknown rather than waiting for safe read access.
	 * @return Return code enum
	 */
	virtual UE::AssetRegistry::EExists TryGetAssetPackageData(FName PackageName,
		FAssetPackageData& OutAssetPackageData, FName& OutCorrectCasePackageName, bool bFailIfLockHeld = false) const = 0;
	
	/**
	 * Gets asset data for all assets in the registry.
	 * This method may be slow, use a filter if possible to avoid iterating over the entire registry.
	 *
	 * @param OutAssetData the list of assets in this path
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category = "AssetRegistry")
	virtual bool GetAllAssets(TArray<FAssetData>& OutAssetData, bool bIncludeOnlyOnDiskAssets = false) const = 0;

	/**
	 * Enumerate asset data for all assets currently cached in the asset registry. 
	 * This method may be slow, use a filter if possible to avoid iterating over the entire registry.
	 * 
	 * @param Callback function to call for each asset data enumerated
	 * @param InEnumerateFlags flags to control enumeration and filtering.
	 *        @see IAssetRegistry class header for EEnumerateAssetsFlags.
	 *        EEnumerateAssetsFlags::Parallel note: in-memory assets enumeration ignores the parallel
	 *        flag and is always single-threaded.
	 */
	virtual bool EnumerateAllAssets(TFunctionRef<bool(const FAssetData&)> Callback,
		UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags = UE::AssetRegistry::EEnumerateAssetsFlags::None) const = 0;

	/**
	 * Gets the LongPackageName for all packages with the given PackageName.
	 * Call to check existence of a LongPackageName or find all packages with a ShortPackageName.
	 * 
	 * @param PackageName Name of the package to find, may be a LongPackageName or ShortPackageName.
	 * @param OutPackageNames All discovered matching LongPackageNames are appended to this array.
	 */
	virtual void GetPackagesByName(FStringView PackageName, TArray<FName>& OutPackageNames) const = 0;

	/**
	 * Returns the first LongPackageName found for the given PackageName.
	 * Issues a warning and returns the first (sorted lexically) if there is more than one.
	 * Call to check existence of a LongPackageName or find a package with a ShortPackageName.
	 *
	 * @param PackageName Name of the package to find, may be a LongPackageName or ShortPackageName.
	 * @return The first LongPackageName of the matching package, or NAME_None if not found.
	 */
	virtual FName GetFirstPackageByName(FStringView PackageName) const = 0;

	/**
	 * Gets a list of AssetIdentifiers or FAssetDependencies that are referenced by the supplied AssetIdentifier.
	 * Only returns dependencies reported in the on-disk package.
	 *
	 * @param AssetIdentifier The name of the package/name for which to gather dependencies.
	 * @param OutDependencies A list of things that are referenced by AssetIdentifier.
	 * @param Category Which category(ies) of dependencies to include in the output list. Dependencies matching ANY
	 *        of the OR'd categories will be returned.
	 * @param Flags	Which flags are required present or not present on the dependencies. Dependencies matching ALL
	 *        required and NONE excluded bits will be returned. For each potentially returned dependency, flags not
	 *        applicable to their category are ignored.
	 */
	virtual bool GetDependencies(const FAssetIdentifier& AssetIdentifier, TArray<FAssetIdentifier>& OutDependencies,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const = 0;
	virtual bool GetDependencies(const FAssetIdentifier& AssetIdentifier, TArray<FAssetDependency>& OutDependencies,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const = 0;

	/**
	 * Gets a list of PackageNames that are referenced by the supplied package.
	 * Only returns dependencies reported in the on-disk package.
	 *
	 * @param PackageName The name of the package for which to gather dependencies (eg, /Game/MyFolder/MyAsset)
	 * @param OutDependencies A list of packages that are referenced by the package whose path is PackageName
	 * @param Category Which category(ies) of dependencies to include in the output list. Dependencies matching
	 *        ANY of the OR'd categories will be returned.
	 * @param Flags	Which flags are required present or not present on the dependencies. Dependencies matching ALL
	 *        required and NONE excluded bits will be returned. For each potentially returned dependency, flags not
	 *        applicable to their category are ignored.
	 */
	virtual bool GetDependencies(FName PackageName, TArray<FName>& OutDependencies,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::Package,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const = 0;

	/**
	 * Return whether the supplied package references the query package. Only considers dependencies reported in the
	 * on-disk package.
	 *
	 * @param PackageName The name of the package for which to inspect dependencies (eg, /Game/MyFolder/MyAsset)
	 * @param QueryDependencyName Which package to look for in PackageName's dependencies
	 * @param Category Which category(ies) of dependencies to look in for QueryDependencyName. Dependencies matching
	 *        ANY of the OR'd categories will be considered.
	 * @param Flags Which flags are required present or not present on the dependencies. Dependencies matching ALL
	 *        required and NONE excluded bits will be considered. For each potentially considered dependency, flags not
	 *        applicable to their category are ignored.
	 */
	virtual bool ContainsDependency(FName PackageName, FName QueryDependencyName,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::Package,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const = 0;

	/**
	 * Gets a list of paths to objects that are referenced by the supplied package. (On disk references ONLY)
	 *
	 * @param PackageName The name of the package for which to gather dependencies (eg, /Game/MyFolder/MyAsset)
	 * @param DependencyOptions Which kinds of dependencies to include in the output list
	 * @param OutDependencies A list of packages that are referenced by the package whose path is PackageName
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category = "AssetRegistry",
		meta=(DisplayName="Get Dependencies", ScriptName="GetDependencies"))
	virtual bool K2_GetDependencies(FName PackageName, const FAssetRegistryDependencyOptions& DependencyOptions,
		TArray<FName>& OutDependencies) const;

	/**
	 * Gets a list of AssetIdentifiers or FAssetDependencies that reference the supplied AssetIdentifier.
	 * Only returns referencers reported in the on-disk package.
	 *
	 * @param AssetIdentifier The name of the package/name for which to gather referencers (eg, /Game/MyFolder/MyAsset)
	 * @param OutReferencers A list of things that reference AssetIdentifier.
	 * @param Category Which category(ies) of referencers to include in the output list. Referencers that have a
	 *        dependency matching ANY of the OR'd categories will be returned.
	 * @param Flags Which flags are required present or not present on the referencer's dependency. Referencers that
	 *        have a dependency matching ALL required and NONE excluded bits will be returned. For each potentially
	 *        returned dependency, flags not applicable to their category are ignored.
	 */
	virtual bool GetReferencers(const FAssetIdentifier& AssetIdentifier, TArray<FAssetIdentifier>& OutReferencers,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const = 0;
	virtual bool GetReferencers(const FAssetIdentifier& AssetIdentifier, TArray<FAssetDependency>& OutReferencers,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const = 0;

	/**
	 * Gets a list of PackageNames that reference the supplied package. (On disk references ONLY)
	 *
	 * @param PackageName The name of the package for which to gather dependencies (eg, /Game/MyFolder/MyAsset)
	 * @param OutReferencers A list of packages that reference the package whose path is PackageName
	 * @param Category Which category(ies) of referencers to include in the output list. Referencers that have a
	 *        dependency matching ANY of the OR'd categories will be returned.
	 * @param Flags Which flags are required present or not present on the referencer's dependency. Referencers that
	 *         have a dependency matching ALL required and NONE excluded bits will be returned. For each potentially
	 *         returned dependency, flags not applicable to their category are ignored.
	 */
	virtual bool GetReferencers(FName PackageName, TArray<FName>& OutReferencers,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::Package,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const = 0;

	/**
	 * Gets a list of packages that reference the supplied package. (On disk references ONLY)
	 *
	 * @param PackageName		the name of the package for which to gather dependencies (eg, /Game/MyFolder/MyAsset)
	 * @param ReferenceOptions	which kinds of references to include in the output list
	 * @param OutReferencers	a list of packages that reference the package whose path is PackageName
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category = "AssetRegistry",
		meta=(DisplayName="Get Referencers", ScriptName="GetReferencers"))
	virtual bool K2_GetReferencers(FName PackageName, const FAssetRegistryDependencyOptions& ReferenceOptions,
		TArray<FName>& OutReferencers) const;

	/** Finds Package data for a package name. This data is only updated on save and can only be accessed for valid packages */
	virtual TOptional<FAssetPackageData> GetAssetPackageDataCopy(FName PackageName) const = 0;

	/** Find Package Data for the packages name. This data is only updated on save and can only be accessed for valid packages */
	virtual TArray<TOptional<FAssetPackageData>> GetAssetPackageDatasCopy(TArrayView<FName> PackageNames) const = 0;

	/**
	 * Enumerate all PackageDatas in the AssetRegistry. The callback is called from within the AssetRegistry's lock, so it must not call
	 * arbitrary code that could call back into the AssetRegistry; doing so would deadlock.
	 * 
	 * @param Callback function to call for each package data enumerated
	 * @param InEnumerateFlags flags to control enumeration
	 *        @see EEnumeratePackagesFlags
	 */
	virtual void EnumerateAllPackages(TFunctionRef<void(FName PackageName, const FAssetPackageData& PackageData)> Callback,
		UE::AssetRegistry::EEnumeratePackagesFlags InEnumerateFlags = UE::AssetRegistry::EEnumeratePackagesFlags::None) const = 0;

	/*
	* Checks if the specified package exists on disk. If the package exists and the optional parameters are provided, 
	* OutCorrectCasePackageName will be populated with the case matching package name found on disk and 
	* OutExtension will be populated with the extension for the package (dot included) if the package contains assets
	* 
	* @param PackageName				the name of the package to check for existence on disk (eg, /Game/MyFolder/MyAsset)
	* @param OutCorrectCasePackageName 	out package name matching the case on disk
	* @param OutExtension				out extension for packages containing assets (dot included)
	*/
	virtual bool DoesPackageExistOnDisk(FName PackageName, FString* OutCorrectCasePackageName = nullptr,
		FString* OutExtension = nullptr) const = 0;

	/**
	 * Uses the asset registry to look for ObjectRedirectors. This will follow the chain of redirectors. It will
	 * return the original path if no redirectors are found.
	 */
	virtual FSoftObjectPath GetRedirectedObjectPath(const FSoftObjectPath& ObjectPath) = 0;

	/**
	 * Returns true if the specified ClassName's ancestors could be found. If so, OutAncestorClassNames is a list of
	 * all its ancestors. This can be slow if temporary caching mode is not on.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "AssetRegistry",
		meta=(DisplayName = "GetAncestorClassNames", ScriptName = "GetAncestorClassNames"))
	virtual bool GetAncestorClassNames(FTopLevelAssetPath ClassPathName,
		TArray<FTopLevelAssetPath>& OutAncestorClassNames) const = 0;

	/**
	 * Returns the names of all classes derived by the supplied class names, excluding any classes matching the
	 * excluded class names. This can be slow if temporary caching mode is not on.
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "AssetRegistry",
		meta=(DisplayName = "GetDerivedClassNames", ScriptName = "GetDerivedClassNames"))
	virtual void GetDerivedClassNames(const TArray<FTopLevelAssetPath>& ClassNames,
		const TSet<FTopLevelAssetPath>& ExcludedClassNames, TSet<FTopLevelAssetPath>& OutDerivedClassNames) const = 0;
	
	/** Gets a list of all paths that are currently cached */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category = "AssetRegistry")
	virtual void GetAllCachedPaths(TArray<FString>& OutPathList) const = 0;

	/** Enumerate all the paths that are currently cached */
	virtual void EnumerateAllCachedPaths(TFunctionRef<bool(FString)> Callback) const = 0;

	/** Enumerate all the paths that are currently cached */
	virtual void EnumerateAllCachedPaths(TFunctionRef<bool(FName)> Callback) const = 0;

	/** Gets a list of all paths that are currently cached below the passed-in base path */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category = "AssetRegistry")
	virtual void GetSubPaths(const FString& InBasePath, TArray<FString>& OutPathList, bool bInRecurse) const = 0;

	/** Gets a list of all paths by name that are currently cached below the passed-in base path */
	virtual void GetSubPaths(const FName& InBasePath, TArray<FName>& OutPathList, bool bInRecurse) const = 0;

	/** Enumerate the all paths that are currently cached below the passed-in base path */
	virtual void EnumerateSubPaths(const FString& InBasePath, TFunctionRef<bool(FString)> Callback, bool bInRecurse) const = 0;

	/** Enumerate the all paths that are currently cached below the passed-in base path */
	virtual void EnumerateSubPaths(const FName InBasePath, TFunctionRef<bool(FName)> Callback, bool bInRecurse) const = 0;

	/** Trims items out of the asset data list that do not pass the supplied filter */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category = "AssetRegistry")
	virtual void RunAssetsThroughFilter(UPARAM(ref) TArray<FAssetData>& AssetDataList, const FARFilter& Filter) const = 0;

	/** Trims items out of the asset data list that pass the supplied filter */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category = "AssetRegistry")
	virtual void UseFilterToExcludeAssets(UPARAM(ref) TArray<FAssetData>& AssetDataList, const FARFilter& Filter) const = 0;

	/** Trims items out of the asset data list that pass the supplied filter */
	virtual void UseFilterToExcludeAssets(TArray<FAssetData>& AssetDataList, const FARCompiledFilter& CompiledFilter) const = 0;
	
	/** Tests to see whether the given asset would be included (passes) the given filter */
	virtual bool IsAssetIncludedByFilter(const FAssetData& AssetData, const FARCompiledFilter& Filter) const = 0;

	/** Tests to see whether the given asset would be excluded (fails) the given filter */
	virtual bool IsAssetExcludedByFilter(const FAssetData& AssetData, const FARCompiledFilter& Filter) const = 0;

	/** Modifies passed in filter optimize it for query and expand any recursive paths and classes */
	virtual void CompileFilter(const FARFilter& InFilter, FARCompiledFilter& OutCompiledFilter) const = 0;

	/**
	 * Enables or disable temporary search caching, when this is enabled scanning/searching is faster because we
	 * assume no objects are loaded between scans. Disabling frees any caches created.
	 */
	virtual void SetTemporaryCachingMode(bool bEnable) = 0;
	/**
	 * Mark that the temporary cached needs to be updated before being used again, because e.g. a new class was loaded.
	 * Does nothing if TemporaryCachingMode is not enabled
	 */
	virtual void SetTemporaryCachingModeInvalidated() = 0;

	/** Returns true if temporary caching mode enabled */
	virtual bool GetTemporaryCachingMode() const = 0;

	/**
	 * Gets the current availability of an asset, primarily for streaming install purposes.
	 *
	 * @param FAssetData the asset to check for availability
	 */
	virtual EAssetAvailability::Type GetAssetAvailability(const FAssetData& AssetData) const = 0;

	/**
	 * Gets an ETA or percentage complete for an asset that is still in the process of being installed.
	 *
	 * @param FAssetData the asset to check for progress status
	 * @param ReportType the type of report to query.
	 */
	virtual float GetAssetAvailabilityProgress(const FAssetData& AssetData,
		EAssetAvailabilityProgressReportingType::Type ReportType) const = 0;

	/**
	 * @param ReportType The report type to query.
	 * Returns if a given report type is supported on the current platform
	 */
	virtual bool GetAssetAvailabilityProgressTypeSupported(EAssetAvailabilityProgressReportingType::Type ReportType) const = 0;	

	/**
	 * Hint the streaming installers to prioritize a specific asset for install.
	 *
	 * @param FAssetData the asset which needs to have installation prioritized
	 */
	virtual void PrioritizeAssetInstall(const FAssetData& AssetData) const = 0;

	/**
	 * Returns whether the supplied folder contains any verse file
	 * 
	 * @param PackagePath the path to query asset data in (e.g. /Game/MyFolder)
	 * @param bRecursive if true, the supplied path will be searched recursively
	 * @return True if any verse file is found under the supplied folder
	 */
	virtual bool HasVerseFiles(FName PackagePath, bool bRecursive = false) const = 0;

	/**
	 * Gets paths for all Verse files in the supplied folder
	 *
	 * @param PackagePath the path to query asset data in (e.g. /Game/MyFolder)
	 * @param OutFilePaths the list of Verse files in this path, as pseudo UE LongPackagePaths with extension
	 *       (e.g. /Game/MyFolder/MyVerseFile.verse)
	 * @param bRecursive if true, the supplied path will be searched recursively
	 */
	virtual bool GetVerseFilesByPath(FName PackagePath, TArray<FName>& OutFilePaths, bool bRecursive = false) const = 0;

	/**
	 * Adds the specified path to the set of cached paths. These will be returned by GetAllCachedPaths(). Returns
	 * true if the path was actually added and false if it already existed.
	 */
	virtual bool AddPath(const FString& PathToAdd) = 0;

	/**
	 * Attempts to remove the specified path to the set of cached paths. This will only succeed if there are no
	 * assets left in the specified path.
	 */
	virtual bool RemovePath(const FString& PathToRemove) = 0;

	/** Queries whether the given path exists in the set of cached paths */
	virtual bool PathExists(const FString& PathToTest) const = 0;
	virtual bool PathExists(const FName PathToTest) const = 0;

	/**
	 * Scan the supplied paths recursively right now and populate the asset registry. If bForceRescan is true, the
	 * paths will be scanned again, even if they were previously scanned.
	 */
	UFUNCTION(BlueprintCallable, Category = "AssetRegistry")
	virtual void ScanPathsSynchronous(const TArray<FString>& InPaths, bool bForceRescan = false,
		bool bIgnoreDenyListScanFilters = false) = 0;

	/**
	 * Scan the specified individual files right now and populate the asset registry. If bForceRescan is true, the
	 * paths will be scanned again, even if they were previously scanned.
	 */
	UFUNCTION(BlueprintCallable, Category = "AssetRegistry")
	virtual void ScanFilesSynchronous(const TArray<FString>& InFilePaths, bool bForceRescan = false) = 0;

	/** Scan the supplied paths and files recursively right now and populate the asset registry. */
	virtual void ScanSynchronous(const TArray<FString>& InPaths, const TArray<FString>& InFilePaths,
		UE::AssetRegistry::EScanFlags InScanFlags = UE::AssetRegistry::EScanFlags::None) = 0;

	/** Look for all assets on disk (can be async or synchronous) */
	UFUNCTION(BlueprintCallable, Category = "AssetRegistry")
	virtual void SearchAllAssets(bool bSynchronousSearch) = 0;

	/**
	 * Whether SearchAllAssets has been called, or was auto-called at startup. When async (editor or cooking), if
	 * SearchAllAssets has ever been called, any newly-mounted directory will be automatically searched.
	 */
	UFUNCTION(BlueprintCallable, Category = "AssetRegistry")
	virtual bool IsSearchAllAssets() const = 0;

	/**
	 * Whether searching is done async (and was started at startup), or synchronously and on-demand, requiring
	 * ScanPathsSynchronous or SearchAllAssets.
	 */
	UFUNCTION(BlueprintCallable, Category = "AssetRegistry")
	virtual bool IsSearchAsync() const = 0;

	/**
	 * Wait for scan to be complete. If called during editor startup before OnPostEngineInit, and there are assets with
	 * classes in not-yet-loaded plugin modules, WaitForCompletion returns silently with those assets still ungathered.
	 */
	UFUNCTION(BlueprintCallable, Category = "AssetRegistry")
	virtual void WaitForCompletion() = 0;

	/**
	 * Wait for premade asset registry load to be completed, for use when querying the asset registry prior to flushing
	 * in ScanPathSynchronous. Will no-op if no such asset registry exists.
	 */
	virtual void WaitForPremadeAssetRegistry() = 0;

	/**
	 * Empty the global gatherer's cache and disable further caching of scans from disk.
	 * Used to save memory when cooking after the scan is complete.
	*/
	virtual void ClearGathererCache() = 0;

	/** Wait for the scan of a specific package to be complete */
	UFUNCTION(BlueprintCallable, Category = "AssetRegistry")
	virtual void WaitForPackage(const FString& PackageName) = 0;

	/**
	 * If assets are currently being asynchronously scanned in the specified path, this will cause them to be scanned
	 * before other assets.
	 */
	UFUNCTION(BlueprintCallable, Category = "AssetRegistry")
	virtual void PrioritizeSearchPath(const FString& PathToPrioritize) = 0;

	/** Forces a rescan of specific filenames, call this when you need to refresh from disk */
	UFUNCTION(BlueprintCallable, Category = "AssetRegistry")
	virtual void ScanModifiedAssetFiles(const TArray<FString>& InFilePaths) = 0;
	virtual void ScanModifiedAssetFiles(const TArray<FString>& InFilePaths, UE::AssetRegistry::EScanFlags ScanFlags) = 0;

	/** Event for when one or more files have been blocked from the registry */
	DECLARE_EVENT_OneParam( IAssetRegistry, FFilesBlockedEvent, const TArray<FString>& /*Files*/ );
	virtual FFilesBlockedEvent& OnFilesBlocked() = 0;

	/**
	 * Events for when paths (folders/directories) are added to and removed from the asset registry.
	 * Both batch and singular events will be called for the same set of paths in all cases.
	 * Prefer the batched versions as the singular versions will be deprecated in a future release.
	 */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam( FPathsEvent, TConstArrayView<FStringView> /* Paths */);
	DECLARE_TS_MULTICAST_DELEGATE_OneParam( FPathAddedEvent, const FString& /*Path*/ );
	DECLARE_EVENT_OneParam( IAssetRegistry, FPathRemovedEvent, const FString& /*Path*/ );
	virtual FPathsEvent& OnPathsAdded() = 0;
	virtual FPathsEvent& OnPathsRemoved() = 0;
	virtual FPathAddedEvent& OnPathAdded() = 0;
	virtual FPathRemovedEvent& OnPathRemoved() = 0;

	/** Informs the asset registry that an in-memory asset has been created */
	virtual void AssetCreated(UObject* NewAsset) = 0;

	/** Informs the asset registry that an in-memory asset has been deleted */
	virtual void AssetDeleted(UObject* DeletedAsset) = 0;

	/** Informs the asset registry that an in-memory asset has been renamed */
	virtual void AssetRenamed(const UObject* RenamedAsset, const FString& OldObjectPath) = 0;

	/**
	 * Called during SavePackage to update the AssetRegistry's copy of the AssetDatas in the package to match the
	 * newly saved values.
	 */
	virtual void AssetsSaved(TArray<FAssetData>&& Assets) = 0;

	/**
	 * Called on demand from systems that need to fully update an AssetData's tags. When an Asset is loaded its tags
	 * are updated by calling GetAssetRegistryTags(EAssetRegistryTagsCaller::AssetRegistryLoad), but that version of
	 * the function is allowed to skip writing expensive tags, so tags that exist in the old AssetData but not in the
	 * results from GetAssetRegistryTags(EAssetRegistryTagsCaller::AssetRegistryLoad) are kept because they might be
	 * skipped expensive tags. When an asset is saved, all old tags are deleted and 
	 * GetAssetRegistryTags(EAssetRegistryTagsCaller::SavePackage) is called. AssetUpdateTags allows a manual trigger
	 * of the on-SavePackage behavior: all old tags are deleted and
	 * GetAssetRegistryTags(Caller) is called. Pass in EAssetRegistryTagsCaller::FullUpdate to behave the same as
	 * SavePackage.
	 */
	virtual void AssetUpdateTags(UObject* Object, EAssetRegistryTagsCaller Caller) = 0;

	/** Informs the asset registry that a Verse file has been created on disk */
	virtual bool VerseCreated(const FString& FilePathOnDisk) = 0;

	/** Informs the asset registry that a Verse file has been deleted from disk */
	virtual bool VerseDeleted(const FString& FilePathOnDisk) = 0;

	/** Informs the asset registry that an in-memory package has been deleted, and all associated assets should be removed */
	virtual void PackageDeleted (UPackage* DeletedPackage) = 0;

	/** Informs the asset registry that an Asset has finalized its tags after loading. Ignored if the Asset's package has been modified. */
	virtual void AssetTagsFinalized(const UObject& FinalizedAsset) = 0;

	DECLARE_TS_MULTICAST_DELEGATE_OneParam( FAssetsEvent, TConstArrayView<FAssetData> );

	/** 
	* EXPERIMENTAL: event for when duplicated assets are found and needs to be resolved, this is experimental as it's
	* a counter measure for a situation that shouldn't happen. Also, DO NOT CALL AssetRegistry functions from delegate
	* subscribers; doing so will create a deadlock.
	*/
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams( FAssetCollisionEvent, FAssetData&, FAssetData&, FAssetData*& );
	virtual FAssetCollisionEvent& OnAssetCollision_Private() = 0;

	/** Event for when assets are added to the registry */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam( FAssetAddedEvent, const FAssetData& );
	virtual FAssetAddedEvent& OnAssetAdded() = 0;

	/** Event for when assets are added to the registry to handle batches more efficiently */
	virtual FAssetsEvent& OnAssetsAdded() = 0;

	/** Event for when assets are removed from the registry */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam( FAssetRemovedEvent, const FAssetData& );
	virtual FAssetRemovedEvent& OnAssetRemoved() = 0;

	/** Event for when assets are removed from the registry */
	virtual FAssetsEvent& OnAssetsRemoved() = 0;

	/** Event for when assets are renamed in the registry */
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams( FAssetRenamedEvent, const FAssetData&, const FString& );
	virtual FAssetRenamedEvent& OnAssetRenamed() = 0;

	/** Event for when assets are updated in the registry */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam( FAssetUpdatedEvent, const FAssetData&);
	virtual FAssetUpdatedEvent& OnAssetUpdated() = 0;

	/** Event for when assets are updated in the registry */
	virtual FAssetsEvent& OnAssetsUpdated() = 0;

	/** Event for when assets are updated on disk and have been refreshed in the assetregistry */
	virtual FAssetUpdatedEvent& OnAssetUpdatedOnDisk() = 0;

	/** Event for when assets are updated on disk and have been refreshed in the assetregistry */
	virtual FAssetsEvent& OnAssetsUpdatedOnDisk() = 0;

	/** Event for when in-memory assets are created */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam( FInMemoryAssetCreatedEvent, UObject* );
	virtual FInMemoryAssetCreatedEvent& OnInMemoryAssetCreated() = 0;

	/** Event for when assets are deleted */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam( FInMemoryAssetDeletedEvent, UObject* );
	virtual FInMemoryAssetDeletedEvent& OnInMemoryAssetDeleted() = 0;

	/** Event for when Verse files are added to the registry */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam( FVerseAddedEvent, const FName /* VerseFilePath */);
	virtual FVerseAddedEvent& OnVerseAdded() = 0;

	/** Event for when Verse files are removed from the registry */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam( FVerseRemovedEvent, const FName /* VerseFilePath */);
	virtual FVerseRemovedEvent& OnVerseRemoved() = 0;

	/** Event for when the asset registry is done loading files */
	DECLARE_TS_MULTICAST_DELEGATE( FFilesLoadedEvent );
	virtual FFilesLoadedEvent& OnFilesLoaded() = 0;

	DECLARE_TS_MULTICAST_DELEGATE(FScanStartedEvent);
	virtual FScanStartedEvent& OnScanStarted() = 0;

	DECLARE_TS_MULTICAST_DELEGATE(FScanEndedEvent);
	virtual FScanEndedEvent& OnScanEnded() = 0;

	/** Event for when the asset registry is done with all currently known gathers. This will fire after both the
	 *  initial asset scan and again after each subsequent batch of additional mount searches
	 */
	DECLARE_TS_MULTICAST_DELEGATE(FKnownGathersCompleteEvent);
	virtual FKnownGathersCompleteEvent& OnKnownGathersComplete() = 0;

	/** Payload data for a file progress update */
	struct FFileLoadProgressUpdateData
	{
		FFileLoadProgressUpdateData(int32 InNumTotalAssets, int32 InNumAssetsProcessedByAssetRegistry,
			int32 InNumAssetsPendingDataLoad, bool InIsDiscoveringAssetFiles)
			: NumTotalAssets(InNumTotalAssets)
			, NumAssetsProcessedByAssetRegistry(InNumAssetsProcessedByAssetRegistry)
			, NumAssetsPendingDataLoad(InNumAssetsPendingDataLoad)
			, bIsDiscoveringAssetFiles(InIsDiscoveringAssetFiles)
		{
		}

		bool operator==(const FFileLoadProgressUpdateData& Other) const
		{
			return NumTotalAssets == Other.NumTotalAssets 
				&& NumAssetsProcessedByAssetRegistry == Other.NumAssetsProcessedByAssetRegistry
				&& NumAssetsPendingDataLoad == Other.NumAssetsPendingDataLoad
				&& bIsDiscoveringAssetFiles == Other.bIsDiscoveringAssetFiles;
		}

		int32 NumTotalAssets;
		int32 NumAssetsProcessedByAssetRegistry;
		int32 NumAssetsPendingDataLoad;
		bool bIsDiscoveringAssetFiles;
	};

	/** Event to update the progress of the background file load */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam( FFileLoadProgressUpdatedEvent, const FFileLoadProgressUpdateData& /*ProgressUpdateData*/ );
	virtual FFileLoadProgressUpdatedEvent& OnFileLoadProgressUpdated() = 0;

	/** Returns true if the asset registry is currently loading files and does not yet know about all assets.
	* This is a legacy function that only returns true when the asset registry is performing its initial search for assets.
	* Prefer using IsGathering() to know if the asset registry is currently loading files.
	*/
	UFUNCTION(BlueprintCallable, Category = "AssetRegistry")
	virtual bool IsLoadingAssets() const = 0;
	/** Returns true if the asset registry is currently loading files and does not yet know about all assets.
	* IsGathering() can be used to indicate if the asset registry is scanning during the initial search as well as any
	* subsequent search the registry may perform (e.g. when a new mount point is added)
	*/
	virtual bool IsGathering() const = 0;

	/** Returns true if the asset registry has written out the discovery cache after discovery has completed. 
	* This function will return false if discovery is not yet complete, discovery caching isn't supported by the current platform 
	* or cache writing has been disabled such as when using -NoAssetRegistryCacheWrite. */
	virtual bool HasSerializedDiscoveryCache() const = 0;

	/** If true, the AssetRegistry updates its on-disk information for an Asset whenever that Asset loads. */
	virtual bool ShouldUpdateDiskCacheAfterLoad() const = 0;

	/** Tick the asset registry */
	virtual void Tick (float DeltaTime) = 0;

	/** Serialize the registry to/from a file, skipping editor only data */
	virtual void Serialize(FArchive& Ar) = 0;
	virtual void Serialize(FStructuredArchive::FRecord Record) = 0;

	/** Append the assets from the incoming state into our own */
	virtual void AppendState(const FAssetRegistryState& InState,
		UE::AssetRegistry::EAppendMode AppendMode = UE::AssetRegistry::EAppendMode::Append) = 0;

	// ActiveMountsLoading: save memory when using cooked AssetRegistries by removing AssetRegistryData
	// for MountPoints that are unmounted. Only beneficial if your game unloads mountpoints during some periods,
	// and only usable if your game does not require knowledge of the assets in those mountpoints when unloaded.
	// Currently only enablable for ClientOnly, since ClientOnly is the configuration that runs on the most memory
	// constrained platforms. Enable by setting Engine.ini:[AssetRegistry]:bActiveMountsLoadingEnabled_Client=true,
	// and calling IAssetRegistry->ActiveMountsInitialize() when ready to start unloading at startup.
	// By default, ActiveMountsLoading runs in auto mode and unloading is triggered whenever
	// FPackageName::OnMountPointDismounted is called, but alternatively you can set 
	// Engine.ini:[AssetRegistry]:bActiveMountsLoadingAuto_Client=false and call ActiveMountsUnloadAsync and
	// ActiveMountsLoadAsync explicitly.

	/**
	 * Returns true if ActiveMountsIsEnabled in the current configuration. Returns true even if ActiveMountsInitialize
	 * has not yet been called. If not enabled, functions with AssetRegistry side effects will such as
	 * ActiveMountsRegister* will still execute the side-effects, but all behavior specific to unloading and reloading
	 * will be skipped by the ActiveMounts functions.
	 */
	virtual bool ActiveMountsIsEnabled() = 0;
	/**
	 * Called once per process to activate ActiveMountsLoading. Calls to ActiveMountsLoadAsync/ActiveMountsUnloadAsync
	 * before this call are ignored.
	 * This function also forces the parsing of the premade AssetRegistry: MountPoints in the premade AssetRegistry
	 * state will be searched at this point in FPackageName or FPlugins; any mountpoints used by the premade that are
	 * not mounted and not findable as a plugin should be registered before this via FPackageName::FindOrAddMountPoint.
	 * Any mountpoints not findable will log an error and AssetDatas under them will always be loaded.
	 */
	virtual void ActiveMountsInitialize() = 0;

	/**
	 * Registers an AssetRegistry.bin file on disk for ActiveMountsLoading, and also immediately loads its
	 * data into the AssetRegistry with AppendState. The default premade AssetRegistry.bin is automatically
	 * registered by the AssetRegistry and does not need this call.
	 * If ActiveMountsLoading is disabled, this function behaves the same as AppendState.
	 * Returns with no action if AssetRegistryState can not be loaded from the given file.
	 * This function also forces the parsing of the premade AssetRegistry, @see ActiveMountsInitialize.
	 * 
	 * @return True if the AssetRegistryState could be loaded from the given file, else false.
	 */
	virtual bool ActiveMountsRegisterAndLoadFilePathSynchronous(FStringView StateFilePath) = 0;
	/**
	 * Runs ActiveMountsRegisterAndLoadFilePathSynchronous on an async thread and calls OnComplete when done.
	 * OnComplete is called from an async thread, and in game error cases might be called after IAssetRegistry
	 * is shutdown. Caller should handle thread safety and should check for IAssetRegistry::Get != nullptr
	 * if it is used in their callback.
	*/
	virtual void ActiveMountsRegisterAndLoadFilePathAsync(FStringView StateFilePath,
		TFunction<void(bool bSucceeded)> OnComplete = TFunction<void(bool bSucceeded)>(),
		UE::Tasks::ETaskPriority TaskPriority = UE::Tasks::ETaskPriority::Default) = 0;
	/**
	 * Requests that AssetDatas under the given MountPoints (looked up via FPackageName::FindOrAddMountPoint) be
	 * reloaded into the AssetRegistry. This only affects MountPoints previously unloaded via auto unload or 
	 * ActiveMountsUnloadAsync, and is only possible for AssetDatas in the premade AssetRegistry or in an AssetRegistry
	 * file registered via an ActiveMountsRegister function. Calls OnComplete once the load completes.
	 * 
	 * @param OnComplete Optional delegate: void OnComplete(bool bCanceled). Called when the load is complete.
	 *        bCanceled is set to true iff ActiveMountsLoading is not enabled or for any requested MountPoint the
	 *        load failed or was overridden by an unload request. If true, test ActiveMountsIsEnabled and call
	 *        ActiveMountsIsLoaded to test state of each mountpoint.
	 *        OnComplete is called from an async thread, caller should handle thread safety in their callback.
	 */
	virtual void ActiveMountsLoadAsync(TConstArrayView<FStringView> MountPointLongPackageNames,
		TFunction<void(bool bCanceled)> OnComplete = TFunction<void(bool bCanceled)>()) = 0;
	/**
	 * Requests that all AssetDatas that have been unloaded by ActiveMounts be reloaded. Calls OnComplete once the
	 * load completes.
	 *
	 * @param OnComplete Optional delegate: void OnComplete(bool bCanceled). Called when the load is complete.
	 *        bCanceled is set to true iff ActiveMountsLoading is not enabled or for any MountPoint the load failed
	 *        or was overridden by an unload request. 
	 *        OnComplete is called from an async thread, caller should handle thread safety in their callback.
	 */
	virtual void ActiveMountsReloadAllAsync(
		TFunction<void(bool bCanceled)> OnComplete = TFunction<void(bool bCanceled)>()) = 0;
	/**
	 * Requests that AssetDatas under the given MountPoints (looked up via FPackageName::FindOrAddMountPoint) be
	 * unloaded from the AssetRegistry. This is only possible for AssetDatas in the premade AssetRegistry or in an
	 * AssetRegistry file registered via an ActiveMountsRegister function. Calls OnComplete once the unload completes.
	 *
	 * @param OnComplete Optional delegate: void OnComplete(bool bCanceled). Called when the unload is complete.
	 *        bCanceled is set to true iff ActiveMountsLoading is not enabled or for any requested MountPoint the
	 *        unload failed or was overridden by a load request. If true, test ActiveMountsIsEnabled and call
	 *        ActiveMountsIsLoaded to test state of each mountpoint.
	 *        OnComplete is called from an async thread, caller should handle thread safety in their callback.
	 */
	virtual void ActiveMountsUnloadAsync(TConstArrayView<FStringView> MountPointLongPackageNames,
		TFunction<void(bool bCanceled)> OnComplete = TFunction<void(bool bCanceled)>()) = 0;
	/**
	 * Requests that AssetDatas under MountPoints that are available for unloading via the premade
	 * AssetRegistry or in an AssetRegistry registered via an ActiveMountsRegister function be unloaded if they
	 * are not currently mounted with FPackageName. Calls OnComplete once the unload completes.
	 *
	 * @param OnComplete Optional delegate: void OnComplete(bool bCanceled). Called when the unload is complete.
	 *        bCanceled is set to true iff ActiveMountsLoading is not enabled or for any MountPoint the unload failed
	 *        or was overridden by a load request. 
	 *        OnComplete is called from an async thread, caller should handle thread safety in their callback.
	 */
	virtual void ActiveMountsUnloadAllUnmountedAsync(
		TFunction<void(bool bCanceled)> OnComplete = TFunction<void(bool bCanceled)>()) = 0;
	/**
	 * Returns whether the given MountPoint (looked up via FPackageName::FindOrAddMountPoint) is loaded in the
	 * ActiveMounts system. Returns false if currently loading or unloading or if unloaded. Only valid if
	 * ActiveMountsLoader is enabled and the MountPoint is used by the PreMade AssetRegistry or a registered
	 * AssetRegistry file. Returns false for all other cases.
	 */
	virtual bool ActiveMountsIsLoaded(FStringView MountPointLongPackageName) = 0;
	/**
	 * Returns whether ActiveMountsLoading is in progress for any MountPoint. Returns false if ActiveMountsLoading is
	 * not enabled.
	 */
	virtual bool ActiveMountsIsInProgress() = 0;
	/**
	 * Call the given callback when all requested ActiveMountsLoading loads and unloads complete and the system
	 * becomes idle. The callback is called outside of the lock, so by the time it is called, a new request may
	 * have come in and the system might be in progress again. Intended use of this function is to guard entry into a
	 * state that relies on AssetRegistry queries after issuing a set of ActiveMounts loads or unloads.
	 * If the system is already idle (ActiveMountsIsInProgress returns false), the callback is called immediately.
	 * If ActiveMountsLoading is not enabled, the callback is called immediately.
	 * The callback might be called from an async thread; caller should handle thread safety in their callback.
	 */
	virtual void ActiveMountsReportWhenIdle(TFunction<void()> OnIdle) = 0;

	/** Returns memory size of entire registry, optionally logging sizes */
	virtual SIZE_T GetAllocatedSize(bool bLogDetailed = false) const = 0;

	/**
	 * Fills in a AssetRegistryState with a copy of the data in the internal cache, overriding some
	 *
	 * @param OutState			This will be filled in with a copy of the asset data, platform data, and dependency data
	 * @param Options			Serialization options that will be used to write this later
	 * @param bRefreshExisting	If true, will not delete or add packages in OutState and will just update things that already exist
	 * @param RequiredPackages  If non-empty, only packages in this set will be kept.
	 * @param RemovePackages	If non-empty, packages in this set will not be kept.
	 */
	virtual void InitializeTemporaryAssetRegistryState(FAssetRegistryState& OutState, const FAssetRegistrySerializationOptions& Options,
		bool bRefreshExisting = false, const TSet<FName>& RequiredPackages = TSet<FName>(),
		const TSet<FName>& RemovePackages = TSet<FName>()) const = 0;

#if ASSET_REGISTRY_STATE_DUMPING_ENABLED
	/**
	 * Writes out the state in textual form. Use arguments to control which segments to emit.
	 * @param Arguments List of segments to emit. Possible values: 'ObjectPath', 'PackageName', 'Path', 'Class',
	 *        'Tag', 'Dependencies' and 'PackageData'
	 * @param OutPages Textual representation will be written to this array; each entry will have LinesPerPage lines
	 *        of the full dump.
	 * @param LinesPerPage - how many lines should be combined into each string element of OutPages, for e.g. breaking
	 *        up the dump into separate files. To facilitate diffing between similar-but-different registries, the
	 *        actual number of lines per page will be slightly less than LinesPerPage; we introduce partially
	 *        deterministic pagebreaks near the end of each page.
	 */
	virtual void DumpState(const TArray<FString>& Arguments, TArray<FString>& OutPages, int32 LinesPerPage = 1) const = 0;
#endif

	/** Returns the set of empty package names fast iteration */
	virtual TSet<FName> GetCachedEmptyPackagesCopy() const = 0;

	/** Return whether the given TagName occurs in the tags of any asset in the AssetRegistry */
	virtual bool ContainsTag(FName TagName) const = 0;

	/** Fills in FAssetRegistrySerializationOptions from ini, optionally using a target platform ini name */
	UE_DEPRECATED(5.7, "Use version taking ITargetPlatform parameter instead")
	ASSETREGISTRY_API virtual void InitializeSerializationOptions(FAssetRegistrySerializationOptions& Options,
		const FString& PlatformIniName,
		UE::AssetRegistry::ESerializationTarget Target) const;

	/** Fills in FAssetRegistrySerializationOptions from ini, optionally using a target platform */
	virtual void InitializeSerializationOptions(FAssetRegistrySerializationOptions& Options,
		const ITargetPlatform* TargetPlatform = nullptr,
		UE::AssetRegistry::ESerializationTarget Target = UE::AssetRegistry::ESerializationTarget::ForGame) const = 0;

	struct FLoadPackageRegistryData
	{
		ASSETREGISTRY_API FLoadPackageRegistryData(bool bInGetDependencies = false);
		ASSETREGISTRY_API ~FLoadPackageRegistryData();

		TArray<FAssetData> Data;
		TArray<FName> DataDependencies;
		bool bGetDependencies;
	};

	/** Load FPackageRegistry data from the supplied package */
	virtual void LoadPackageRegistryData(FArchive& Ar, FLoadPackageRegistryData& InOutData) const = 0;
	
	/** Load FAssetData from the specified package filename */
	virtual void LoadPackageRegistryData(const FString& PackageFilename, FLoadPackageRegistryData& InOutData) const = 0;

	typedef TFunctionRef<bool(const FAssetData* AssetData)> FAssetDataFunc;
	typedef TFunctionRef<bool(FAssetDataFunc AssetCallback)> FEnumerateAssetDatasFunc;

	/**
	 * Enumerate all pairs in State->TagToAssetDataMapAssetRegistry and call a callback on each pair.
	 * To avoid copies, the callback is called from within the ReadLock.
	 * DO NOT CALL AssetRegistry functions from the callback; doing so will create a deadlock.
	 */
	virtual void ReadLockEnumerateAllTagToAssetDatas(TFunctionRef<bool(FName TagName,
		FEnumerateAssetDatasFunc EnumerateAssets)> Callback) const = 0;

	/**
	 * Predicate called to decide whether to recurse into a reference when setting manager references
	 *
	 * @param Manager			Identifier of what manager will be set
	 * @param Source			Identifier of the reference currently being iterated
	 * @param Target			Identifier that will managed by manager
	 * @param DependencyType	Type of dependencies to recurse over
	 * @param Flags				Flags describing this particular set attempt
	 */
	typedef TFunction<EAssetSetManagerResult::Type(const FAssetIdentifier& Manager, const FAssetIdentifier& Source,
		const FAssetIdentifier& Target, UE::AssetRegistry::EDependencyCategory Category,
		UE::AssetRegistry::EDependencyProperty Properties, EAssetSetManagerFlags::Type Flags)>
		ShouldSetManagerPredicate;

	/** 
	 *	Indicates if path should be beautified before presented to the user.
	 * @param InAssetPath	Path of the asset to check
	 * @return True if the path should be beautified
	 */
	virtual bool IsPathBeautificationNeeded(const FString& InAssetPath) const = 0;

	/** Request to pause background processing of scan results while this object is in scope
	 *  This can be used to allow a priority thread to perform along sequence of operations
	 *  without having to contend with the background thread for data access
	 */
	struct FPauseBackgroundProcessingScope
	{
		FPauseBackgroundProcessingScope() { IAssetRegistry::GetChecked().RequestPauseBackgroundProcessing(); }
		~FPauseBackgroundProcessingScope() { IAssetRegistry::GetChecked().RequestResumeBackgroundProcessing(); }
	};

	/**
	 * Event triggered when a query of the AssetRegistry's on-disk assets is made.
	 * The event is triggered only if the filter is valid.
	 * The event is not triggered for queries of in-memory assets only, such as GetInMemoryAssets.
	 */
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FEnumerateAssetsEvent, const FARCompiledFilter& /*Filter*/, UE::AssetRegistry::EEnumerateAssetsFlags /*Flag*/)
	virtual FEnumerateAssetsEvent& OnEnumerateAssetsEvent() = 0;

	/**
	 * Request to pause paths mounting.
	 * Can be used by the calling code to temporarily pause loading of data inside the AssetRegistry in response
	 * to mounted MountPoints (e.g. ActiveMountsLoading). All mounting requests will be executed in one batch at the
	 * end of that scope.
	 */
	struct FSuspendPathsMounting
	{
		UE_NODISCARD_CTOR FSuspendPathsMounting() { IAssetRegistry::GetChecked().SuspendPathsMounting(); }
		~FSuspendPathsMounting() { IAssetRegistry* AR = IAssetRegistry::Get(); if (AR) AR->ResumePathsMounting(); }
	};

	/**
	 * Request to pause paths dismounting.
	 * Can be used by the calling code to temporarily pause clearing of data inside the AssetRegistry.
	 * All unmounting requests will be executed in one batch at the end of that scope.
	 */
	struct FSuspendPathsDismounting
	{
		UE_NODISCARD_CTOR FSuspendPathsDismounting() { IAssetRegistry::GetChecked().SuspendPathsDismounting(); }
		~FSuspendPathsDismounting() { IAssetRegistry* AR = IAssetRegistry::Get(); if (AR) { AR->ResumePathsDismounting(); } }
	};

protected:

	// Implementation methods for FSuspendPathsMounting and FSuspendPathsDismounting
	virtual void SuspendPathsMounting() {};
	virtual void ResumePathsMounting() {};
	virtual void SuspendPathsDismounting() {};
	virtual void ResumePathsDismounting() {};

	// Implementation methods for FPauseBackgroundProcessingScope
	virtual void RequestPauseBackgroundProcessing() {}
	virtual void RequestResumeBackgroundProcessing() {}

	// Functions specifically for calling from the asset manager
	friend class UAssetManager;

	UE_DEPRECATED(5.6, "Use version that takes FSetManageReferencesContext.")
	virtual void SetManageReferences(const TMultiMap<FAssetIdentifier, FAssetIdentifier>& ManagerMap,
		bool bClearExisting, UE::AssetRegistry::EDependencyCategory RecurseType,
		TSet<FDependsNode*>& ExistingManagedNodes, ShouldSetManagerPredicate ShouldSetManager = nullptr) = 0;

	/**
	 * Specifies a list of manager mappings, optionally recursing to dependencies. These mappings can then be queried
	 * later to see which assets "manage" other assets. This function is only meant to be called by the AssetManager,
	 * calls from anywhere else will conflict and lose data.
	 * @see UE::AssetRegistry::FSetManageReferencesContext for arguments.
	 */
	virtual void SetManageReferences(UE::AssetRegistry::FSetManageReferencesContext& Context) = 0;

	/**
	 * Sets the PrimaryAssetId for a specific asset. This should only be called by the AssetManager, and is needed
	 * when the AssetManager is more up to date than the on disk Registry.
	 */
	virtual bool SetPrimaryAssetIdForObjectPath(const FSoftObjectPath& ObjectPath, FPrimaryAssetId PrimaryAssetId) = 0;

	// Functions specifically for calling from cooker's FAssetRegistryGenerator
	friend class FAssetRegistryGenerator;

	/**
	 * Copy all manage dependencies in the global AssetRegistry that target any assets that exist in InOutState into
	 * the manage dependencies of InOutState.
	 */
	virtual void CopyManageDependenciesToState(FAssetRegistryState& InOutState) const = 0;

};

namespace UE::AssetRegistry
{
	enum EReadPackageDataMainErrorCode
	{
		Unknown = 0,
		InvalidObjectCount = 1,
		InvalidTagCount = 2,
		InvalidTag = 3,
	};

	struct FDeserializePackageData
	{
		int64 DependencyDataOffset = INDEX_NONE;
		int32 ObjectCount = 0;

		ASSETREGISTRY_API bool DoSerialize(FArchive& BinaryArchive, const FPackageFileSummary& PackageFileSummary,
			EReadPackageDataMainErrorCode& OutError);
	};

	struct FDeserializeObjectPackageData
	{
		FString ObjectPath;
		FString ObjectClassName;
		int32 TagCount = 0;

		ASSETREGISTRY_API bool DoSerialize(FArchive& BinaryArchive, EReadPackageDataMainErrorCode& OutError);
	};

	struct FDeserializeTagData
	{
		FString Key;
		FString Value;

		ASSETREGISTRY_API bool DoSerialize(FArchive& BinaryArchive, EReadPackageDataMainErrorCode& OutError);
	};

	// Functions to read and write the data used by the AssetRegistry in each package; the format of this data is
	// separate from the format of the data in the asset registry.
	// WritePackageData is declared in AssetRegistryInterface.h, in the CoreUObject module, because it is needed by
	// SavePackage in CoreUObject
	ASSETREGISTRY_API bool ReadPackageDataMain(FArchive& BinaryArchive, const FString& PackageName,
		const FPackageFileSummary& PackageFileSummary, int64& OutDependencyDataOffset,
		TArray<FAssetData*>& OutAssetDataList, EReadPackageDataMainErrorCode& OutError,
		const TArray<FObjectImport>* InImports = nullptr, const TArray<FObjectExport>* InExports = nullptr);
	struct FReadPackageDataDependenciesArgs
	{
		// Required inputs, must be initialized and non-null
		FArchive* BinaryNameAwareArchive = nullptr;
		int64 AssetRegistryDependencyDataOffset = -1;
		int32 NumImports = -1;
		int32 NumSoftPackageReferences = -1;
		FPackageFileVersion PackageVersion;

		// Outputs
		TBitArray<> ImportUsedInGame;
		TBitArray<> SoftPackageUsedInGame;
		TArray<TPair<FName, EExtraDependencyFlags>> ExtraPackageDependencies;
		int64 AssetRegistryDependencyDataSize = 0;
	};
	ASSETREGISTRY_API bool ReadPackageDataDependencies(FReadPackageDataDependenciesArgs& Args);

	/**
	 * Given a list of packages, gather the most important assets for each package.
	 * If multiple assets are in a package, the most important asset will be added.
	 * If a package does not exist or does not have any assets, no entry will be added for that package name.
	 */
	ASSETREGISTRY_API void GetAssetForPackages(TConstArrayView<FName> PackageNames, TMap<FName,
		FAssetData>& OutPackageToAssetData);

	enum class EGetMostImportantAssetFlags
	{
		None = 0,
		
		/* Returns nullptr if there are multiple top level assets. */
		RequireOneTopLevelAsset = 0x1,

		/* Don't skip AR filtered classes (i.e. BP and BPGC). This is cruicial if you are
		* running in environments where the skip classes can't be initialized (i.e. Programs) as you
		* could get different results in such cases. */
		IgnoreSkipClasses = 0x2
	};

	/**
	 * Given a list of asset datas for a specific package, find an asset considered "most important" or "representative".
	 * This is distinct from a Primary asset, and is used for user facing representation of a package or other cases
	 * where you need to relate information about a package to an asset.
	 * 
	 * Usually there is only 1 asset per package so this is straightforward, however in the multiple asset case it:
	 *  Tries to find the "UAsset" via the FAssetData::IsUAsset() function. (i.e. asset name matches package name)
	 *  If none exist, tries to find a "Top Level Asset" using FAssetData::IsToplevelAsset(). (i.e. outer == package)
	 *  If only one exists, use that.
	 *  Otherwise, if EGetMostImportantAssetFlags::RequireOneTopLevelAsset isn't set, gather the set of possibles and
	 *  return the first sorted on asset class then name.
	 *	    If no top level assets, all package assets
	 *	    If multiple top level assets, all top level assets
	 * 
	 * A good source for PackageAssetDatas is FAssetRegistryState::GetAssetsByPackageName.
	 */
	ASSETREGISTRY_API const FAssetData* GetMostImportantAsset(TConstArrayView<const FAssetData*> PackageAssetDatas,
		EGetMostImportantAssetFlags InFlags = EGetMostImportantAssetFlags::None);

	/*
	* Returns true if the asset registry should start searching all assets on startup
	*/
	ASSETREGISTRY_API bool ShouldSearchAllAssetsAtStart();

	// Wildcards (*) used when looking up assets in the asset registry
	extern ASSETREGISTRY_API const FName WildcardFName;
	extern ASSETREGISTRY_API const FTopLevelAssetPath WildcardPathName;


	/*
	* Various FNames for asset tags that get added during staging if asset registry writeback is enabled.
	* Note that these sizes refer to the package as a whole, and some packages have more than one asset per
	* package. In that case, the data is stored on the asset returned by GetMostImportantAsset with IgnoreSkipClasses.
	* 
	* NOTE some platforms do not compress in UnrealPak. For those platforms, all compressed sizes are actually
	* uncompressed!
	* 
	* Stage_ChunkCountFName				Total iostore chunks in the package.
	* Stage_ChunkSizeFName				The total uncompressed size of the chunks in the package.
	* Stage_ChunkCompressedSizeFName	The total compressed size of the chunks in the package. SEE NOTE ABOVE!
	* Stage_ChunkInstalledSizeFName		The compressed size of all chunks that must be installed with the game/plugin.
	* Stage_ChunkStreamingSizeFName		The compressed size of all chunks that are delivered using IAS.
	* Stage_ChunkOptionalSizeFName		The compressed size of all chunks that are placed in an optional container.
	* 
	* See CookMetadata.h - UE::Cook::EPluginSizeTypes for more information on size types.
	*/
	extern ASSETREGISTRY_API const FName Stage_ChunkCountFName;
	extern ASSETREGISTRY_API const FName Stage_ChunkSizeFName;
	extern ASSETREGISTRY_API const FName Stage_ChunkCompressedSizeFName;
	extern ASSETREGISTRY_API const FName Stage_ChunkInstalledSizeFName;
	extern ASSETREGISTRY_API const FName Stage_ChunkStreamingSizeFName;
	extern ASSETREGISTRY_API const FName Stage_ChunkOptionalSizeFName;
} // namespace UE::AssetRegistry

/** Returns the filename without filepath for the DevelopmentAssetRegistry written by the cooker. */
ASSETREGISTRY_API const TCHAR* GetDevelopmentAssetRegistryFilename();

