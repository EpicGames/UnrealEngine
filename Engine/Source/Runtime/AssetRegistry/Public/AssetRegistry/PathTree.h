// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/Set.h"
#include "HAL/Platform.h"
#include "UObject/NameTypes.h"

template <typename FuncType> class TFunctionRef;

class FPathTree
{
public:
	ASSETREGISTRY_API FPathTree();
	
	/** Reserve extra space for the expected number of paths that will be added in future. 
	 * Count is added to the current number of stored paths and not any previous reservation. */
	ASSETREGISTRY_API void EnsureAdditionalCapacity(int32 NumNewPaths);
	
	/** Adds the specified path to the tree, creating nodes as needed and calling OnPathAdded for any new paths added. Returns true if the specified path was actually added (as opposed to already existed) */
	ASSETREGISTRY_API bool CachePath(FName Path, TFunctionRef<void(FName)> OnPathAdded);

	/** Removes the specified path from the tree, calling OnPathRemoved for any existing paths removed. Returns true if the path was found and removed. */
	ASSETREGISTRY_API bool RemovePath(FName Path, TFunctionRef<void(FName)> OnPathRemoved);

	/** Checks whether the given path is one that we know about */
	ASSETREGISTRY_API bool PathExists(FName Path) const;

	/** Get all of the paths we know about */
	ASSETREGISTRY_API bool GetAllPaths(TSet<FName>& OutPaths) const;

	/** Enumerate all of the paths we know about */
	ASSETREGISTRY_API void EnumerateAllPaths(TFunctionRef<bool(FName)> Callback) const;

	/** Recursively gathers all child paths from the specified base path relative to this node */
	ASSETREGISTRY_API bool GetSubPaths(FName BasePath, TSet<FName>& OutPaths, bool bRecurse = true) const;

	/** Recursively enumerates all child paths from the specified base path relative to this node */
	ASSETREGISTRY_API bool EnumerateSubPaths(FName BasePath, TFunctionRef<bool(FName)> Callback, bool bRecurse = true) const;

	/** Shrinks the internal structures reducing the memory they use. */
	ASSETREGISTRY_API void Shrink();

	int32 NumPaths() const
	{
		return ParentPathToChildPaths.Num();
	}

	SIZE_T GetAllocatedSize(void) const
	{
		SIZE_T AllocatedSize = ParentPathToChildPaths.GetAllocatedSize() + ChildPathToParentPath.GetAllocatedSize();

		for (const TPair<FName, TSet<FName>>& Pair : ParentPathToChildPaths)
		{
			AllocatedSize += Pair.Value.GetAllocatedSize();
		}

		return AllocatedSize;
	}

	/** Normalize the given PackagePath (/Game/SubDir) by removing a trailing slash if it exists. */
	static ASSETREGISTRY_API FName NormalizePackagePath(FName In);

private:
	/** A one-to-many mapping between a parent path and its child paths. */
	TMap<FName, TSet<FName>> ParentPathToChildPaths;

	/** A one-to-one mapping between a child path and its parent path. Paths without a parent (root paths) will not appear in this map. */
	TMap<FName, FName> ChildPathToParentPath;
};
