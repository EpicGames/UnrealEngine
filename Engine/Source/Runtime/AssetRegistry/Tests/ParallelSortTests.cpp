// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "Algo/IsSorted.h"
#include "AssetRegistry/ParallelSort.h"
#include "TestHarness.h"
#include "TestMacros/Assertions.h"

TEST_CASE("Algo::ParallelSort")
{
	struct FKeyAndValue
	{
		int32 Key;
		int32 Value;
	};
	auto CompareKeyAndValueKey = [](const FKeyAndValue& A, const FKeyAndValue& B)
		{
			return A.Key < B.Key;
		};
	auto CompareKeyAndValueFull = [](const FKeyAndValue& A, const FKeyAndValue& B)
		{
			int32 Diff = A.Key - B.Key;
			if (Diff != 0)
			{
				return Diff < 0;
			}
			return A.Value < B.Value;
		};

	// Small test cases; these do not test parallel code because it falls back to single threaded for small cases.
	{
		TArray<int32> TestCase({ 1,2,3,4,5 });
		Algo::ParallelSort(TestCase);
		CHECK(Algo::IsSorted(TestCase));
	}
	{
		TArray<int32> TestCase({ 5,4,3,2,1 });
		Algo::ParallelSort(TestCase);
		CHECK(Algo::IsSorted(TestCase));
	}
	{
		TArray<int32> TestCase({ 1,5,2,4,3 });
		Algo::ParallelSort(TestCase);
		CHECK(Algo::IsSorted(TestCase));
	}
	{
		// Verify the sort is stable
		TArray<FKeyAndValue> TestCase({ { 1,1 }, { 2,2 }, { 3,3 }, { 4,4 }, { 5,5 },
			{ 1,6 }, { 2,7 }, { 3,8 }, { 4,9 }, { 5,10 } });
		Algo::ParallelSort(TestCase, CompareKeyAndValueKey);
		CHECK(Algo::IsSorted(TestCase, CompareKeyAndValueFull));
	}


	// Test cases large enough to test the parallel code
	{
		// Already sorted
		TArray<int32> TestCase;
		TestCase.SetNum(1024);
		for (int32 n = 0; n < TestCase.Num(); ++n)
		{
			TestCase[n] = n;
		}
		Algo::ParallelSort(TestCase);
		CHECK(Algo::IsSorted(TestCase));
	}
	{
		// Backwards sorted
		TArray<int32> TestCase;
		TestCase.SetNum(1024);
		for (int32 n = 0; n < TestCase.Num(); ++n)
		{
			TestCase[n] = -n;
		}
		Algo::ParallelSort(TestCase);
		CHECK(Algo::IsSorted(TestCase));
	}
	{
		// Parabola with origin halfway through list
		TArray<int32> TestCase;
		TestCase.SetNum(1024);
		for (int32 n = 0; n < TestCase.Num(); ++n)
		{
			int32 Recentered = n - TestCase.Num() / 2;
			TestCase[n] = Recentered*Recentered;
		}
		Algo::ParallelSort(TestCase);
		CHECK(Algo::IsSorted(TestCase));
	}
	{
		// Saw tooth with small period
		TArray<int32> TestCase;
		TestCase.SetNum(1024);
		for (int32 n = 0; n < TestCase.Num(); ++n)
		{
			TestCase[n] = n % 5;
		}
		Algo::ParallelSort(TestCase);
		CHECK(Algo::IsSorted(TestCase));
	}
	{
		// Saw tooth with big period
		TArray<int32> TestCase;
		TestCase.SetNum(1024);
		for (int32 n = 0; n < TestCase.Num(); ++n)
		{
			TestCase[n] = n % 100;
		}
		Algo::ParallelSort(TestCase);
		CHECK(Algo::IsSorted(TestCase));
	}
	{
		// Verify the sort is stable using a saw tooth with small period
		TArray<FKeyAndValue> TestCase;
		TestCase.SetNum(1024);
		for (int32 n = 0; n < TestCase.Num(); ++n)
		{
			TestCase[n] = { n % 5, n };
		}
		Algo::ParallelSort(TestCase, CompareKeyAndValueKey);
		CHECK(Algo::IsSorted(TestCase, CompareKeyAndValueFull));
	}
	{
		// Verify that the sort works on every possible size up to and including the second-smallest power
		// of two that uses parallel sort rather than falling back to single-threaded sort.
		for (int32 Size = 0; Size <= 512; ++Size)
		{
			TArray<int32> TestCase;
			TestCase.SetNum(Size);
			// Use a reverse sorted list so that every merge level has work to do and can't early exit.
			for (int32 n = 0; n < TestCase.Num(); ++n)
			{
				TestCase[n] = -n;
			}
			Algo::ParallelSort(TestCase);
			CHECK(Algo::IsSorted(TestCase));
		}
	}


}

#endif