// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "TestHarness.h"
#include "TestMacros/Assertions.h"
#include "AssetRegistry/PathTree.h"

TEST_CASE("UE::AssetRegistry::PathTree")
{
    FPathTree Tree;
    TArray<FName> NewPaths;
    auto Callback = [&NewPaths](FName NewPath) { NewPaths.Add(NewPath); };
    Tree.CachePath(FName("/Game/Maps/Arena"), Callback);
    Tree.CachePath(FName("/Game/Maps/Town"), Callback);
    Tree.CachePath(FName("/Game/Characters/Knight"), Callback);
    
    SECTION("NewPathCallbacks")
    {    
        // No duplicates should be added from successive operations
        CHECK(NewPaths.Num() == (3 + 1 + 2));
        CHECK(NewPaths.Contains(FName("/Game")));
        CHECK(NewPaths.Contains(FName("/Game/Maps")));
        CHECK(NewPaths.Contains(FName("/Game/Maps/Arena")));

        CHECK(NewPaths.Contains(FName("/Game/Maps/Town")));
        
        CHECK(NewPaths.Contains(FName("/Game/Characters")));
        CHECK(NewPaths.Contains(FName("/Game/Characters/Knight")));
    }
    
    SECTION("AllPaths")
    {
        TSet<FName> AllPaths;
        Tree.GetAllPaths(AllPaths);

        CHECK(AllPaths.Num() == 7);        
        CHECK(AllPaths.Contains(FName("/")));
        CHECK(AllPaths.Contains(FName("/Game")));
        CHECK(AllPaths.Contains(FName("/Game/Maps")));
        CHECK(AllPaths.Contains(FName("/Game/Maps/Arena")));
        CHECK(AllPaths.Contains(FName("/Game/Maps/Town")));
        CHECK(AllPaths.Contains(FName("/Game/Characters")));
        CHECK(AllPaths.Contains(FName("/Game/Characters/Knight")));
    }
    
    SECTION("SubPaths")
    {
        TSet<FName> MapPaths;
        Tree.GetSubPaths(FName("/Game/Maps"), MapPaths);

        CHECK(MapPaths.Num() == 2);
        CHECK(MapPaths.Contains(FName("/Game/Maps/Arena")));
        CHECK(MapPaths.Contains(FName("/Game/Maps/Town")));
    }

    SECTION("Remove")
    {
        TSet<FName> RemovedPaths;
        Tree.RemovePath(FName("/Game/Maps"), [&RemovedPaths](FName Path) { RemovedPaths.Add(Path); });
        
        CHECK(RemovedPaths.Num() == 3);
        CHECK(RemovedPaths.Contains("/Game/Maps"));
        CHECK(RemovedPaths.Contains("/Game/Maps/Arena"));
        CHECK(RemovedPaths.Contains("/Game/Maps/Town"));
        
        TSet<FName> GamePaths;
        Tree.GetSubPaths(FName("/Game"), GamePaths);

        CHECK(GamePaths.Num() == 2);
        CHECK(GamePaths.Contains(FName("/Game/Characters")));
        CHECK(GamePaths.Contains(FName("/Game/Characters/Knight")));
    }
}

#endif